[CoreRedirects]
+ClassRedirects=(OldName="/Script/ModelViewViewModelBlueprint.MVVMViewBlueprintListViewBaseExtension", NewName="/Script/ModelViewViewModelBlueprint.MVVMBlueprintViewExtension_ListViewBase")
+ClassRedirects=(OldName="/Script/ModelViewViewModelBlueprint.MVVMViewListViewBaseExtension", NewName="/Script/ModelViewViewModelBlueprint.MVVMViewListViewBaseClassExtension")
+ClassRedirects=(OldName="/Script/ModelViewViewModelEditor.MVVMK2Node_MakeBrushFromSoftMaterial", NewName="/Script/ModelViewViewModelBlueprint.MVVMK2Node_MakeBrushFromSoftMaterial")
+ClassRedirects=(OldName="/Script/ModelViewViewModelEditor.MVVMK2Node_MakeBrushFromSoftResource", NewName="/Script/ModelViewViewModelBlueprint.MVVMK2Node_MakeBrushFromSoftResource")
+ClassRedirects=(OldName="/Script/ModelViewViewModelEditor.MVVMK2Node_MakeBrushFromSoftTexture", NewName="/Script/ModelViewViewModelBlueprint.MVVMK2Node_MakeBrushFromSoftTexture")
+ClassRedirects=(OldName="/Script/ModelViewViewModelEditor.MVVMK2Node_LoadSoftResource", NewName="/Script/ModelViewViewModelBlueprint.MVVMK2Node_LoadSoftResource")
+ClassRedirects=(OldName="/Script/ModelViewViewModelEditor.MVVMK2Node_LoadSoftTexture", NewName="/Script/ModelViewViewModelBlueprint.MVVMK2Node_LoadSoftTexture")
+ClassRedirects=(OldName="/Script/ModelViewViewModelEditor.MVVMK2Node_LoadSoftMaterial", NewName="/Script/ModelViewViewModelBlueprint.MVVMK2Node_LoadSoftMaterial")
+ClassRedirects=(OldName="/Script/ModelViewViewModelEditor.MVVMK2Node_LoadSoftInputAction", NewName="/Script/ModelViewViewModelBlueprint.MVVMK2Node_LoadSoftInputAction")
+ClassRedirects=(OldName="/Script/ModelViewViewModelEditor.MVVMK2Node_SetSoftTextureParameter", NewName="/Script/ModelViewViewModelBlueprint.MVVMK2Node_SetSoftTextureParameter")
+StructRedirects=(OldName="MVVMViewModelPropertyPath",NewName="/Script/ModelViewViewModelBlueprint.MVVMBlueprintPropertyPath")
+StructRedirects=(OldName="MVVMWidgetPropertyPath",NewName="/Script/ModelViewViewModelBlueprint.MVVMBlueprintPropertyPath")
+FunctionRedirects=(OldName="/Script/ModelViewViewModel.MVVMSubsystem.GetViewFromUserWidget", NewName="/Script/ModelViewViewModel.MVVMSubsystem.K2_GetViewFromUserWidget")
+FunctionRedirects=(OldName="/Script/ModelViewViewModel.MVVMSubsystem.GetAvailableBindings", NewName="/Script/ModelViewViewModel.MVVMSubsystem.K2_GetAvailableBindings")
+FunctionRedirects=(OldName="/Script/ModelViewViewModel.MVVMSubsystem.GetAvailableBinding", NewName="/Script/ModelViewViewModel.MVVMSubsystem.K2_GetAvailableBinding")
+PropertyRedirects=(OldName="/Script/ModelViewViewModelBlueprint.MVVMBlueprintViewBinding.ViewModelPath",NewName="SourcePath")
+PropertyRedirects=(OldName="/Script/ModelViewViewModelBlueprint.MVVMBlueprintViewBinding.WidgetPath",NewName="DestinationPath")

[/Script/ModelViewViewModelBlueprint.MVVMDeveloperProjectSettings]
+FieldSelectorPermissions=(("/Script/UMG.Widget", (DisallowedFieldNames=("TooltipWidget","ForceVolatile","AddExtension", "SetUserFocus","Navigation","GetParent","GetOwningLocalPlayer","GetGameInstance","BindToAnimationEvent","BindToAnimationFinished","BindToAnimationStarted","UnbindFromAnimationFinished","UnbindFromAnimationStarted","UnbindAllFromAnimationFinished","UnbindAllFromAnimationStarted","CancelLatentActions","FlushAnimations","ForceLayoutPrepass","InvalidateLayoutAndVolatility","K2_AddFieldValueChangedDelegate","ListenForInputAction","StopListeningForInputAction","StopListeningForAllInputActions","RemoveExtension","RemoveExtensions","SetAllNavigationRules","SetNavigationRuleBase","SetNavigationRuleCustom","SetNavigationRuleCustomBoundary","SetNavigationRuleExplicit","SetOwningPlayer"),AdvancedFieldNames=())), ("/Script/ModelViewViewModel.MVVMViewModelBase", (DisallowedFieldNames=("K2_AddFieldValueChangedDelegate","K2_RemoveFieldValueChangedDelegate"),AdvancedFieldNames=())))