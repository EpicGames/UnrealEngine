{
	"FileVersion": 3,

	"FriendlyName": "UMG Viewmodel",
	"Version": 1,
	"VersionName": "1.0",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Description": "A plugin to support the Model-View-Viewmodel pattern in UMG.",
	"Category": "UI",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,

	"Modules": [
		{
			"Name": "ModelViewViewModel",
			"Type": "Runtime",
			"LoadingPhase": "Default"

		},
		{
			"Name": "ModelViewViewModelBlueprint",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelViewViewModelEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelViewViewModelDebugger",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelViewViewModelDebuggerEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],

	"Plugins": [
		{
			"Name": "EnhancedInput",
			"Enabled": true
		},
	]
}

