// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using UnrealBuildTool;

public class ModelViewViewModel : ModuleRules 
{
	public ModelViewViewModel(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"FieldNotification",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"SlateCore",
				"Slate",
				"UMG",
			});

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"UnrealEd",
			});
		}

		if (Target.Configuration == UnrealTargetConfiguration.Test || Target.Configuration == UnrealTargetConfiguration.Shipping)
		{
			PublicDefinitions.Add("UE_WITH_MVVM_DEBUGGING=0");
		}
		else
		{
			PublicDefinitions.Add("UE_WITH_MVVM_DEBUGGING=1");
		}
	}
}
