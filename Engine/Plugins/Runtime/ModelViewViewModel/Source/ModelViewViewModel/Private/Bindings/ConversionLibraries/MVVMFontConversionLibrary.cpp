// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/ConversionLibraries/MVVMFontConversionLibrary.h"

#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMFontConversionLibrary)

FSlateFontInfo UMVVMFontConversionLibrary::Conv_SetScalarParameter(FSlateFontInfo Font, FName ParameterName, float Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Font, nullptr))
	{
		DynamicMaterial->SetScalarParameterValue(ParameterName, Value);
	}

	return Font;
}

FSlateFontInfo UMVVMFontConversionLibrary::Conv_SetVectorParameter(FSlateFontInfo Font, FName ParameterName, FLinearColor Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Font, nullptr))
	{
		DynamicMaterial->SetVectorParameterValue(ParameterName, Value);
	}

	return Font;
}

FSlateFontInfo UMVVMFontConversionLibrary::Conv_SetVectorParameter_FColor(FSlateFontInfo Font, FName ParameterName, FColor Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Font, nullptr))
	{
		FLinearColor Color(Value);
		DynamicMaterial->SetVectorParameterValue(ParameterName, Color);
	}

	return Font;
}

FSlateFontInfo UMVVMFontConversionLibrary::Conv_SetTextureParameter(FSlateFontInfo Font, FName ParameterName, UTexture* Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Font, nullptr))
	{
		DynamicMaterial->SetTextureParameterValue(ParameterName, Value);
	}

	return Font;
}

FSlateFontInfo UMVVMFontConversionLibrary::Conv_SetScalarParameterMID(FSlateFontInfo Font, UMaterialInterface* Material, FName ParameterName, float Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Font, nullptr, Material))
	{
		DynamicMaterial->SetScalarParameterValue(ParameterName, Value);
	}

	return Font;
}

FSlateFontInfo UMVVMFontConversionLibrary::Conv_SetVectorParameterMID(FSlateFontInfo Font, UMaterialInterface* Material, FName ParameterName, FLinearColor Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Font, nullptr, Material))
	{
		DynamicMaterial->SetVectorParameterValue(ParameterName, Value);
	}

	return Font;
}

FSlateFontInfo UMVVMFontConversionLibrary::Conv_SetVectorParameterMID_FColor(FSlateFontInfo Font, UMaterialInterface* Material, FName ParameterName, FColor Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Font, nullptr, Material))
	{
		FLinearColor Color(Value);
		DynamicMaterial->SetVectorParameterValue(ParameterName, Color);
	}

	return Font;
}

FSlateFontInfo UMVVMFontConversionLibrary::Conv_SetTextureParameterMID(FSlateFontInfo Font, UMaterialInterface* Material, FName ParameterName, UTexture* Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Font, nullptr, Material))
	{
		DynamicMaterial->SetTextureParameterValue(ParameterName, Value);
	}

	return Font;
}

UMaterialInstanceDynamic* UMVVMFontConversionLibrary::TryGetDynamicMaterial(FSlateFontInfo& InFont, UObject* InOuter, UMaterialInterface* InTargetMaterial)
{
	 UMaterialInterface* SourceMaterial = Cast<UMaterialInterface>(InFont.FontMaterial);

	 // Want to either make the provided material if different, or make a MID of the existing material
	 UMaterialInterface* DesiredMaterial = nullptr;
	 if (InTargetMaterial)
	 {
		 DesiredMaterial = InTargetMaterial;
	 }
	 else
	 {
		 DesiredMaterial = SourceMaterial;
	 }

	 if (DesiredMaterial)
	 {
	 	UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(DesiredMaterial);

	 	if ( !DynamicMaterial )
	 	{
			DynamicMaterial = UMaterialInstanceDynamic::Create(DesiredMaterial, InOuter);
			InFont.FontMaterial = DynamicMaterial;
	 	}
	 	return DynamicMaterial;
	 }

	return nullptr;
}
