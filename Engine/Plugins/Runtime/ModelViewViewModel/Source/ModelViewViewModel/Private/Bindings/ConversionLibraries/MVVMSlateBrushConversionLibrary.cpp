// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/ConversionLibraries/MVVMSlateBrushConversionLibrary.h"

#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMSlateBrushConversionLibrary)

FSlateBrush UMVVMSlateBrushConversionLibrary::Conv_SetScalarParameter(FSlateBrush Brush, FName ParameterName, float Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Brush, nullptr))
	{
		DynamicMaterial->SetScalarParameterValue(ParameterName, Value);
	}

	return Brush;
}

FSlateBrush UMVVMSlateBrushConversionLibrary::Conv_SetVectorParameter(FSlateBrush Brush, FName ParameterName, FLinearColor Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Brush, nullptr))
	{
		DynamicMaterial->SetVectorParameterValue(ParameterName, Value);
	}

	return Brush;
}

FSlateBrush UMVVMSlateBrushConversionLibrary::Conv_SetVectorParameter_FColor(FSlateBrush Brush, FName ParameterName, FColor Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Brush, nullptr))
	{
		FLinearColor Color(Value);
		DynamicMaterial->SetVectorParameterValue(ParameterName, Color);
	}

	return Brush;
}

FSlateBrush UMVVMSlateBrushConversionLibrary::Conv_SetTextureParameter(FSlateBrush Brush, FName ParameterName, UTexture* Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Brush, nullptr))
	{
		DynamicMaterial->SetTextureParameterValue(ParameterName, Value);
	}

	return Brush;
}

FSlateBrush UMVVMSlateBrushConversionLibrary::Conv_SetScalarParameterMID(FSlateBrush Brush, UMaterialInterface* Material, FName ParameterName, float Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Brush, nullptr, Material))
	{
		DynamicMaterial->SetScalarParameterValue(ParameterName, Value);
	}

	return Brush;
}

FSlateBrush UMVVMSlateBrushConversionLibrary::Conv_SetVectorParameterMID(FSlateBrush Brush, UMaterialInterface* Material, FName ParameterName, FLinearColor Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Brush, nullptr, Material))
	{
		DynamicMaterial->SetVectorParameterValue(ParameterName, Value);
	}

	return Brush;
}

FSlateBrush UMVVMSlateBrushConversionLibrary::Conv_SetVectorParameterMID_FColor(FSlateBrush Brush, UMaterialInterface* Material, FName ParameterName, FColor Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Brush, nullptr, Material))
	{
		FLinearColor Color(Value);
		DynamicMaterial->SetVectorParameterValue(ParameterName, Color);
	}

	return Brush;
}

FSlateBrush UMVVMSlateBrushConversionLibrary::Conv_SetTextureParameterMID(FSlateBrush Brush, UMaterialInterface* Material, FName ParameterName, UTexture* Value)
{
	if (UMaterialInstanceDynamic* DynamicMaterial = TryGetDynamicMaterial(Brush, nullptr, Material))
	{
		DynamicMaterial->SetTextureParameterValue(ParameterName, Value);
	}

	return Brush;
}

UMaterialInstanceDynamic* UMVVMSlateBrushConversionLibrary::TryGetDynamicMaterial(FSlateBrush& InBrush, UObject* InOuter, UMaterialInterface* InTargetMaterial)
{
	 UMaterialInterface* SourceMaterial = Cast<UMaterialInterface>(InBrush.GetResourceObject());

	 // Want to either make the provided material if different, or make a MID of the existing material
	 UMaterialInterface* DesiredMaterial = nullptr;
	 if (InTargetMaterial)
	 {
		 DesiredMaterial = InTargetMaterial;
	 }
	 else
	 {
		 DesiredMaterial = SourceMaterial;
	 }

	 if (DesiredMaterial)
	 {
	 	UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(DesiredMaterial);

	 	if ( !DynamicMaterial )
	 	{
	 		DynamicMaterial = UMaterialInstanceDynamic::Create(DesiredMaterial, InOuter);
			InBrush.SetResourceObject(DynamicMaterial);
	 	}
	 	return DynamicMaterial;
	 }

	return nullptr;
}
