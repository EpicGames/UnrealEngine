// Copyright Epic Games, Inc. All Rights Reserved.


#include "Bindings/ImplicitConverters/MVVMNumericImplicitConverter.h"

#include "UObject/UnrealType.h"

#ifndef UE_MVVM_ALLOW_AUTO_INTEGRAL_CONVERSION
#define UE_MVVM_ALLOW_AUTO_INTEGRAL_CONVERSION 1
#endif

bool FMVVMNumericImplicitConverter::CanConvertImplicitly(const FProperty* Source, const FProperty* Destination)
{
	const FNumericProperty* SourceNumericProperty = CastField<const FNumericProperty>(Source);
	const FNumericProperty* DestinationNumericProperty = CastField<const FNumericProperty>(Destination);
	if (SourceNumericProperty && DestinationNumericProperty)
	{
		const bool bSameType = Destination->SameType(Source);
		const bool bBothFloatingPoint = SourceNumericProperty->IsFloatingPoint() && DestinationNumericProperty->IsFloatingPoint();
#if UE_MVVM_ALLOW_AUTO_INTEGRAL_CONVERSION
		const bool bBothIntegral = SourceNumericProperty->IsInteger() && DestinationNumericProperty->IsInteger();
		const bool bOneIsEnum = SourceNumericProperty->IsEnum() || DestinationNumericProperty->IsEnum();
		return !bSameType && (bBothFloatingPoint || (bBothIntegral && !bOneIsEnum));
#else
		return !bSameType && bBothFloatingPoint;
#endif
	}

	return false;
}

void FMVVMNumericImplicitConverter::ConvertImplicitly(const FProperty* Source, const FProperty* Destination, void* Data)
{
	const FNumericProperty* SourceNumericProperty = CastField<const FNumericProperty>(Source);
	const FNumericProperty* DestinationNumericProperty = CastField<const FNumericProperty>(Destination);
	if (SourceNumericProperty && DestinationNumericProperty)
	{
		if (SourceNumericProperty->IsFloatingPoint() && DestinationNumericProperty->IsFloatingPoint())
		{
			//floating to floating
			const void* SrcElemValue = static_cast<const uint8*>(Data);
			void* DestElemValue = static_cast<uint8*>(Data);

			const double Value = SourceNumericProperty->GetFloatingPointPropertyValue(SrcElemValue);
			DestinationNumericProperty->SetFloatingPointPropertyValue(DestElemValue, Value);
		}
#if UE_MVVM_ALLOW_AUTO_INTEGRAL_CONVERSION
		else if (SourceNumericProperty->IsInteger() && DestinationNumericProperty->IsInteger())
		{
			//integral to integral
			const void* SrcElemValue = static_cast<const uint8*>(Data);
			void* DestElemValue = static_cast<uint8*>(Data);

			const int64 Value = SourceNumericProperty->GetSignedIntPropertyValue(SrcElemValue);
			DestinationNumericProperty->SetIntPropertyValue(DestElemValue, Value);
		}
#endif
	}
}
