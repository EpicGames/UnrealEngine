// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MVVMBindingHelper.h"

#include "Bindings/ImplicitConverters/MVVMImplicitConverter.h"
#include "Blueprint/UserWidget.h"
#include "ModelViewViewModelModule.h"
#include "Modules/ModuleManager.h"
#include "Templates/ValueOrError.h"
#include "UObject/PropertyAccessUtil.h"
#include "Types/MVVMBindingName.h"
#include "Types/MVVMFieldContext.h"
#include "Types/MVVMFunctionContext.h"

#define LOCTEXT_NAMESPACE "MVVMBindingHelper"

namespace UE::MVVM::BindingHelper
{
	namespace Private
	{
		static const FName NAME_BlueprintPrivate = "BlueprintPrivate";
		static const FName NAME_DeprecatedFunction = "DeprecatedFunction";
		static const FName NAME_BlueprintGetter = "BlueprintGetter";
		static const FName NAME_BlueprintSetter = "BlueprintSetter";


		bool IsValidCommon(const UFunction* InFunction)
		{
			bool bResult = InFunction != nullptr
				&& !InFunction->HasAnyFunctionFlags(FUNC_Net | FUNC_Event | FUNC_EditorOnly)
				&& InFunction->HasAllFunctionFlags(FUNC_BlueprintCallable);

#if WITH_EDITOR
			if (bResult)
			{
				if (InFunction->HasMetaData(NAME_DeprecatedFunction))
				{
					bResult = false;
				}
			}
#endif

			return bResult;
		}
	} // namespace

	// CPF_Protected is not used
	bool IsValidForSourceBinding(const FProperty* InProperty)
	{
		return InProperty != nullptr
			&& InProperty->ArrayDim == 1
			&& !InProperty->HasAnyPropertyFlags(CPF_Deprecated | CPF_EditorOnly)
			&& InProperty->HasAnyPropertyFlags(CPF_BlueprintVisible | CPF_BlueprintAssignable);
	}


	bool IsValidForDestinationBinding(const FProperty* InProperty)
	{
		return InProperty != nullptr
			&& InProperty->ArrayDim == 1
			&& !InProperty->HasAnyPropertyFlags(CPF_Deprecated | CPF_EditorOnly | CPF_BlueprintReadOnly)
			&& InProperty->HasAnyPropertyFlags(CPF_BlueprintVisible | CPF_BlueprintAssignable);
	}


	bool IsValidForSourceBinding(const UFunction* InFunction)
	{
		//UE::FieldNotification::Helpers::IsValidAsField(InFunction)
		return Private::IsValidCommon(InFunction)
			&& InFunction->HasAllFunctionFlags(FUNC_Const)
			&& InFunction->NumParms == 1
			&& GetReturnProperty(InFunction) != nullptr;
	}


	bool IsValidForDestinationBinding(const UFunction* InFunction)
	{
		// A setter can return a value "bool SetValue(int32)" but is not valid for MVVM
		//&& ((InFunction->NumParms == 1 && GetReturnProperty() == nullptr) || (InFunction->NumParms == 2 && GetReturnProperty() != nullptr));

		return Private::IsValidCommon(InFunction)
			&& !InFunction->HasAnyFunctionFlags(FUNC_Const | FUNC_BlueprintPure)
			&& InFunction->NumParms == 1
			&& GetFirstArgumentProperty(InFunction) != nullptr;
	}


	bool IsValidForSourceBinding(const FMVVMConstFieldVariant InVariant)
	{
		if (InVariant.IsProperty())
		{
			return IsValidForSourceBinding(InVariant.GetProperty());
		}
		else if (InVariant.IsFunction())
		{
			return IsValidForSourceBinding(InVariant.GetFunction());
		}
		return false;
	}


	bool IsValidForDestinationBinding(const FMVVMConstFieldVariant InVariant)
	{
		if (InVariant.IsProperty())
		{
			return IsValidForDestinationBinding(InVariant.GetProperty());
		}
		else if (InVariant.IsFunction())
		{
			return IsValidForDestinationBinding(InVariant.GetFunction());
		}
		return false;
	}


	bool IsValidForEventBinding(const FMVVMConstFieldVariant InVariant)
	{
		if (InVariant.IsProperty())
		{
			// Same rules as other bindings apply for event binding to properties.
			return IsValidForDestinationBinding(InVariant.GetProperty());
		}
		else if (InVariant.IsFunction())
		{
			return IsValidForEventBinding(InVariant.GetFunction());
		}
		return false;
	}


	bool IsValidForSimpleRuntimeConversion(const UFunction* InFunction)
	{
		if (Private::IsValidCommon(InFunction)
			&& (InFunction->HasAllFunctionFlags(FUNC_Static) || InFunction->HasAllFunctionFlags(FUNC_Const | FUNC_BlueprintPure))
			&& InFunction->NumParms <= 2)
		{
			const FProperty* ReturnProperty = GetReturnProperty(InFunction);
			const FProperty* FirstArgumentProperty = GetFirstArgumentProperty(InFunction);
			return ReturnProperty && FirstArgumentProperty && ReturnProperty != FirstArgumentProperty;
		}
		return false;
	}
	
	
	bool IsValidForComplexRuntimeConversion(const UFunction* InFunction)
	{
		if (Private::IsValidCommon(InFunction)
			&& (InFunction->HasAllFunctionFlags(FUNC_Static) || InFunction->HasAllFunctionFlags(FUNC_Const | FUNC_BlueprintPure))
			&& InFunction->NumParms <= 1)
		{
			const FProperty* ReturnProperty = GetReturnProperty(InFunction);
			const FProperty* FirstArgumentProperty = GetFirstArgumentProperty(InFunction);
			return ReturnProperty && FirstArgumentProperty == nullptr;
		}
		return false;
	}


	bool IsValidForDelegateSignatureBinding(const UFunction* InFunction)
	{
		return Private::IsValidCommon(InFunction) && InFunction->HasAnyFunctionFlags(FUNC_Delegate | FUNC_BlueprintEvent);
	}


	bool IsValidForEventBinding(const UFunction* InFunction)
	{
		return Private::IsValidCommon(InFunction) && !InFunction->HasAnyFunctionFlags(FUNC_Const | FUNC_BlueprintPure);
	}


#if WITH_EDITOR
	bool IsAccessibleDirectlyForSourceBinding(const FProperty* InProperty)
	{
		return IsValidForSourceBinding(InProperty) && !InProperty->GetBoolMetaData(Private::NAME_BlueprintPrivate);
	}


	bool IsAccessibleDirectlyForDestinationBinding(const FProperty* InProperty)
	{
		return IsValidForDestinationBinding(InProperty) && !InProperty->GetBoolMetaData(Private::NAME_BlueprintPrivate);
	}


	bool IsAccessibleWithGetterForSourceBinding(const FProperty* InProperty)
	{
		return IsValidForSourceBinding(InProperty) && InProperty->HasMetaData(Private::NAME_BlueprintGetter);
	}


	bool IsAccessibleWithSetterForDestinationBinding(const FProperty* InProperty)
	{
		return IsValidForSourceBinding(InProperty) && InProperty->HasMetaData(Private::NAME_BlueprintSetter);
	}
#endif //WITH_EDITOR

	FName GetDelegateSignatureName(FName InGraphName)
	{
		TStringBuilder<512> StringBuilder;
		StringBuilder << InGraphName.ToString();
		StringBuilder << HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX;
		return FName(StringBuilder.ToString());
	}

	FMVVMFieldVariant FindFieldByName(const UStruct* Container, FMVVMBindingName BindingName)
	{
		if (Container)
		{
			if (const UClass* Class = Cast<const UClass>(Container))
			{
				if (UFunction* Function = Class->FindFunctionByName(BindingName.ToName()))
				{
					return FMVVMFieldVariant(Function);
				}
			}
			if (FProperty* Property = PropertyAccessUtil::FindPropertyByName(BindingName.ToName(), Container))
			{
				return FMVVMFieldVariant(Property);
			}
		}
		return FMVVMFieldVariant();
	}


	namespace Private
	{
		FText TryGetPropertyTypeCommon(const FProperty* InProperty)
		{
			if (InProperty == nullptr)
			{
				return LOCTEXT("PropertyInvalid", "The property is invalid.");
			}
			
			if (InProperty->ArrayDim != 1)
			{
				return FText::Format(LOCTEXT("PropertyStaticArray", "The property '{0}' is a static array."), FText::FromString(InProperty->GetName()));
			}

			if (InProperty->HasAnyPropertyFlags(CPF_Deprecated))
			{
				return FText::Format(LOCTEXT("PropertyDeprecated", "The property '{0}' is deprecated."), FText::FromString(InProperty->GetName()));
			}

			if (InProperty->HasAnyPropertyFlags(CPF_EditorOnly))
			{
				return FText::Format(LOCTEXT("PropertyEditorOnly", "The property '{0}' is only available in the editor."), FText::FromString(InProperty->GetName()));
			}
			
			if (!InProperty->HasAnyPropertyFlags(CPF_BlueprintVisible))
			{
				return FText::Format(LOCTEXT("PropertyNotBlueprintVisible", "The property '{0}' is not visible to the Blueprint script."), FText::FromString(InProperty->GetName()));
			}

			return FText();
		}

		FText TryGetPropertyTypeCommon(const UFunction* InFunction)
		{
			if (InFunction == nullptr)
			{
				return LOCTEXT("FunctionInvalid", "The function is invalid.");
			}

			if (!InFunction->HasAllFunctionFlags(FUNC_BlueprintCallable))
			{
				return FText::Format(LOCTEXT("FunctionNotBlueprintCallable", "The function '{0}' is not BlueprintCallable."), FText::FromString(InFunction->GetName()));
			}

			if (InFunction->HasAllFunctionFlags(FUNC_Net))
			{
				return FText::Format(LOCTEXT("FunctionNetworked", "The function '{0}' is networked."), FText::FromString(InFunction->GetName()));
			}

			if (InFunction->HasAllFunctionFlags(FUNC_Event))
			{

				return FText::Format(LOCTEXT("FunctionEvent", "The function '{0}' is an event."), FText::FromString(InFunction->GetName()));
			}

			if (InFunction->HasAllFunctionFlags(FUNC_EditorOnly))
			{
				return FText::Format(LOCTEXT("FunctionEditorOnly", "The function '{0}' is editor only"), FText::FromString(InFunction->GetName()));
			}

#if WITH_EDITOR
			{
				if (InFunction->HasMetaData(NAME_DeprecatedFunction))
				{
					return FText::Format(LOCTEXT("FunctionDeprecated", "The function '{0}' is deprecated"), FText::FromString(InFunction->GetName()));
				}
			}
#endif

			return FText();
		}
	} // namespace

	TValueOrError<const FProperty*, FText> TryGetPropertyTypeForSourceBinding(const FProperty* Property)
	{
		FText Result = Private::TryGetPropertyTypeCommon(Property);
		if (!Result.IsEmpty())
		{
			return MakeError(MoveTemp(Result));
		}

		return MakeValue(Property);
	}

	TValueOrError<const FProperty*, FText> TryGetPropertyTypeForSourceBinding(const UFunction* Function)
	{
		FText Result = Private::TryGetPropertyTypeCommon(Function);
		if (!Result.IsEmpty())
		{
			return MakeError(MoveTemp(Result));
		}

		if (!Function->HasAllFunctionFlags(FUNC_Const))
		{
			return MakeError(FText::Format(LOCTEXT("FunctionNotConst", "The function '{0}' is not const."), FText::FromString(Function->GetName())));
		}

		//if (!InFunction->HasAllFunctionFlags(FUNC_BlueprintPure))
		//{
		//	return MakeError(FText::Format(LOCTEXT("FunctionNotPure", "The function '{0}' is not pure."), FText::FromString(Function->GetName()));
		//}

		if (Function->NumParms != 1)
		{
			return MakeError(FText::Format(LOCTEXT("FunctionWithoutReturn", "The function '{0}' doesn't have a return value."), FText::FromString(Function->GetName())));
		}

		const FProperty* ReturnProperty = GetReturnProperty(Function);
		if (ReturnProperty == nullptr)
		{
			return MakeError(FText::Format(LOCTEXT("FunctionReturnInvalid", "The return value for function '{0}' is invalid."), FText::FromString(Function->GetName())));
		}

		return MakeValue(ReturnProperty);
	}

	TValueOrError<const FProperty*, FText> TryGetPropertyTypeForSourceBinding(const FMVVMConstFieldVariant& InField)
	{
		if (InField.IsEmpty())
		{
			return MakeError(LOCTEXT("NoSourceForBinding", "No source was provided for the binding."));
		}

		if (InField.IsProperty())
		{
			return TryGetPropertyTypeForSourceBinding(InField.GetProperty());
		}
		else
		{
			check(InField.IsFunction());
			return TryGetPropertyTypeForSourceBinding(InField.GetFunction());
		}
	}

	TValueOrError<const FProperty*, FText> TryGetPropertyTypeForDestinationBinding(const FProperty* Property)
	{
		FText Result = Private::TryGetPropertyTypeCommon(Property);
		if (!Result.IsEmpty())
		{
			return MakeError(MoveTemp(Result));
		}

		if (Property->HasAnyPropertyFlags(CPF_BlueprintReadOnly))
		{
			return MakeError(FText::Format(LOCTEXT("PropertyReadOnly", "The property '{0}' is read only."), FText::FromString(Property->GetName())));
		}

		return MakeValue(Property);
	}

	TValueOrError<const FProperty*, FText> TryGetPropertyTypeForDestinationBinding(const UFunction* Function)
	{
		FText Result = Private::TryGetPropertyTypeCommon(Function);
		if (!Result.IsEmpty())
		{
			return MakeError(MoveTemp(Result));
		}

		if (Function->NumParms != 1)
		{
			return MakeError(FText::Format(LOCTEXT("FunctionTooManyArguments", "The function '{0}' has more than one argument."), FText::FromString(Function->GetName())));
		}

		if (Function->HasAllFunctionFlags(FUNC_Const))
		{
			return MakeError(FText::Format(LOCTEXT("FunctionConst", "The function '{0}' is const."), FText::FromString(Function->GetName())));
		}

		if (Function->HasAllFunctionFlags(FUNC_BlueprintPure))
		{
			return MakeError(FText::Format(LOCTEXT("FunctionPure", "The function '{0}' is pure."), FText::FromString(Function->GetName())));
		}

		const FProperty* ReturnProperty = GetReturnProperty(Function);
		if (ReturnProperty != nullptr)
		{
			return MakeError(FText::Format(LOCTEXT("FunctionHasReturn", "The function '{0}' has a return value."), FText::FromString(Function->GetName())));
		}

		const FProperty* FirstArgumentProperty = GetFirstArgumentProperty(Function);
		if (FirstArgumentProperty == nullptr)
		{
			return MakeError(FText::Format(LOCTEXT("FunctionNoArgument", "The function '{0}' doesn't have a valid (single) argument."), FText::FromString(Function->GetName())));
		}

		return MakeValue(FirstArgumentProperty);
	}

	TValueOrError<const FProperty*, FText> TryGetPropertyTypeForDestinationBinding(const FMVVMConstFieldVariant& InField)
	{
		if (InField.IsEmpty())
		{
			return MakeError(LOCTEXT("NoDestinationForBinding", "No destination was provided for the binding."));
		}

		if (InField.IsProperty())
		{
			return TryGetPropertyTypeForDestinationBinding(InField.GetProperty());
		}
		else
		{
			check(InField.IsFunction());

			return TryGetPropertyTypeForDestinationBinding(InField.GetFunction());
		}
	}

	TValueOrError<const FProperty*, FText> TryGetReturnTypeForConversionFunction(const UFunction* InFunction)
	{
		FText CommonResult = Private::TryGetPropertyTypeCommon(InFunction);
		if (!CommonResult.IsEmpty())
		{
			return MakeError(MoveTemp(CommonResult));
		}

		if (!InFunction->HasAllFunctionFlags(FUNC_Static) && !InFunction->HasAllFunctionFlags(FUNC_Const | FUNC_BlueprintPure))
		{
			return MakeError(FText::Format(LOCTEXT("FunctionFlagsWrong", "The function '{0}' is not static or is not const and pure."), FText::FromString(InFunction->GetName())));
		}

		const FProperty* ReturnProperty = GetReturnProperty(InFunction);
		if (ReturnProperty == nullptr)
		{
			return MakeError(FText::Format(LOCTEXT("FunctionReturnInvalid", "The return value for function '{0}' is invalid."), FText::FromString(InFunction->GetName())));
		}

		return MakeValue(ReturnProperty);
	}

	TValueOrError<TArray<const FProperty*>, FText> TryGetArgumentsForConversionFunction(const UFunction* InFunction)
	{
		FText CommonResult = Private::TryGetPropertyTypeCommon(InFunction);
		if (!CommonResult.IsEmpty())
		{
			return MakeError(MoveTemp(CommonResult));
		}

		if (!InFunction->HasAllFunctionFlags(FUNC_Static) && !InFunction->HasAnyFunctionFlags(FUNC_Const | FUNC_BlueprintPure))
		{
			return MakeError(FText::Format(LOCTEXT("FunctionFlagsWrong", "The function '{0}' is not static or is not const and pure."), FText::FromString(InFunction->GetName())));
		}

		if (InFunction->NumParms < 2)
		{
			return MakeError(FText::Format(LOCTEXT("FunctionWrongNumberOfArgs", "The function '{0}' does not have the correct number of arguments."), FText::FromString(InFunction->GetName())));
		}

		const FProperty* ReturnProperty = GetReturnProperty(InFunction);
		if (ReturnProperty == nullptr)
		{
			return MakeError(FText::Format(LOCTEXT("FunctionReturnInvalid", "The return value for function '{0}' is invalid."), FText::FromString(InFunction->GetName())));
		}

		TArray<const FProperty*> Arguments;
		for (TFieldIterator<FProperty> It(InFunction); It && (It->PropertyFlags & CPF_Parm); ++It)
		{
			const FProperty* Property = *It;
			if (IsValidArgumentProperty(Property))
			{
				Arguments.Add(Property);
			}
		}

		return MakeValue(Arguments);
	}

	namespace Private
	{
		bool IsObjectPropertyCompatible(const FProperty* Source, const FProperty* Destination)
		{
			const FObjectPropertyBase* SourceObjectProperty = CastField<const FObjectPropertyBase>(Source);
			const FObjectPropertyBase* DestinationObjectProperty = CastField<const FObjectPropertyBase>(Destination);
			return SourceObjectProperty
				&& DestinationObjectProperty
				&& SourceObjectProperty->PropertyClass
				&& DestinationObjectProperty->PropertyClass
				&& SourceObjectProperty->PropertyClass->IsChildOf(DestinationObjectProperty->PropertyClass)
				&& (
					SourceObjectProperty->GetClass() == DestinationObjectProperty->GetClass()
					|| SourceObjectProperty->AllowObjectTypeReinterpretationTo(DestinationObjectProperty) // prevents TWeakObjectPtr to UObject*, allow TObjectPtr to UObject*
				);
		}

		bool IsObjectArrayPropertyCompatible(const FProperty* Source, const FProperty* Destination)
		{
			const FArrayProperty* SourceArrayObjectProperty = CastField<const FArrayProperty>(Source);
			const FArrayProperty* DestinationArrayObjectProperty = CastField<const FArrayProperty>(Destination);
			return SourceArrayObjectProperty
				&& DestinationArrayObjectProperty
				&& IsObjectPropertyCompatible(SourceArrayObjectProperty->Inner, DestinationArrayObjectProperty->Inner);
		}


		bool CanConvertImplicitly(const FProperty* Source, const FProperty* Destination)
		{
			const FModelViewViewModelModule& MVVMModule = FModuleManager::GetModuleChecked<FModelViewViewModelModule>("ModelViewViewModel");
			for (const TSharedPtr<FMVVMImplicitConverter>& Converter : MVVMModule.GetImplicitConverters())
			{
				if (Converter.IsValid() && Converter->CanConvertImplicitly(Source, Destination))
				{
					return true;
				}
			}

			return false;
		}


		void ConvertImplicitly(const FProperty* Source, const FProperty* Destination, void* Data)
		{
			check(Source);
			check(Destination);
			check(Data);

			const FModelViewViewModelModule& MVVMModule = FModuleManager::GetModuleChecked<FModelViewViewModelModule>("ModelViewViewModel");
			for (const TSharedPtr<FMVVMImplicitConverter>& Converter : MVVMModule.GetImplicitConverters())
			{
				if (Converter.IsValid() && Converter->CanConvertImplicitly(Source, Destination))
				{
					Converter->ConvertImplicitly(Source, Destination, Data);
					break;
				}
			}
		}
	} // namespace

	bool ArePropertiesCompatible(const FProperty* Source, const FProperty* Destination)
	{
		if (Source == nullptr || Destination == nullptr)
		{
			return false;
		}

		return Source->SameType(Destination)
				|| Private::CanConvertImplicitly(Source, Destination)
				|| Private::IsObjectPropertyCompatible(Source, Destination)
				|| Private::IsObjectArrayPropertyCompatible(Source, Destination);
	}


	const FProperty* GetReturnProperty(const UFunction* InFunction)
	{
		check(InFunction);
		FProperty* Result = InFunction->GetReturnProperty();

		if (InFunction->HasAllFunctionFlags(FUNC_HasOutParms))
		{
			for (TFieldIterator<FProperty> It(InFunction); It && (It->PropertyFlags & CPF_Parm); ++It)
			{
				if (It->HasAllPropertyFlags(CPF_OutParm) && !It->HasAnyPropertyFlags(CPF_ConstParm | CPF_ReferenceParm))
				{
					if (Result == nullptr)
					{
						if (!It->HasAnyPropertyFlags(CPF_Deprecated | CPF_EditorOnly))
						{
							Result = *It;
						}
					}
					else if (Result != *It)
					{
						Result = nullptr;
						break;
					}
				}
			}
		}

		return Result;
	}

	bool IsValidArgumentProperty(const FProperty* Property)
	{
		if (Property->HasAllPropertyFlags(CPF_Parm) && !Property->HasAnyPropertyFlags(CPF_Deprecated | CPF_EditorOnly | CPF_ReturnParm))
		{
			if (Property->HasAllPropertyFlags(CPF_OutParm) && !Property->HasAnyPropertyFlags(CPF_ConstParm | CPF_ReferenceParm))
			{
				return false;
			}

			return true;
		}

		return false;
	}

	// We only accept copy argument or const ref
	const FProperty* GetFirstArgumentProperty(const UFunction* InFunction)
	{
		check(InFunction);
		for (TFieldIterator<FProperty> It(InFunction); It && (It->PropertyFlags & CPF_Parm); ++It)
		{
			if (IsValidArgumentProperty(*It))
			{
				return *It;
			}
		}

		return nullptr;
	}

	TArray<const FProperty*> GetAllArgumentProperties(const UFunction* InFunction)
	{
		check(InFunction);

		TArray<const FProperty*> Arguments;
		for (TFieldIterator<FProperty> It(InFunction); It && (It->PropertyFlags & CPF_Parm); ++It)
		{
			if (IsValidArgumentProperty(*It))
			{
				Arguments.Add(*It);
			}
		}

		return Arguments;
	}

	void ExecuteFunction_NoReturnValue(UFunction* InFunction, UObject* UserWidget)
	{
		void* ConversionFunctionDataPtr = FMemory_Alloca_Aligned(InFunction->ParmsSize, InFunction->GetMinAlignment());
		if (InFunction->ParmsSize > 0)
		{
			FMemory::Memzero(ConversionFunctionDataPtr, InFunction->ParmsSize);
		}
		for (TFieldIterator<FProperty> It(InFunction); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			It->InitializeValue_InContainer(ConversionFunctionDataPtr);
		}

		UserWidget->ProcessEvent(InFunction, ConversionFunctionDataPtr);

		for (TFieldIterator<FProperty> It(InFunction); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			It->DestroyValue_InContainer(ConversionFunctionDataPtr);
		}
	}

	namespace Private
	{
		const FProperty* GetRuntimeReturnProperty(const UFunction* InFunction)
		{
			check(InFunction);
			FProperty* Result = InFunction->GetReturnProperty();

			if (Result == nullptr && InFunction->HasAllFunctionFlags(FUNC_HasOutParms))
			{
				for (TFieldIterator<FProperty> It(InFunction); It && (It->PropertyFlags & CPF_Parm); ++It)
				{
					if (It->HasAllPropertyFlags(CPF_OutParm) && !It->HasAnyPropertyFlags(CPF_Deprecated | CPF_EditorOnly | CPF_ConstParm | CPF_ReferenceParm))
					{
						Result = *It;
						break;
					}
				}
			}

			return Result;
		}
	}

	void ExecuteBinding_NoCheck(const FFieldContext& Source, const FFieldContext& Destination)
	{
		check(!Source.GetObjectVariant().IsNull() && !Destination.GetObjectVariant().IsNull());
		check(!Source.GetFieldVariant().IsEmpty() && !Destination.GetFieldVariant().IsEmpty());

		const bool bIsSourceBindingIsProperty = Source.GetFieldVariant().IsProperty();
		const FProperty* GetterType = bIsSourceBindingIsProperty ? Source.GetFieldVariant().GetProperty() : Private::GetRuntimeReturnProperty(Source.GetFieldVariant().GetFunction());
		check(GetterType);

		const bool bIsDestinationBindingIsProperty = Destination.GetFieldVariant().IsProperty();
		const FProperty* SetterType = bIsDestinationBindingIsProperty ? Destination.GetFieldVariant().GetProperty() : GetFirstArgumentProperty(Destination.GetFieldVariant().GetFunction());
		check(SetterType);

		check(ArePropertiesCompatible(GetterType, SetterType));

		const int32 AllocationSize = FMath::Max(GetterType->GetSize(), SetterType->GetSize());
		const int32 AllocationMinAlignment = FMath::Max(GetterType->GetMinAlignment(), SetterType->GetMinAlignment());
		void* DataPtr = FMemory_Alloca_Aligned(AllocationSize, AllocationMinAlignment);
		GetterType->InitializeValue(DataPtr);

		if (bIsSourceBindingIsProperty)
		{
			GetterType->GetValue_InContainer(Source.GetObjectVariant().GetData(), DataPtr);
		}
		else
		{
			check(GetterType->GetSize() == Source.GetFieldVariant().GetFunction()->ParmsSize); // only 1 param
			check(Source.GetObjectVariant().IsUObject());
			Source.GetObjectVariant().GetUObject()->ProcessEvent(Source.GetFieldVariant().GetFunction(), DataPtr);
		}

		Private::ConvertImplicitly(GetterType, SetterType, DataPtr);

		if (bIsDestinationBindingIsProperty)
		{
			SetterType->SetValue_InContainer(Destination.GetObjectVariant().GetData(), DataPtr);
		}
		else
		{
			check(SetterType->GetSize() == Destination.GetFieldVariant().GetFunction()->ParmsSize); // only 1 param
			check(Destination.GetObjectVariant().IsUObject());
			Destination.GetObjectVariant().GetUObject()->ProcessEvent(Destination.GetFieldVariant().GetFunction(), DataPtr);
		}

		GetterType->DestroyValue(DataPtr);
	}


	void ExecuteBinding_NoCheck(const FFieldContext& Destination, const FFunctionContext& ConversionFunction)
	{
		check(!Destination.GetObjectVariant().IsNull());
		check(!Destination.GetFieldVariant().IsEmpty());
		check(ConversionFunction.GetFunction() && ConversionFunction.GetObject());

		void* ConversionFunctionDataPtr = FMemory_Alloca_Aligned(ConversionFunction.GetFunction()->ParmsSize, ConversionFunction.GetFunction()->GetMinAlignment());
		for (TFieldIterator<FProperty> It(ConversionFunction.GetFunction()); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			It->InitializeValue_InContainer(ConversionFunctionDataPtr);
		}

		ConversionFunction.GetObject()->ProcessEvent(ConversionFunction.GetFunction(), ConversionFunctionDataPtr);

		// The return property will be baked into async conversion functions, so it may not exist
		if (const FProperty* ReturnConversionProperty = Private::GetRuntimeReturnProperty(ConversionFunction.GetFunction()))
		{
			const bool bIsDestinationBindingIsProperty = Destination.GetFieldVariant().IsProperty();
			const FProperty* SetterType = bIsDestinationBindingIsProperty ? Destination.GetFieldVariant().GetProperty() : GetFirstArgumentProperty(Destination.GetFieldVariant().GetFunction());
			check(SetterType);
			check(ArePropertiesCompatible(ReturnConversionProperty, SetterType));

			if (Private::CanConvertImplicitly(ReturnConversionProperty, SetterType))
			{
				// we need to do a copy because we may destroy the ReturnConversionProperty (imagine the return type is a TArray)
				const int32 AllocationSize = FMath::Max(SetterType->GetSize(), ReturnConversionProperty->GetSize());
				const int32 AllocationMinAlignment = FMath::Max(SetterType->GetMinAlignment(), ReturnConversionProperty->GetMinAlignment());
				void* SetterDataPtr = FMemory_Alloca_Aligned(AllocationSize, AllocationMinAlignment);
				SetterType->InitializeValue(SetterDataPtr); // probably not needed since they are double/float

				ReturnConversionProperty->CopyCompleteValue(SetterDataPtr, ReturnConversionProperty->ContainerPtrToValuePtr<void>(ConversionFunctionDataPtr));
				Private::ConvertImplicitly(ReturnConversionProperty, SetterType, SetterDataPtr);

				if (bIsDestinationBindingIsProperty)
				{
					SetterType->SetValue_InContainer(Destination.GetObjectVariant().GetData(), SetterDataPtr);
				}
				else
				{
					check(SetterType->GetSize() == Destination.GetFieldVariant().GetFunction()->ParmsSize); // only 1 param
					check(Destination.GetObjectVariant().IsUObject());
					Destination.GetObjectVariant().GetUObject()->ProcessEvent(Destination.GetFieldVariant().GetFunction(), SetterDataPtr);
				}

				SetterType->DestroyValue(SetterDataPtr);
			}
			else
			{
				// Re use the same buffer, no need to create a new copy
				void* DestinationDataPtr = ReturnConversionProperty->ContainerPtrToValuePtr<void>(ConversionFunctionDataPtr);
				if (bIsDestinationBindingIsProperty)
				{
					SetterType->SetValue_InContainer(Destination.GetObjectVariant().GetData(), DestinationDataPtr);
				}
				else
				{
					check(SetterType->GetSize() == Destination.GetFieldVariant().GetFunction()->ParmsSize); // only 1 param
					check(Destination.GetObjectVariant().IsUObject());
					Destination.GetObjectVariant().GetUObject()->ProcessEvent(Destination.GetFieldVariant().GetFunction(), DestinationDataPtr);
				}
			}
		}

		for (TFieldIterator<FProperty> It(ConversionFunction.GetFunction()); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			It->DestroyValue_InContainer(ConversionFunctionDataPtr);
		}
	}


	void ExecuteBinding_NoCheck(const FFieldContext& Source, const FFieldContext& Destination, const FFunctionContext& ConversionFunction)
	{
		check(!Source.GetObjectVariant().IsNull() && !Destination.GetObjectVariant().IsNull());
		check(!Source.GetFieldVariant().IsEmpty() && !Destination.GetFieldVariant().IsEmpty());
		check(ConversionFunction.GetFunction() && ConversionFunction.GetObject());

		void* ConversionFunctionDataPtr = FMemory_Alloca_Aligned(ConversionFunction.GetFunction()->ParmsSize, ConversionFunction.GetFunction()->GetMinAlignment());
		for (TFieldIterator<FProperty> It(ConversionFunction.GetFunction()); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			It->InitializeValue_InContainer(ConversionFunctionDataPtr);
		}

		// Get the value
		{
			const FProperty* ArgumentConversionProperty = GetFirstArgumentProperty(ConversionFunction.GetFunction());
			check(ArgumentConversionProperty);

			const bool bIsSourceBindingIsProperty = Source.GetFieldVariant().IsProperty();
			const FProperty* GetterType = bIsSourceBindingIsProperty ? Source.GetFieldVariant().GetProperty() : Private::GetRuntimeReturnProperty(Source.GetFieldVariant().GetFunction());
			check(GetterType);
			check(ArePropertiesCompatible(GetterType, ArgumentConversionProperty));

			if (Private::CanConvertImplicitly(GetterType, ArgumentConversionProperty))
			{
				// we need to do a copy because we may destroy the ReturnConversionProperty (imagine the return type is a TArray)
				const int32 AllocationSize = FMath::Max(GetterType->GetSize(), ArgumentConversionProperty->GetSize());
				const int32 AllocationMinAlignment = FMath::Max(GetterType->GetMinAlignment(), ArgumentConversionProperty->GetMinAlignment());
				void* GetterDataPtr = FMemory_Alloca_Aligned(AllocationSize, AllocationMinAlignment);
				GetterType->InitializeValue(GetterDataPtr); // probably not needed since they are double/float

				if (bIsSourceBindingIsProperty)
				{
					GetterType->GetValue_InContainer(Source.GetObjectVariant().GetData(), GetterDataPtr);
				}
				else
				{
					check(GetterType->GetSize() == Source.GetFieldVariant().GetFunction()->ParmsSize); // only 1 param
					check(Source.GetObjectVariant().IsUObject());
					Source.GetObjectVariant().GetUObject()->ProcessEvent(Source.GetFieldVariant().GetFunction(), GetterDataPtr);
				}

				Private::ConvertImplicitly(GetterType, ArgumentConversionProperty, GetterDataPtr);
				ArgumentConversionProperty->CopyCompleteValue(ArgumentConversionProperty->ContainerPtrToValuePtr<void>(ConversionFunctionDataPtr), GetterDataPtr);

				GetterType->DestroyValue(GetterDataPtr);

			}
			else
			{
				// Re use the same buffer, no need to create a new copy
				void* SourceDataPtr = ArgumentConversionProperty->ContainerPtrToValuePtr<void>(ConversionFunctionDataPtr);
				if (bIsSourceBindingIsProperty)
				{
					GetterType->GetValue_InContainer(Source.GetObjectVariant().GetData(), SourceDataPtr);
				}
				else
				{
					check(GetterType->GetSize() == Source.GetFieldVariant().GetFunction()->ParmsSize); // only 1 param
					check(Source.GetObjectVariant().IsUObject());
					Source.GetObjectVariant().GetUObject()->ProcessEvent(Source.GetFieldVariant().GetFunction(), SourceDataPtr);
				}
			}
		}

		ConversionFunction.GetObject()->ProcessEvent(ConversionFunction.GetFunction(), ConversionFunctionDataPtr);

		{
			const FProperty* ReturnConversionProperty = Private::GetRuntimeReturnProperty(ConversionFunction.GetFunction());
			check(ReturnConversionProperty);

			const bool bIsDestinationBindingIsProperty = Destination.GetFieldVariant().IsProperty();
			const FProperty* SetterType = bIsDestinationBindingIsProperty ? Destination.GetFieldVariant().GetProperty() : GetFirstArgumentProperty(Destination.GetFieldVariant().GetFunction());
			check(SetterType);
			check(ArePropertiesCompatible(ReturnConversionProperty, SetterType));

			if (Private::CanConvertImplicitly(ReturnConversionProperty, SetterType))
			{
				// we need to do a copy because we may destroy the ReturnConversionProperty (imagine the return type is a TArray)
				const int32 AllocationSize = FMath::Max(SetterType->GetSize(), ReturnConversionProperty->GetSize());
				const int32 AllocationMinAlignment = FMath::Max(SetterType->GetMinAlignment(), ReturnConversionProperty->GetMinAlignment());
				void* SetterDataPtr = FMemory_Alloca_Aligned(AllocationSize, AllocationMinAlignment);
				SetterType->InitializeValue(SetterDataPtr); // probably not needed since they are double/float

				ReturnConversionProperty->CopyCompleteValue(SetterDataPtr, ReturnConversionProperty->ContainerPtrToValuePtr<void>(ConversionFunctionDataPtr));
				Private::ConvertImplicitly(ReturnConversionProperty, SetterType, SetterDataPtr);

				if (bIsDestinationBindingIsProperty)
				{
					SetterType->SetValue_InContainer(Destination.GetObjectVariant().GetData(), SetterDataPtr);
				}
				else
				{
					check(SetterType->GetSize() == Destination.GetFieldVariant().GetFunction()->ParmsSize); // only 1 param
					check(Destination.GetObjectVariant().IsUObject());
					Destination.GetObjectVariant().GetUObject()->ProcessEvent(Destination.GetFieldVariant().GetFunction(), SetterDataPtr);
				}

				SetterType->DestroyValue(SetterDataPtr);
			}
			else
			{
				// Re use the same buffer, no need to create a new copy
				void* DestinationDataPtr = ReturnConversionProperty->ContainerPtrToValuePtr<void>(ConversionFunctionDataPtr);
				if (bIsDestinationBindingIsProperty)
				{
					SetterType->SetValue_InContainer(Destination.GetObjectVariant().GetData(), DestinationDataPtr);
				}
				else
				{
					check(SetterType->GetSize() == Destination.GetFieldVariant().GetFunction()->ParmsSize); // only 1 param
					check(Destination.GetObjectVariant().IsUObject());
					Destination.GetObjectVariant().GetUObject()->ProcessEvent(Destination.GetFieldVariant().GetFunction(), DestinationDataPtr);
				}
			}
		}

		for (TFieldIterator<FProperty> It(ConversionFunction.GetFunction()); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			It->DestroyValue_InContainer(ConversionFunctionDataPtr);
		}
	}
}

#undef LOCTEXT_NAMESPACE
