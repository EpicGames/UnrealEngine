// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MVVMCompiledBindingLibrary.h"

#include "Bindings/MVVMBindingHelper.h"
#include "IFieldNotificationClassDescriptor.h"
#include "INotifyFieldValueChanged.h"
#include "ModelViewViewModelModule.h"
#include "MVVMMessageLog.h"
#include "Templates/ValueOrError.h"
#include "Types/MVVMFieldContext.h"
#include "Types/MVVMFunctionContext.h"
#include "UObject/PropertyAccessUtil.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMCompiledBindingLibrary)

DECLARE_CYCLE_STAT(TEXT("Load Library"), STAT_UMG_Viewmodel_LoadLibrary, STATGROUP_UMG_Viewmodel);

#define LOCTEXT_NAMESPACE "MVVMCompiledBindingLibrary"

namespace UE::MVVM::Private
{
	bool IsFunctionVirtual(const UFunction* Function)
	{
		return !Function->HasAnyFunctionFlags(FUNC_Static | FUNC_Final);
	}
}

/**
 *
 */
FProperty* FMVVMVCompiledFields::GetProperty(FName PropertyName) const
{
	return PropertyAccessUtil::FindPropertyByName(PropertyName, ClassOrScriptStruct);
}


UFunction* FMVVMVCompiledFields::GetFunction(FName FunctionName) const
{
	// ScriptStruct should not have UFunction
	return CastChecked<UClass>(ClassOrScriptStruct)->FindFunctionByName(FunctionName);
}


/**
 *
 */
FMVVMCompiledBindingLibrary::FLoadedFunction::FLoadedFunction(const UFunction* Function)
{
	if (Function)
	{
		ClassOwner = Function->GetOwnerClass();
		FunctionName = Function->GetFName();
		bIsFunctionVirtual = UE::MVVM::Private::IsFunctionVirtual(Function);
	}
}

UFunction* FMVVMCompiledBindingLibrary::FLoadedFunction::GetFunction() const
{
	const UClass* ClassPtr = ClassOwner.Get();
	if (ClassPtr)
	{
		check(!FunctionName.IsNone());
		return ClassPtr->FindFunctionByName(FunctionName);
	}
	return nullptr;
}

UFunction* FMVVMCompiledBindingLibrary::FLoadedFunction::GetFunction(const UObject* CallingContext) const
{
	if (bIsFunctionVirtual)
	{
		check(!FunctionName.IsNone());
		if (CallingContext)
		{
			return CallingContext->GetClass()->FindFunctionByName(FunctionName);
		}
		return nullptr;
	}
	return GetFunction();
}

/**
 *
 */
FMVVMCompiledBindingLibrary::FMVVMCompiledBindingLibrary()
#if WITH_EDITORONLY_DATA
	: CompiledBindingLibraryId(FGuid::NewGuid())
#endif
{

}

#if WITH_EDITOR
FMVVMCompiledBindingLibrary::FMVVMCompiledBindingLibrary(FGuid LibraryId)
	: CompiledBindingLibraryId(LibraryId)
{

}
#endif


FText FMVVMCompiledBindingLibrary::LexToText(EExecutionFailingReason Reason)
{
	switch (Reason)
	{
	case EExecutionFailingReason::IncompatibleLibrary: return LOCTEXT("FailingReasonIncompatibleLibrary", "Incompatible Library");
	case EExecutionFailingReason::InvalidSource: return LOCTEXT("FailingReasonInvalidSource", "Invalid Source");
	case EExecutionFailingReason::InvalidDestination: return LOCTEXT("FailingReasonInvalidDestination", "Invalid Destination");
	case EExecutionFailingReason::InvalidConversionFunction: return LOCTEXT("FailingReasonInvalidConversionFunction", "Invalid Conversion Function");
	case EExecutionFailingReason::InvalidCast: return LOCTEXT("FailingReasonInvalidCast", "Invalid Cast");
	}
	return FText();
}


void FMVVMCompiledBindingLibrary::Load()
{
	SCOPE_CYCLE_COUNTER(STAT_UMG_Viewmodel_LoadLibrary);

	ensureAlwaysMsgf(LoadedProperties.Num() == 0, TEXT("The binding library was loaded more than once."));
	ensureAlwaysMsgf(LoadedFunctions.Num() == 0, TEXT("The binding library was loaded more than once."));

	for (const FMVVMVCompiledFields& Field : CompiledFields)
	{
		if (const UClass* Class = Cast<UClass>(Field.GetStruct()))
		{
			ensureAlwaysMsgf(!Class->HasAnyClassFlags(CLASS_NewerVersionExists), TEXT("The field is invalid. Property and Functions will not be loaded correctly."));
		}

		{
			const int32 NumberOfProperties = Field.GetPropertyNum();
			for (int32 Index = 0; Index < NumberOfProperties; ++Index)
			{
				FName FieldName = Field.GetPropertyName(CompiledFieldNames, Index);
				FProperty* LoadedProperty = Field.GetProperty(FieldName);
				LoadedProperties.Add(LoadedProperty); // add it even if none to keep the index valid
				ensureAlwaysMsgf(LoadedProperty != nullptr, TEXT("The property '%s:%s' could not be loaded."), (Field.GetStruct() ? *Field.GetStruct()->GetName() : TEXT("None")), *FieldName.ToString());
			}
		}
		{
			const int32 NumberOfFunctions = Field.GetFunctionNum();
			for (int32 Index = 0; Index < NumberOfFunctions; ++Index)
			{
				FName FieldName = Field.GetFunctionName(CompiledFieldNames, Index);
				UFunction* LoadedFunction = Field.GetFunction(FieldName);
				ensureAlwaysMsgf(LoadedFunction != nullptr, TEXT("The function '%s:%s' could not be loaded."), (Field.GetStruct() ? *Field.GetStruct()->GetName() : TEXT("None")), *FieldName.ToString());

				LoadedFunctions.Emplace(LoadedFunction); // add it even if none to keep the index valid
			}
		}
	}
}


bool FMVVMCompiledBindingLibrary::IsLoaded() const
{
	return LoadedProperties.Num() > 0 || LoadedFunctions.Num() > 0 || CompiledFields.Num() == 0;
}


void FMVVMCompiledBindingLibrary::Unload()
{
	LoadedProperties.Empty();
	LoadedFunctions.Empty();
}


TValueOrError<void, FMVVMCompiledBindingLibrary::EExecutionFailingReason> FMVVMCompiledBindingLibrary::Execute(UObject* InExecutionSource, const FMVVMVCompiledBinding& InBinding, EConversionFunctionType InFunctionType) const
{
	check(InExecutionSource);

	return ExecuteImpl(InExecutionSource, InBinding, nullptr, InFunctionType);
}


TValueOrError<void, FMVVMCompiledBindingLibrary::EExecutionFailingReason> FMVVMCompiledBindingLibrary::ExecuteWithSource(UObject* InExecutionSource, const FMVVMVCompiledBinding& InBinding, UObject* InSource) const
{
	check(InExecutionSource);
	check(InSource);

	return ExecuteImpl(InExecutionSource, InBinding, InSource, EConversionFunctionType::Simple);
}


TValueOrError<void, FMVVMCompiledBindingLibrary::EExecutionFailingReason> FMVVMCompiledBindingLibrary::ExecuteImpl(UObject* InExecutionSource, const FMVVMVCompiledBinding& InBinding, UObject* InSource, EConversionFunctionType InFunctionType) const
{
#if WITH_EDITORONLY_DATA
	const bool bIsValidBinding = InBinding.CompiledBindingLibraryId == CompiledBindingLibraryId;
	if (InBinding.CompiledBindingLibraryId.IsValid() && !bIsValidBinding)
	{
		ensureAlwaysMsgf(false, TEXT("The binding is from a different library."));
		return MakeError(EExecutionFailingReason::IncompatibleLibrary);
	}
#endif

	UE::MVVM::FFieldContext ValidSource;
	if (InFunctionType != EConversionFunctionType::Complex)
	{
		if (InSource)
		{
#if DO_CHECK
			TValueOrError<UE::MVVM::FFieldContext, void> TestSource = EvaluateFieldPath(InExecutionSource, InBinding.SourceFieldPath);
			if (TestSource.HasError())
			{
				ensureAlwaysMsgf(false, TEXT("The source was provided but it was not able to fetch it."));
				return MakeError(EExecutionFailingReason::InvalidSource);
			}
			if (!TestSource.GetValue().GetObjectVariant().IsUObject())
			{
				ensureAlwaysMsgf(false, TEXT("The source was provided as a UObject but the fetched source is not a UObject."));
				return MakeError(EExecutionFailingReason::InvalidSource);
			}
			if (InSource != TestSource.GetValue().GetObjectVariant().GetUObject())
			{
				ensureAlwaysMsgf(false, TEXT("The source was provided and it's not the same as the fetched source."));
				return MakeError(EExecutionFailingReason::InvalidSource);
			}
			checkf(TestSource.GetValue().GetObjectVariant().GetData() != nullptr, TEXT("Tested in EvaluateFieldPath"));
#endif

			TValueOrError<UE::MVVM::FMVVMFieldVariant, void> FinalPath = GetFinalFieldFromPathImpl(UE::MVVM::FObjectVariant(InSource), InBinding.SourceFieldPath);
			if (FinalPath.HasError())
			{
				return MakeError(EExecutionFailingReason::InvalidSource);
			}
			ValidSource = UE::MVVM::FFieldContext(InSource, FinalPath.StealValue());
		}
		else
		{
			TValueOrError<UE::MVVM::FFieldContext, void> FoundSource = EvaluateFieldPath(InExecutionSource, InBinding.SourceFieldPath);
			if (FoundSource.HasError())
			{
				return MakeError(EExecutionFailingReason::InvalidSource);
			}
			checkf(FoundSource.GetValue().GetObjectVariant().GetData() != nullptr, TEXT("Tested in EvaluateFieldPath"));

			ValidSource = FoundSource.StealValue();
		}
	}

	TValueOrError<UE::MVVM::FFieldContext, void> FoundDestination = EvaluateFieldPath(InExecutionSource, InBinding.DestinationFieldPath);
	if (FoundDestination.HasError())
	{
		return MakeError(EExecutionFailingReason::InvalidDestination);
	}
	checkf(FoundDestination.GetValue().GetObjectVariant().GetData() != nullptr, TEXT("Tested in EvaluateFieldPath"));

	UE::MVVM::FFunctionContext ConversionFunction;
	if (InBinding.ConversionFunctionFieldPath.IsValid())
	{
		TValueOrError<UE::MVVM::FFieldContext, void> FoundConversionFunction = EvaluateFieldPath(InExecutionSource, InBinding.ConversionFunctionFieldPath);
		if (FoundConversionFunction.HasError() || !FoundConversionFunction.GetValue().GetFieldVariant().IsFunction())
		{
			return MakeError(EExecutionFailingReason::InvalidConversionFunction);
		}

		if (FoundConversionFunction.GetValue().GetFieldVariant().GetFunction()->HasAllFunctionFlags(FUNC_Static))
		{
			ConversionFunction = UE::MVVM::FFunctionContext::MakeStaticFunction(FoundConversionFunction.GetValue().GetFieldVariant().GetFunction());
		}
		else if (!FoundConversionFunction.GetValue().GetObjectVariant().IsUObject())
		{
			return MakeError(EExecutionFailingReason::InvalidConversionFunction);
		}
		else
		{
			ConversionFunction = UE::MVVM::FFunctionContext(FoundConversionFunction.GetValue().GetObjectVariant().GetUObject(), FoundConversionFunction.GetValue().GetFieldVariant().GetFunction());
		}
	}

	return ExecuteImpl(ValidSource, FoundDestination.GetValue(), ConversionFunction, InFunctionType);
}


TValueOrError<void, FMVVMCompiledBindingLibrary::EExecutionFailingReason> FMVVMCompiledBindingLibrary::ExecuteImpl(UE::MVVM::FFieldContext& Source, UE::MVVM::FFieldContext& Destination, UE::MVVM::FFunctionContext& ConversionFunction, EConversionFunctionType FunctionType) const
{
	if (FunctionType != EConversionFunctionType::Complex)
	{
		check(!Source.GetObjectVariant().IsEmpty());
		check(!Source.GetObjectVariant().IsNull());
		check(!Source.GetFieldVariant().IsEmpty());
		if (!Source.GetObjectVariant().GetOwner()->IsChildOf(Source.GetFieldVariant().GetOwner()))
		{
			return MakeError(EExecutionFailingReason::InvalidCast);
		}
	}
	check(!Destination.GetObjectVariant().IsEmpty());
	check(!Destination.GetObjectVariant().IsNull());
	check(!Destination.GetFieldVariant().IsEmpty());
	if (!Destination.GetObjectVariant().GetOwner()->IsChildOf(Destination.GetFieldVariant().GetOwner()))
	{
		return MakeError(EExecutionFailingReason::InvalidCast);
	}

	if (ConversionFunction.GetFunction())
	{
		if (FunctionType == EConversionFunctionType::Complex)
		{
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Destination, ConversionFunction);
		}
		else
		{
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination, ConversionFunction);
		}
	}
	else
	{
		UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination);
	}

	return MakeValue();
}


namespace UE::MVVM::Private
{
	UObject* EvaluateFieldPathObject(UObject* Container, UFunction* Function, const FObjectPropertyBase* ReturnObjectProperty)
	{
		void* DataPtr = FMemory_Alloca_Aligned(Function->ParmsSize, Function->GetMinAlignment());
		ReturnObjectProperty->InitializeValue(DataPtr);
		Container->ProcessEvent(Function, DataPtr);
		UObject* NewContainer = ReturnObjectProperty->GetObjectPropertyValue_InContainer(DataPtr);
		ReturnObjectProperty->DestroyValue(DataPtr);
		return NewContainer;
	}
}

bool FMVVMCompiledBindingLibrary::CanAccessLoadedProperiesAtIndex(int16 index) const
{
	if (LoadedProperties.IsValidIndex(index))
	{
		return true;
	}
	FString LoadedPropertiesString;
	for (int32 i = 0; i < LoadedProperties.Num(); ++i)
	{
		LoadedPropertiesString.Append(FString::Format(TEXT("{0}, "), { LoadedProperties[i]->GetNameCPP() }));
	}
	FString DebugLoadedProperties = FString::Format(TEXT("{0} on {1}. LoadedProperties : {2}"), { index, LoadedProperties.Num(), LoadedPropertiesString });
	//checkf(false, TEXT("LoadedProperties.IsValidIndex(PathIndex.Index) failed. '%s'"), *DebugLoadedProperties);
	UE_LOGF(LogMVVM, Error, "LoadedProperties.IsValidIndex(PathIndex.Index) failed. '%ls'", *DebugLoadedProperties);
	return false;
}

TValueOrError<UE::MVVM::FFieldContext, void> FMVVMCompiledBindingLibrary::EvaluateFieldPath(UObject* InExecutionSource, const FMVVMVCompiledFieldPath& InFieldPath) const
{
	if (InExecutionSource == nullptr)
	{
		return MakeError();
	}

#if WITH_EDITORONLY_DATA
	const bool bIsValidBinding = InFieldPath.CompiledBindingLibraryId == CompiledBindingLibraryId;
	if (InFieldPath.CompiledBindingLibraryId.IsValid() && !bIsValidBinding)
	{
		ensureAlwaysMsgf(false, TEXT("The binding is from a different library."));
		return MakeError();
	}
#endif

	if (!InFieldPath.IsValid())
	{
		return MakeError();
	}

	UE::MVVM::FObjectVariant CurrentContainer = UE::MVVM::FObjectVariant(InExecutionSource);
	const int32 IterationMax = InFieldPath.StartIndex + InFieldPath.Num - 1;
	checkf(IterationMax < FieldPaths.Num(), TEXT("The number of field is bigger than the number of field that was compiled."));
	for (int32 Index = InFieldPath.StartIndex; Index < IterationMax; ++Index)
	{
		if (CurrentContainer.IsNull())
		{
			return MakeError();
		}

		check(FieldPaths.IsValidIndex(Index));
		FMVVMCompiledLoadedPropertyOrFunctionIndex PathIndex = FieldPaths[Index];
		if (PathIndex.bIsProperty)
		{
			if (!CanAccessLoadedProperiesAtIndex(PathIndex.Index))
			{
				return MakeError();
			}
			const FProperty* Property = LoadedProperties[PathIndex.Index];
			if (!Property)
			{
				return MakeError();
			}

			if (PathIndex.bIsObjectProperty)
			{
				const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property);
				check(ObjectProperty);
				UObject* NewContainer = ObjectProperty->GetObjectPropertyValue_InContainer(CurrentContainer.GetData()); // this skip any Getter
				CurrentContainer.SetUObject(NewContainer);
			}
			else if (PathIndex.bIsScriptStructProperty)
			{
				const FStructProperty* StrutProperty = CastFieldChecked<const FStructProperty>(Property);
				if (StrutProperty->HasGetter())
				{
					ensureAlwaysMsgf(false, TEXT("This is not supported since it would create a temporary structure and we would not able to return a valid container from that structure."));
					return MakeError();
				}

				CurrentContainer.SetUScriptStruct(StrutProperty->Struct, StrutProperty->ContainerPtrToValuePtr<void>(CurrentContainer.GetData()));
			}
			else
			{
				return MakeError();
			}
		}
		else
		{
			check(LoadedFunctions.IsValidIndex(PathIndex.Index));
			const FLoadedFunction& LoadedFunction = LoadedFunctions[PathIndex.Index];
			UFunction* Function = CurrentContainer.IsUObject() ? LoadedFunction.GetFunction(CurrentContainer.GetUObject()) : LoadedFunction.GetFunction();
			if (!Function)
			{
				return MakeError();
			}

			if (PathIndex.bIsObjectProperty)
			{
				const FObjectPropertyBase* ReturnObjectProperty = CastFieldChecked<const FObjectPropertyBase>(UE::MVVM::BindingHelper::GetReturnProperty(Function));
				check(ReturnObjectProperty);
				check(Function->NumParms == 1);

				if (!CurrentContainer.IsUObject())
				{
					return MakeError();
				}

				CurrentContainer.SetUObject(UE::MVVM::Private::EvaluateFieldPathObject(CurrentContainer.GetUObject(), Function, ReturnObjectProperty));
			}
			else if (PathIndex.bIsScriptStructProperty)
			{
				ensureAlwaysMsgf(false, TEXT("bIsScriptStructProperty is not supported since it would create a temporary structure and we would not able to return a valid container from that structure."));
				return MakeError();
			}
			else
			{
				return MakeError();
			}
		}
	}

	if (CurrentContainer.IsNull())
	{
		return MakeError();
	}

	TValueOrError<UE::MVVM::FMVVMFieldVariant, void> FinalPath = GetFinalFieldFromPathImpl(CurrentContainer, InFieldPath);
	if (FinalPath.HasError())
	{
		return MakeError();
	}
	return MakeValue(MoveTemp(CurrentContainer), FinalPath.StealValue());
}


TValueOrError<UE::MVVM::FMVVMFieldVariant, void> FMVVMCompiledBindingLibrary::GetFinalFieldFromPathImpl(UE::MVVM::FObjectVariant CurrentContainer, const FMVVMVCompiledFieldPath& InFieldPath) const
{
	const int32 IterationMax = InFieldPath.StartIndex + InFieldPath.Num - 1;

	UE::MVVM::FMVVMFieldVariant FinalField;
	check(FieldPaths.IsValidIndex(IterationMax));
	FMVVMCompiledLoadedPropertyOrFunctionIndex FinalPathIndex = FieldPaths[IterationMax];
	if (FinalPathIndex.bIsProperty)
	{
		if (!CanAccessLoadedProperiesAtIndex(FinalPathIndex.Index))
		{
			return MakeError();
		}
		FProperty* Property = LoadedProperties[FinalPathIndex.Index];
		if (!Property)
		{
			return MakeError();
		}
		FinalField.SetProperty(Property);
	}
	else
	{
		check(LoadedFunctions.IsValidIndex(FinalPathIndex.Index));
		const FLoadedFunction& LoadedFunction = LoadedFunctions[FinalPathIndex.Index];
		UFunction* Function = CurrentContainer.IsUObject() ? LoadedFunction.GetFunction(CurrentContainer.GetUObject()) : LoadedFunction.GetFunction();
		if (!Function)
		{
			return MakeError();
		}
		FinalField.SetFunction(Function);
	}

	return MakeValue(MoveTemp(FinalField));
}


TValueOrError<FString, FString> FMVVMCompiledBindingLibrary::FieldPathToString(FMVVMVCompiledFieldPath FieldPath, bool bUseDisplayName) const
{
#if WITH_EDITORONLY_DATA
	const bool bIsValidBinding = FieldPath.CompiledBindingLibraryId == CompiledBindingLibraryId;
	if (FieldPath.CompiledBindingLibraryId.IsValid() && !bIsValidBinding)
	{
		ensureAlwaysMsgf(false, TEXT("The binding is from a different library."));
		return MakeError<FString>(TEXT("The binding is from a different library."));
	}
#endif

	const bool bIsLoaded = IsLoaded();

	bool bHasError = false;
	TStringBuilder<512> StringBuilder;
	const int32 IterationMax = FieldPath.StartIndex + FieldPath.Num;
	checkf(IterationMax <= FieldPaths.Num(), TEXT("The number of field is bigger than the number of field that was compiled."));
	for (int32 Index = FieldPath.StartIndex; Index < IterationMax; ++Index)
	{
		if (StringBuilder.Len() > 0)
		{
			StringBuilder << TEXT('.');
		}

		check(FieldPaths.IsValidIndex(Index));
		FMVVMCompiledLoadedPropertyOrFunctionIndex PathIndex = FieldPaths[Index];
		if (bIsLoaded)
		{
			if (PathIndex.bIsProperty)
			{
				if (LoadedProperties.IsValidIndex(PathIndex.Index))
				{
					const FProperty* Property = LoadedProperties[PathIndex.Index];
					if (!Property)
					{
						StringBuilder << TEXT("<Invalid>");
						bHasError = true;
					}
					else
					{
	#if WITH_EDITOR
						if (bUseDisplayName)
						{
							StringBuilder << Property->GetDisplayNameText().ToString();
						}
						else
	#endif
						{
							StringBuilder << Property->GetFName();
						}
					}
				}
				else
				{
					StringBuilder << TEXT("<Invalid>");
					bHasError = true;
				}
			}
			else
			{
				if (LoadedFunctions.IsValidIndex(PathIndex.Index))
				{
					const FLoadedFunction& LoadedFunction = LoadedFunctions[PathIndex.Index];
					const UFunction* Function = LoadedFunction.GetFunction();
					if (!Function)
					{
						StringBuilder << TEXT("<Invalid>");
						bHasError = true;
					}
					else
					{
	#if WITH_EDITOR
						if (bUseDisplayName)
						{
							StringBuilder << Function->GetDisplayNameText().ToString();
						}
						else
	#endif
						{
							StringBuilder << Function->GetFName();
						}
					}
				}
				else
				{
					StringBuilder << TEXT("<Invalid>");
					bHasError = true;
				}
			}
		}
		else
		{
			check(!bIsLoaded);
			int32 CurrentIndex = PathIndex.Index;
			bool bFound = false;
			for (const FMVVMVCompiledFields& Field : CompiledFields)
			{
				if (PathIndex.bIsProperty)
				{
					const int32 NumberOfProperties = Field.GetPropertyNum();
					if (CurrentIndex < NumberOfProperties)
					{
						FName FieldName = Field.GetPropertyName(CompiledFieldNames, CurrentIndex);
						StringBuilder << FieldName;
						bFound = true;
						break;
					}
					CurrentIndex -= NumberOfProperties;
				}
				else
				{
					const int32 NumberOfFunctions = Field.GetFunctionNum();
					if (CurrentIndex < NumberOfFunctions)
					{
						FName FieldName = Field.GetFunctionName(CompiledFieldNames, CurrentIndex);
						StringBuilder << FieldName;
						bFound = true;
						break;
					}
					CurrentIndex -= NumberOfFunctions;
				}
			}

			if (!bFound)
			{
				StringBuilder << TEXT("<Invalid>");
				bHasError = true;
			}
		}
	}

	if (bHasError)
	{
		return MakeError<FString>(StringBuilder.ToString());
	}
	else
	{

		return MakeValue<FString>(StringBuilder.ToString());
	}
}

#undef LOCTEXT_NAMESPACE
