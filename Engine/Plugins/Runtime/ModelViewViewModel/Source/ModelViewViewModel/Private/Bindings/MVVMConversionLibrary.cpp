// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MVVMConversionLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMConversionLibrary)

ESlateVisibility UMVVMConversionLibrary::Conv_BoolToSlateVisibility(bool bIsVisible, ESlateVisibility TrueCaseVisibility, ESlateVisibility FalseCaseVisibility)
{
	return bIsVisible ? TrueCaseVisibility : FalseCaseVisibility;
}
