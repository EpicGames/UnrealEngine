// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MVVMFieldPathHelper.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Extensions/UIComponent.h"
#include "Misc/MemStack.h"
#include "MVVMSubsystem.h"
#include "Templates/ValueOrError.h"
#include "Types/MVVMAvailableBinding.h"
#include "Types/MVVMFieldContext.h"

#if WITH_EDITOR
#include "Kismet2/BlueprintEditorUtils.h"
#endif

#define LOCTEXT_NAMESPACE "MVVMFieldPathHelper"

namespace UE::MVVM::FieldPathHelper
{

namespace Private
{

static const FName NAME_BlueprintGetter = "BlueprintGetter";
static const FName NAME_BlueprintSetter = "BlueprintSetter";


TValueOrError<UStruct*, FText> FindContainer(const FProperty* Property)
{
	const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property);
	const FStructProperty* StructProperty = CastField<const FStructProperty>(Property);

	if (ObjectProperty)
	{
		return MakeValue(ObjectProperty->PropertyClass);
	}
	else if (StructProperty)
	{
		return MakeValue(StructProperty->Struct);
	}
	return MakeError(FText::Format(LOCTEXT("InvalidPropertyTypeInSourcePath", "Only object or struct properties can be used as source paths. '{0}' is a '{1}'."), FText::FromString(Property->GetName()), FText::FromString(Property->GetClass()->GetName())));
}


const FProperty* FindProperty(FMVVMConstFieldVariant Field)
{
	if (Field.IsProperty())
	{
		return Field.GetProperty();
	}
	else if (Field.IsFunction() && Field.GetFunction())
	{
		return BindingHelper::GetReturnProperty(Field.GetFunction());
	}
	return nullptr;
}

const UStruct* GetMostUpToDateStruct(const UStruct* Struct)
{
#if WITH_EDITOR
	if (const UClass* Class = Cast<UClass>(Struct))
	{
		return FBlueprintEditorUtils::GetMostUpToDateClass(Class);
	}
	return Struct;
#else
	return Struct;
#endif
}


TValueOrError<FMVVMConstFieldVariant, FText> TransformWithAccessor(const UStruct* CurrentContainer, FMVVMConstFieldVariant CurrentField, bool bForReading)
{
#if WITH_EDITORONLY_DATA
	if (bForReading)
	{
		if (!CurrentField.GetProperty()->HasGetter())
		{
			const FString& BlueprintGetter = CurrentField.GetProperty()->GetMetaData(Private::NAME_BlueprintGetter);
			if (!BlueprintGetter.IsEmpty())
			{
				FMVVMConstFieldVariant NewField = BindingHelper::FindFieldByName(CurrentContainer, FMVVMBindingName(*BlueprintGetter));
				if (NewField.IsFunction())
				{
					CurrentField = NewField;
				}
				else
				{
					return MakeError(FText::Format(LOCTEXT("BlueprintGetterNotFound", "The BlueprintGetter '{0}' could not be found on object '{1}'."), 
						FText::FromString(BlueprintGetter), 
						FText::FromString(CurrentContainer->GetName()))
					);
				}
			}
		}
	}
	else
	{
		if (!CurrentField.GetProperty()->HasSetter())
		{
			const FString& BlueprintSetter = CurrentField.GetProperty()->GetMetaData(Private::NAME_BlueprintSetter);
			if (!BlueprintSetter.IsEmpty())
			{
				FMVVMConstFieldVariant NewField = BindingHelper::FindFieldByName(CurrentContainer, FMVVMBindingName(*BlueprintSetter));
				if (NewField.IsFunction())
				{
					CurrentField = NewField;
				}
				else
				{
					return MakeError(FText::Format(LOCTEXT("BlueprintSetterNotFound", "The BlueprintSetter '{0}' could not be found on object '{1}'."), 
						FText::FromString(BlueprintSetter),
						FText::FromString(CurrentContainer->GetName()))
					);
				}
			}
		}
	}
#endif
	return MakeValue(CurrentField);
}

} // namespace


TValueOrError<TArray<FMVVMConstFieldVariant>, FText> GenerateFieldPathList(const UClass* InFrom, FStringView InFieldPath, bool bForReading)
{
	if (InFrom == nullptr)
	{
		return MakeError(LOCTEXT("SourceClassInvalid", "The source class is invalid."));
	}
	if (InFieldPath.IsEmpty())
	{
		return MakeError(LOCTEXT("FieldPathEmpty", "The field path is empty."));
	}
	if (InFieldPath[InFieldPath.Len() - 1] == TEXT('.'))
	{
		return MakeError(LOCTEXT("FieldPathEndsInPeriod", "The field path cannot end with a '.' character."));
	}

	FMemMark Mark(FMemStack::Get());
	TArray<FMVVMConstFieldVariant, TMemStackAllocator<>> Result;

	const UStruct* CurrentContainer = InFrom;

	// Split the string into property or function names
	//ie. myvar.myfunction.myvar
	int32 FoundIndex = INDEX_NONE;
	while (InFieldPath.FindChar(TEXT('.'), FoundIndex))
	{
		FMVVMConstFieldVariant Field = BindingHelper::FindFieldByName(CurrentContainer, FMVVMBindingName(FName(FoundIndex, InFieldPath.GetData())));
		if (Field.IsEmpty())
		{
			return MakeError(FText::Format(LOCTEXT("FieldDoesNotExistInStruct", "The field '{0}' does not exist in the struct '{1}'."), 
				FText::FromName(FName(FoundIndex, InFieldPath.GetData())), 
				CurrentContainer ? FText::FromString(CurrentContainer->GetName()) : LOCTEXT("None", "<none>"))
			);
		}
		else if (Field.IsProperty())
		{
			TValueOrError<UStruct*, FText> FoundContainer = Private::FindContainer(Field.GetProperty());
			if (FoundContainer.HasError())
			{
				return MakeError(FoundContainer.StealError());
			}
			CurrentContainer = FoundContainer.GetValue();
		}
		
		if (Field.IsFunction())
		{
			const FProperty* ReturnProperty = BindingHelper::GetReturnProperty(Field.GetFunction());
			TValueOrError<UStruct*, FText> FoundContainer = Private::FindContainer(ReturnProperty);
			if (FoundContainer.HasError())
			{
				return MakeError(FoundContainer.StealError());
			}
			CurrentContainer = FoundContainer.GetValue();
		}

		InFieldPath = InFieldPath.RightChop(FoundIndex + 1);
		Result.Add(Field);
	}

	// The last field can be anything (that is what we are going to bind to)
	if (InFieldPath.Len() > 0)
	{
		FMVVMConstFieldVariant Field = BindingHelper::FindFieldByName(CurrentContainer, FMVVMBindingName(InFieldPath.GetData()));
		if (Field.IsEmpty())
		{
			return MakeError(FText::Format(LOCTEXT("FieldDoesNotExistInStruct", "The field '{0}' does not exist in the struct '{1}'."), 
				FText::FromStringView(InFieldPath), 
				CurrentContainer ? FText::FromString(CurrentContainer->GetName()) : LOCTEXT("None", "<none>"))
			);
		}

		Result.Add(Field);
	}

	return GenerateFieldPathList(Result, bForReading);
}


/**
 * Rules for reading:
 *		* Build path using Getter or BlueprintGetter if needed.
 *		* If the FProperty is a FStructProperty and a function was used, then the runtime may use dynamic memory instead of stack memory.
 * Rules for writing:
 *		* Build path using Getter or BlueprintGetter, the last element should use Setter or BlueprintSetter.
 *		* If one of the element in the path has a Setter or BlueprintSetter, then the path needs to stop there and be divided in 3 paths
 *			* ex: PropertyA.PropertyB.PropertyC.PropertyD.PropertyE and PropertyC has a Setter/BlueprintSetter
 *			* 1. To write: Property.Property.SetPropertyC()
 *			* 2. To read: PropertyA.PropertyB.PropertyC or PropertyA.PropertyB.GetPropertyC()
 *			* 3. To continue the reading: PropertyD.PropertyE
 *				PropertyD and PropertyE cannot have Getter/BlueprintGetter if they are FStructProperty
 *		* We can only have one Setter/BlueprintSetter in the path
 */
TValueOrError<TArray<FMVVMConstFieldVariant>, FText> GenerateFieldPathList(TArrayView<const FMVVMConstFieldVariant> InFieldPath, bool bForSourceBinding)
{
	if (InFieldPath.Num() == 0)
	{
		return MakeError(LOCTEXT("FieldPathEmpty", "The field path is empty."));
	}

	TArray<FMVVMConstFieldVariant> Result;
	Result.Reserve(InFieldPath.Num());
	const UStruct* CurrentContainer = InFieldPath[0].GetOwner();

	for (int32 Index = 0; Index < InFieldPath.Num(); ++Index)
	{
		bool bLastField = Index == InFieldPath.Num() - 1;
		FMVVMConstFieldVariant Field = InFieldPath[Index];

		if (Field.IsEmpty())
		{
			return MakeError(FText::Format(LOCTEXT("FieldIsEmpty", "The field at index '{0}' is empty."), Index));
		}

		if (CurrentContainer == nullptr)
		{
			return MakeError(FText::Format(LOCTEXT("FieldDoesNotExist", "The field '{0}' does not exist."), FText::FromName(Field.GetName())));
		}

		const UStruct* SkeletalOwner = Private::GetMostUpToDateStruct(Field.GetOwner());
		const UStruct* SkeletalCurrentContainer = Private::GetMostUpToDateStruct(CurrentContainer);
		const bool bIsChild = SkeletalOwner->IsChildOf(SkeletalCurrentContainer);
		const bool bIsDownCast = SkeletalCurrentContainer->IsChildOf(SkeletalOwner);
		if (!(bIsChild || bIsDownCast))
		{
			return MakeError(FText::Format(LOCTEXT("FieldDoesNotExistInStruct", "The field '{0}' does not exist in the struct '{1}'."), 
				FText::FromName(Field.GetName()), 
				FText::FromString(CurrentContainer->GetName()))
			);
		}
		
		if (Field.IsProperty())
		{
			TValueOrError<FMVVMConstFieldVariant, FText> TransformedField = Private::TransformWithAccessor(Field.GetOwner(), Field, (bForSourceBinding || !bLastField));
			if (TransformedField.HasError())
			{
				return MakeError(TransformedField.StealError());
			}
			Field = TransformedField.StealValue();
			check(!Field.IsEmpty());

			if (!bLastField && Field.IsProperty())
			{
				TValueOrError<UStruct*, FText> FoundContainer = Private::FindContainer(Field.GetProperty());
				if (FoundContainer.HasError())
				{
					return MakeError(FoundContainer.StealError());
				}
				CurrentContainer = FoundContainer.GetValue();
			}
		}

		if (!bLastField && Field.IsFunction())
		{
			const FProperty* ReturnProperty = BindingHelper::GetReturnProperty(Field.GetFunction());
			TValueOrError<UStruct*, FText> FoundContainer = Private::FindContainer(ReturnProperty);
			if (FoundContainer.HasError())
			{
				return MakeError(FoundContainer.StealError());
			}
			CurrentContainer = FoundContainer.GetValue();
		}

		Result.Add(Field);
	}

	return MakeValue(MoveTemp(Result));
}


TValueOrError<FParsedNotifyBindingInfo, FText> GetNotifyBindingInfoFromFieldPath(const UClass* InAccessor, TArrayView<const FMVVMConstFieldVariant> InFieldPath)
{
	// The class is the UserWidget.
	// We assume that the first property is the viewmodel/widget added in the editor
	// To be a valid oneway path, all properties after the initial viewmodel has to be "notify"
	// or after the last "notify" all the path after are onetime (example vectornotify.x)
	if (InAccessor == nullptr)
	{
		return MakeError(LOCTEXT("AccessorClassNull", "The accessor class is null."));
	}
	if (InFieldPath.Num() == 0)
	{
		return MakeError(LOCTEXT("FieldPathEmpty", "The field path is empty."));
	}

	{
		const UStruct* SkeletalFieldOwner = Private::GetMostUpToDateStruct(InFieldPath[0].GetOwner());
		const UStruct* SkeletalAccessor = Private::GetMostUpToDateStruct(InAccessor);
		if (!SkeletalAccessor->IsChildOf(SkeletalFieldOwner))
		{
			return MakeError(LOCTEXT("FieldPathDoesntStartWithAccessor", "Field path that doesn't start with the accessor is not supported."));
		}
	}

	FParsedNotifyBindingInfo Result;
	bool bFoundViewModel = false;
	for (int32 Index = InFieldPath.Num() - 1; Index >= 0; --Index)
	{
		FMVVMConstFieldVariant Field = InFieldPath[Index];

		if (Field.IsEmpty())
		{
			return MakeError(FText::Format(LOCTEXT("FieldIsEmpty", "The field at index '{0}' is empty."), Index));
		}

		const UStruct* FieldOwner = Field.GetOwner();
		check(FieldOwner);
		const UStruct* ParentContainerType = InAccessor;
		check(ParentContainerType);
		if (Index != 0)
		{
			const FProperty* ParentProperty = Private::FindProperty(InFieldPath[Index - 1]);
			if (ParentProperty == nullptr)
			{
				return MakeError(FText::Format(LOCTEXT("InvalidParentProperty", "The field at index '{0}' has an invalid parent."), Index));
			}

			TValueOrError<UStruct*, FText> FoundParentContainer = Private::FindContainer(ParentProperty);
			if (FoundParentContainer.HasError())
			{
				return MakeError(FoundParentContainer.StealError());
			}
			ParentContainerType = FoundParentContainer.GetValue();
		}

		if (ParentContainerType == nullptr)
		{
			return MakeError(FText::Format(LOCTEXT("InvalidFieldParent", "The field '{0}' is not a container."), Index - 1));
		}

		const UStruct* SkeletalFieldOwner = Private::GetMostUpToDateStruct(FieldOwner);
		const UStruct* SkeletalParentContainer = Private::GetMostUpToDateStruct(ParentContainerType);
		const bool bIsChild = SkeletalFieldOwner->IsChildOf(SkeletalParentContainer);
		const bool bIsDownCast = SkeletalParentContainer->IsChildOf(SkeletalFieldOwner);
		if (!(bIsChild || bIsDownCast))
		{
			return MakeError(FText::Format(LOCTEXT("FieldDoesNotExistInStruct", "The field '{0}' does not exist in the struct '{1}'."),
				FText::FromName(Field.GetName()),
				FText::FromString(ParentContainerType->GetName()))
			);
		}

		// Is field a viewmodel
		const UClass* FieldOwnerAsClass = Cast<const UClass>(FieldOwner);
		if (FieldOwnerAsClass && FieldOwnerAsClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
		{
			FMVVMAvailableBinding AvailableBinding = UMVVMSubsystem::GetAvailableBindingForField(Field, InAccessor);
			if (AvailableBinding.IsValid() && AvailableBinding.HasNotify())
			{
				if (!bFoundViewModel)
				{
					Result.NotifyFieldClass = const_cast<UClass*>(FieldOwnerAsClass);
					Result.NotifyFieldId = FFieldNotificationId(AvailableBinding.GetBindingName().ToName());
					Result.ViewModelIndex = Index - 1;
					bFoundViewModel = true;
				}
			}
			else if (bFoundViewModel && Index != 0)
			{
				Result = FParsedNotifyBindingInfo();
			}
		}
		else if (bFoundViewModel)
		{
			Result = FParsedNotifyBindingInfo();
		}
	}

	return MakeValue(MoveTemp(Result));
}


FString ToString(TArrayView<const FMVVMFieldVariant> Fields)
{
	TStringBuilder<512> Builder;
	for (int32 Index = 0; Index < Fields.Num(); ++Index)
	{
		if (Index != 0)
		{
			Builder << TEXT(".");
		}
		Builder << Fields[Index].GetName();
	}
	return Builder.ToString();
}


FString ToString(TArrayView<const FMVVMConstFieldVariant> Fields)
{
	TStringBuilder<512> Builder;
	for (int32 Index = 0; Index < Fields.Num(); ++Index)
	{
		if (Index != 0)
		{
			Builder << TEXT(".");
		}
		Builder << Fields[Index].GetName();
	}
	return Builder.ToString();
}

FText ToText(TArrayView<const FMVVMFieldVariant> Fields)
{
	TArray<FText> FieldNames;
	FieldNames.Reserve(Fields.Num());

	for (const FMVVMFieldVariant& Field : Fields)
	{
		FieldNames.Add(FText::FromName(Field.GetName()));
	}

	return FText::Join(LOCTEXT("PropertyPathDelim", "."), FieldNames);
}

FText ToText(TArrayView<const FMVVMConstFieldVariant> Fields)
{
	TArray<FText> FieldNames;
	FieldNames.Reserve(Fields.Num());

	for (const FMVVMConstFieldVariant& Field : Fields)
	{
		FieldNames.Add(FText::FromName(Field.GetName()));
	}

	return FText::Join(LOCTEXT("PropertyPathDelim", "."), FieldNames);
}

TValueOrError<UObject*, void> EvaluateObjectProperty(const FFieldContext& InSource)
{
	if (InSource.GetObjectVariant().IsNull())
	{
		return MakeError();
	}

	const bool bIsProperty = InSource.GetFieldVariant().IsProperty();
	const FProperty* GetterType = bIsProperty ? InSource.GetFieldVariant().GetProperty() : BindingHelper::GetReturnProperty(InSource.GetFieldVariant().GetFunction());
	check(GetterType);

	const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(GetterType);
	if (ObjectProperty == nullptr)
	{
		return MakeError();
	}

	if (bIsProperty)
	{
		return MakeValue(ObjectProperty->GetObjectPropertyValue_InContainer(InSource.GetObjectVariant().GetData()));
	}
	else
	{
		check(InSource.GetObjectVariant().IsUObject());
		check(InSource.GetObjectVariant().GetUObject());

		UFunction* Function = InSource.GetFieldVariant().GetFunction();
		const UStruct* SkeletalFunctionOwner = Private::GetMostUpToDateStruct(Function->GetOuterUClass());
		const UStruct* SkeletalObjectClass = Private::GetMostUpToDateStruct(InSource.GetObjectVariant().GetUObject()->GetClass());
		if (!SkeletalObjectClass->IsChildOf(SkeletalFunctionOwner))
		{
			return MakeError();
		}

		void* DataPtr = FMemory_Alloca_Aligned(Function->ParmsSize, Function->GetMinAlignment());
		ObjectProperty->InitializeValue(DataPtr);
		InSource.GetObjectVariant().GetUObject()->ProcessEvent(Function, DataPtr);
		UObject* Result = ObjectProperty->GetObjectPropertyValue_InContainer(DataPtr);
		ObjectProperty->DestroyValue(DataPtr);
		return MakeValue(Result);
	}
}

TValueOrError<const UStruct*, void> GetFieldAsContainer(const UE::MVVM::FMVVMConstFieldVariant Field)
{
	const FProperty* PropertyToTest = nullptr;
	if (Field.IsProperty())
	{
		PropertyToTest = Field.GetProperty();
	}
	else if (Field.IsFunction())
	{
		PropertyToTest = BindingHelper::GetReturnProperty(Field.GetFunction());
	}

	if (PropertyToTest)
	{
		if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(PropertyToTest))
		{
			return MakeValue(ObjectProperty->PropertyClass);
		}
		else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(PropertyToTest))
		{
			return MakeValue(StructProperty->Struct);
		}
	}
	return MakeError();
}

const UWidgetBlueprintGeneratedClass* GetComponentPropertyPathSource(TArray<UE::MVVM::FMVVMConstFieldVariant> ComponentPaths, const UWidgetBlueprintGeneratedClass* Accessor)
{
	const UWidgetBlueprintGeneratedClass* ResultWidgetBlueprintGeneratedClass = Accessor;

	for (int32 Index = 0; Index < ComponentPaths.Num(); ++Index)
	{
		const UE::MVVM::FMVVMConstFieldVariant& Field = ComponentPaths[Index];
		if (Field.IsProperty())
		{
			if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Field.GetProperty()))
			{
				if (ObjectProperty->PropertyClass->IsChildOf(UUIComponent::StaticClass()))
				{
					if (const UWidgetBlueprintGeneratedClass* OwnerWidgetBlueprintGeneratedClass = Cast<UWidgetBlueprintGeneratedClass>(ObjectProperty->GetOwnerClass()))
					{
						ResultWidgetBlueprintGeneratedClass = OwnerWidgetBlueprintGeneratedClass;
						break;
					}
				}
			}
		}
	}
	return ResultWidgetBlueprintGeneratedClass;
}

} // namespace

#undef LOCTEXT_NAMESPACE
