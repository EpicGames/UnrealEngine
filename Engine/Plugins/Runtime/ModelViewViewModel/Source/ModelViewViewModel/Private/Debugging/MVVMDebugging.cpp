// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugging/MVVMDebugging.h"

#if UE_WITH_MVVM_DEBUGGING

#include "View/MVVMView.h"

#define LOCTEXT_NAMESPACE "MVVMDebugging"

namespace UE::MVVM
{
/** */
FDebugging::FView::FView(const UMVVMView* InView)
	: UserWidget(InView->GetOuterUUserWidget())
	, View(InView)
{
}

FDebugging::FView::FView(const UUserWidget* InUserWidget, const UMVVMView* InView)
	: UserWidget(InUserWidget)
	, View(InView)
{
}

/** */
FDebugging::FViewConstructed FDebugging::OnViewConstructed;

void FDebugging::BroadcastViewConstructed(const UMVVMView* InView)
{
	OnViewConstructed.Broadcast(InView, FViewConstructedArgs());
}

/** */
FDebugging::FViewDestructing FDebugging::OnViewBeginDestruction;

void FDebugging::BroadcastViewBeginDestruction(const UMVVMView* InView)
{
	OnViewBeginDestruction.Broadcast(InView, FViewBeginDestructionArgs());
}

/** */
FDebugging::FViewSourceValueArgs::FViewSourceValueArgs(const FMVVMViewClass_SourceKey& InClass, const FMVVMView_SourceKey& InView)
	: ClassSource(InClass)
	, ViewSource(InView)
{
}

FDebugging::FViewSourceValueChanged FDebugging::OnViewSourceValueChanged;

void FDebugging::BroadcastViewSourceValueChanged(const UMVVMView* InView, const FMVVMViewClass_SourceKey ClassSourceKey, const FMVVMView_SourceKey ViewSourceKey)
{
	OnViewSourceValueChanged.Broadcast(InView, FViewSourceValueArgs(ClassSourceKey, ViewSourceKey));
}

/** */
FDebugging::FLibraryBindingExecutedArgs::FLibraryBindingExecutedArgs(FMVVMViewClass_BindingKey InBinding)
	: Binding(InBinding)
{
}

FDebugging::FLibraryBindingExecutedArgs::FLibraryBindingExecutedArgs(FMVVMViewClass_BindingKey InBinding, FMVVMCompiledBindingLibrary::EExecutionFailingReason InResult)
	: Binding(InBinding)
	, FailingReason(InResult)
{
}

FDebugging::FLibraryBindingExecuted FDebugging::OnLibraryBindingExecuted;

void FDebugging::BroadcastLibraryBindingExecuted(const UMVVMView* InView, const FLibraryBindingExecutedArgs& Args)
{
	check(InView);
	OnLibraryBindingExecuted.Broadcast(FView(InView), Args);
}

void FDebugging::BroadcastLibraryBindingExecuted(const UMVVMView* InView, FMVVMViewClass_BindingKey InBinding)
{
	if (OnLibraryBindingExecuted.IsBound())
	{
		check(InView);
		OnLibraryBindingExecuted.Broadcast(FView(InView), FLibraryBindingExecutedArgs(InBinding));
	}
}

void FDebugging::BroadcastLibraryBindingExecuted(const UMVVMView* InView, FMVVMViewClass_BindingKey InBinding, FMVVMCompiledBindingLibrary::EExecutionFailingReason InResult)
{
	if (OnLibraryBindingExecuted.IsBound())
	{
		check(InView);
		OnLibraryBindingExecuted.Broadcast(FView(InView), FLibraryBindingExecutedArgs(InBinding, InResult));
	}
}
} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE

#endif //UE_WITH_MVVM_DEBUGGING
