// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/MVVMViewListViewBaseExtension.h"

#include "Bindings/MVVMFieldPathHelper.h"
#include "Components/ListViewBase.h"
#include "MVVMMessageLog.h"
#include "MVVMSubsystem.h"
#include "Templates/ValueOrError.h"
#include "Types/MVVMFieldContext.h"
#include "View/MVVMView.h"
#include "View/MVVMViewClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMViewListViewBaseExtension)

#define LOCTEXT_NAMESPACE "MVVMViewListViewBaseExtension"

#if WITH_EDITOR
void UMVVMViewListViewBaseClassExtension::Initialize(UMVVMViewListViewBaseClassExtension::FInitListViewBaseExtensionArgs InArgs)
{
	WidgetName = InArgs.WidgetName;
	WidgetPath = InArgs.WidgetPath;
	EntryViewModelName = InArgs.EntryViewModelName;
}
#endif

void UMVVMViewListViewBaseClassExtension::OnSourcesInitialized(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension)
{
	check(View->GetViewClass());
	TValueOrError<UE::MVVM::FFieldContext, void> FieldPathResult = View->GetViewClass()->GetBindingLibrary().EvaluateFieldPath(UserWidget, WidgetPath);

	if (FieldPathResult.HasValue())
	{
		TValueOrError<UObject*, void> ObjectResult = UE::MVVM::FieldPathHelper::EvaluateObjectProperty(FieldPathResult.GetValue());
		if (ObjectResult.HasValue() && ObjectResult.GetValue() != nullptr)
		{
			if (UListViewBase* ListWidget = Cast<UListViewBase>(ObjectResult.GetValue()))
			{
				CachedListViewWidgets.Add(FObjectKey(View), ListWidget);
				ListWidget->OnEntryWidgetGenerated().AddUObject(this, &UMVVMViewListViewBaseClassExtension::HandleSetViewModelOnEntryWidget, UserWidget, ListWidget, EntryViewModelName);
			}
			else
			{
				UE::MVVM::FMessageLog Log(UserWidget);
				Log.Error(FText::Format(LOCTEXT("BindToEntryGenerationFailWidgetNotList", "The object property '{0}' is not of type ListViewBase, but has an Viewmodel extension meant for ListViewBase widgets. The extension won't have any effects.")
					, FText::FromName(ObjectResult.GetValue()->GetFName())
				));
			}
		}
		else
		{
			UE::MVVM::FMessageLog Log(UserWidget);
			Log.Error(FText::Format(LOCTEXT("BindToEntryGenerationFailInvalidObjectPropertyWidget", "The property object for list-type widget '{0}' is not found, so viewmodels won't be bound to its entries.")
				, FText::FromName(WidgetName)
			));
		}
	}
	else
	{
		UE::MVVM::FMessageLog Log(UserWidget);
		Log.Error(FText::Format(LOCTEXT("BindToEntryGenerationFailInvalidFieldPathWidget", "The field path for list-type widget '{0}' is invalid, so viewmodels won't be bound to its entries.")
			, FText::FromName(WidgetName)
		));
	}
}

void UMVVMViewListViewBaseClassExtension::OnSourcesUninitialized(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension)
{
	TWeakObjectPtr<UListViewBase> ListWidget;
	if (CachedListViewWidgets.RemoveAndCopyValue(FObjectKey(View), ListWidget))
	{
		if (UListViewBase* ListWidgetPtr = ListWidget.Get())
		{
			ListWidgetPtr->OnEntryWidgetGenerated().RemoveAll(this);
		}
	}
}

void UMVVMViewListViewBaseClassExtension::HandleSetViewModelOnEntryWidget(UUserWidget& EntryWidget, UUserWidget* OwningUserWidget, UListViewBase* ListWidget, FName EntryViewmodelName)
{
	if (UMVVMView* View = UMVVMSubsystem::GetViewFromUserWidget(&EntryWidget))
	{
		if (UObject* EntryObj = ListWidget->GetListObjectFromEntry(EntryWidget))
		{
			if (EntryObj->Implements<UNotifyFieldValueChanged>())
			{
				View->SetViewModel(EntryViewmodelName, EntryObj);
			}
			else
			{
				UE::MVVM::FMessageLog Log(OwningUserWidget);
				Log.Error(FText::Format(LOCTEXT("SetViewModelOnEntryWidgetFailNotViewModel", "Trying to set an object that is not a viewmodel on entries of list-type widget '{0}'. Check the implementation of widget '{0}' and make sure the function GetListObjectFromEntry is overriden to return a viewmodel object. If you do not wish to set viewmodels on the entries of this widget, please remove the corresonding Viewmodel extension from it.")
					, FText::FromName(WidgetName)
				));
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE