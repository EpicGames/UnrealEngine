// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/MVVMViewPanelWidgetExtension.h"

#include "Bindings/MVVMFieldPathHelper.h"
#include "Components/PanelWidget.h"
#include "MVVMMessageLog.h"
#include "MVVMSubsystem.h"
#include "Templates/ValueOrError.h"
#include "Types/MVVMFieldContext.h"
#include "View/MVVMView.h"
#include "View/MVVMViewClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMViewPanelWidgetExtension)

#define LOCTEXT_NAMESPACE "MVVMViewPanelWidgetExtension"

void UMVVMPanelWidgetViewExtension::Initialize(UMVVMViewPanelWidgetClassExtension* InClassExtension, UPanelWidget* InPanelWidget)
{
	ClassExtension = InClassExtension;
	PanelWidget = InPanelWidget;
}

void UMVVMPanelWidgetViewExtension::BP_SetItems(const TArray<UObject*>& InItems)
{
	if (!PanelWidget || !ClassExtension)
	{
		return;
	}

	// Store all the reusable slots in a temporary array so that we don't re-create them.
	TArray<TTuple<UPanelSlot*, TScriptInterface<INotifyFieldValueChanged>>> PreviousSlots;
	for (UPanelSlot* Slot : PanelWidget->GetSlots())
	{
		if (UUserWidget* Content = Cast<UUserWidget>(Slot->Content))
		{
			// The class of the content should strictly match the EntryWidgetClass.
			if (Content->GetClass() == ClassExtension->GetEntryWidgetClass().Get())
			{
				if (UMVVMView* View = UMVVMSubsystem::GetViewFromUserWidget(Content))
				{
					TScriptInterface<INotifyFieldValueChanged> Interface = View->GetViewModel(ClassExtension->GetEntryViewModelName());
					if (Interface.GetObject() && Interface.GetObject()->GetClass() == ClassExtension->GetEntryViewModelClass())
					{
						PreviousSlots.Emplace(Slot, Interface);
					}
				}
			}
		}
	}

	UClass* SelectedVMClass = ClassExtension->GetEntryViewModelClass();
	UUserWidget* OwningUserWidget = GetUserWidget();

	TArray<TTuple<UPanelSlot*, UWidget*>> NewSlots;
	for (int32 ItemIndex = 0; ItemIndex < InItems.Num(); ++ItemIndex)
	{
		if (UObject* Item = InItems[ItemIndex])
		{
			const TTuple<UPanelSlot*, TScriptInterface<INotifyFieldValueChanged>>* FoundObject = PreviousSlots.FindByPredicate([Item](const auto& Other)
				{
					return Other.Value.GetObject() == Item;
				});

			if (Item && OwningUserWidget && Item->GetClass() != SelectedVMClass)
			{
				UE::MVVM::FMessageLog Log(OwningUserWidget);
				Log.Warning(FText::Format(LOCTEXT("SetPanelWidgetItemsViewmodelTypeMismatch", "The item {0} passed as an entry of widget {1} is not a viewmodel of the selected type {2}.")
					, FText::FromString(Item->GetName()), FText::FromString(PanelWidget->GetName()), FText::FromString(SelectedVMClass->GetName())
				));
			}

			if (!FoundObject)
			{
				UUserWidget* EntryWidget = UUserWidget::CreateWidgetInstance(*PanelWidget, ClassExtension->GetEntryWidgetClass(), NAME_None);
				ensure(EntryWidget);

				SetViewModelOnEntryWidget(EntryWidget, Item, OwningUserWidget);
				NewSlots.Add(TTuple<UPanelSlot*, UWidget*>(ClassExtension->GetSlotTemplate(), EntryWidget));
			}
			else
			{
				NewSlots.Add(TTuple<UPanelSlot*, UWidget*>(FoundObject->Key, FoundObject->Key->Content));
				PreviousSlots.RemoveSingleSwap(*FoundObject);
			}
		}
		else
		{
			UE::MVVM::FMessageLog Log(OwningUserWidget);
			Log.Warning(FText::Format(LOCTEXT("SetPanelWidgetItemsViewmodelNullObject", "The item at index {0} passed as an entry of widget {1} is null. An entry widget won't be generated for this item.")
				, ItemIndex, FText::FromString(PanelWidget->GetName())
			));
		}
	}

	ReplaceAllSlots(NewSlots);
}

void UMVVMPanelWidgetViewExtension::SetViewModelOnEntryWidget(UUserWidget* EntryWidget, UObject* ViewModelObject, UUserWidget* OwningUserWidget)
{
	if (UMVVMView* View = UMVVMSubsystem::GetViewFromUserWidget(EntryWidget))
	{
		if (ViewModelObject->Implements<UNotifyFieldValueChanged>())
		{
			View->SetViewModel(ClassExtension->GetEntryViewModelName(), ViewModelObject);
		}
		else
		{
			if (OwningUserWidget)
			{
				UE::MVVM::FMessageLog Log(OwningUserWidget);
				Log.Error(FText::Format(LOCTEXT("SetViewModelOnEntryWidgetFailNotViewModel", "Trying to set an object that is not a viewmodel on entries of panel-type widget '{0}'. If you do not wish to set viewmodels on the entries of this widget, please remove the corresonding Viewmodel extension from it.")
					, FText::FromName(ClassExtension->GetWidgetName())
				));
			}
		}
	}
}

void UMVVMPanelWidgetViewExtension::ReplaceAllSlots(TArrayView<TTuple<UPanelSlot*, UWidget*>> NewSlots)
{
	if (!PanelWidget || !ClassExtension)
	{
		return;
	}

	const int32 OldSlotsNum = PanelWidget->GetSlots().Num();
	const int32 NewSlotsNum = NewSlots.Num();
	const int32 MinSlotNum = FMath::Min(OldSlotsNum, NewSlotsNum);

	// as long as we're within the boundaries of both arrays, compare and replace elements
	for (int32 SlotIndex = 0; SlotIndex < MinSlotNum; ++SlotIndex)
	{
		PanelWidget->RemoveChildAt(SlotIndex);
		UPanelSlot* NewSlot = PanelWidget->InsertChildAt(SlotIndex, NewSlots[SlotIndex].Value, NewSlots[SlotIndex].Key);
	}

	// If we have more old slots than new ones, remove all the extra ones.
	if (OldSlotsNum > NewSlotsNum)
	{
		for (int32 SlotIndex = OldSlotsNum - 1; SlotIndex >= NewSlotsNum; SlotIndex--)
		{
			PanelWidget->RemoveChildAt(SlotIndex);
		}
	}
	// If we have more new slots than old ones, simply append the new ones.
	else if (NewSlotsNum > OldSlotsNum)
	{
		for (int32 SlotIndex = OldSlotsNum; SlotIndex < NewSlotsNum; SlotIndex++)
		{
			UPanelSlot* NewSlot = PanelWidget->AddChild(NewSlots[SlotIndex].Value, NewSlots[SlotIndex].Key);
		}
	}

	OnItemsChanged.Broadcast();
}

UUserWidget* UMVVMPanelWidgetViewExtension::GetUserWidget() const
{
	return GetView()->GetOuterUUserWidget();
}

#if WITH_EDITOR
void UMVVMViewPanelWidgetClassExtension::Initialize(UMVVMViewPanelWidgetClassExtension::FInitPanelWidgetExtensionArgs InArgs)
{
	WidgetName = InArgs.WidgetName;
	WidgetPath = InArgs.WidgetPath;
	EntryViewModelName = InArgs.EntryViewModelName;
	EntryWidgetClass = InArgs.EntryWidgetClass;
	SlotTemplate = DuplicateObject(InArgs.SlotTemplate, this);
	PanelPropertyName = InArgs.PanelPropertyName;
	EntryViewModelClass = InArgs.EntryViewModelClass;
}
#endif

UMVVMViewExtension* UMVVMViewPanelWidgetClassExtension::ViewConstructed(UUserWidget* UserWidget, UMVVMView* View)
{
	check(View->GetViewClass());

	UMVVMPanelWidgetViewExtension* ResultExtension = nullptr;

	// Fetch and cache the panel widget
	TValueOrError<UE::MVVM::FFieldContext, void> FieldPathResult = View->GetViewClass()->GetBindingLibrary().EvaluateFieldPath(UserWidget, WidgetPath);

	if (FieldPathResult.HasValue())
	{
		TValueOrError<UObject*, void> ObjectResult = UE::MVVM::FieldPathHelper::EvaluateObjectProperty(FieldPathResult.GetValue());
		if (ObjectResult.HasValue() && ObjectResult.GetValue() != nullptr)
		{
			if (UPanelWidget* PanelWidget = Cast<UPanelWidget>(ObjectResult.GetValue()))
			{
				ResultExtension = NewObject<UMVVMPanelWidgetViewExtension>(View);
				ResultExtension->Initialize(this, PanelWidget);

				// Set the extension object on the runtime user widget.
				FObjectPropertyBase* FoundPanelObjectProperty = FindFProperty<FObjectPropertyBase>(UserWidget->GetClass(), PanelPropertyName);
				bool bValidPanelObject = FoundPanelObjectProperty && FoundPanelObjectProperty->PropertyClass->IsChildOf(UMVVMPanelWidgetViewExtension::StaticClass());
				if (ensureAlwaysMsgf(bValidPanelObject, TEXT("The compiler should have added the property")))
				{
					FoundPanelObjectProperty->SetObjectPropertyValue_InContainer(UserWidget, ResultExtension);
				}
			}
			else
			{
				UE::MVVM::FMessageLog Log(UserWidget);
				Log.Error(FText::Format(LOCTEXT("BindToEntryGenerationFailWidgetNotPanel", "The object property '{0}' is not of type panel widget, but has an Viewmodel extension meant for panel widgets. The extension won't have any effects.")
					, FText::FromName(ObjectResult.GetValue()->GetFName())
				));
			}
		}
		else
		{
			UE::MVVM::FMessageLog Log(UserWidget);
			Log.Error(FText::Format(LOCTEXT("BindToEntryGenerationFailInvalidObjectPropertyWidget", "The property object for panel-type widget '{0}' is not found, so viewmodels won't be bound to its entries.")
				, FText::FromName(WidgetName)
			));
		}
	}
	else
	{
		UE::MVVM::FMessageLog Log(UserWidget);
		Log.Error(FText::Format(LOCTEXT("BindToEntryGenerationFailInvalidFieldPathWidget", "The field path for panel-type widget '{0}' is invalid, so viewmodels won't be bound to its entries.")
			, FText::FromName(WidgetName)
		));
	}

	return ResultExtension;
}

void UMVVMViewPanelWidgetClassExtension::OnViewDestructed(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension)
{
	FObjectPropertyBase* FoundPanelObjectProperty = FindFProperty<FObjectPropertyBase>(UserWidget->GetClass(), PanelPropertyName);
	if (ensureAlwaysMsgf(FoundPanelObjectProperty, TEXT("The compiler should have added the property")))
	{
		FoundPanelObjectProperty->SetObjectPropertyValue_InContainer(UserWidget, nullptr);
	}
}

#undef LOCTEXT_NAMESPACE