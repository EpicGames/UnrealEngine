// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBlueprintLibrary.h"

#include "Blueprint/UserWidget.h"
#include "MVVMMessageLog.h"
#include "View/MVVMView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBlueprintLibrary)

#define LOCTEXT_NAMESPACE "MVVMBlueprintLibrary"

void UMVVMBlueprintLibrary::SetViewModelByClass(UUserWidget* Widget, TScriptInterface<INotifyFieldValueChanged> ViewModel)
{
	if (UMVVMView* View = Widget ? Widget->GetExtension<UMVVMView>() : nullptr)
	{
		View->SetViewModelByClass(ViewModel);
	}
	else
	{
		UE::MVVM::FMessageLog Log(Widget); // Null widget is ok for log
		if (Widget)
		{
			Log.Error(LOCTEXT("MissingView", "Widget does not have an MVVMView, cannot set any View Model"));
		}
		else
		{
			Log.Error(LOCTEXT("MissingWidget", "No widget passed into this function"));
		}
	}
}

#undef LOCTEXT_NAMESPACE
