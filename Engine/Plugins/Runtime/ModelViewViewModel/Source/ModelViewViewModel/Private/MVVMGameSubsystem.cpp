// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMGameSubsystem.h"

#include "Types/MVVMViewModelContextInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMGameSubsystem)


void UMVVMGameSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	ViewModelCollection = NewObject<UMVVMViewModelCollectionObject>(this);
}


void UMVVMGameSubsystem::Deinitialize()
{
	ViewModelCollection = nullptr;
	Super::Deinitialize();
}
