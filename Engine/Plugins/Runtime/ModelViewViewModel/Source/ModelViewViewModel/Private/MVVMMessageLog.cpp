// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMMessageLog.h"

#include "Blueprint/UserWidget.h"
#include "View/MVVMViewClass.h"

#include "Logging/TokenizedMessage.h"
#include "Misc/UObjectToken.h"

#if WITH_EDITOR
#include "Kismet2/KismetEditorUtilities.h" //UnrealEd
#endif

#define LOCTEXT_NAMESPACE "FMVVMMessageLog"

DEFINE_LOG_CATEGORY(LogMVVM);
const FName UE::MVVM::FMessageLog::LogName = "Model View Viewmodel";


namespace UE::MVVM
{

namespace Private
{
void OnMessageLogLinkActivated_UObjectToken(const TSharedRef<IMessageToken>& Token)
{
#if WITH_EDITOR
	if (Token->GetType() == EMessageToken::Object)
	{
		const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(Token);
		if (const UClass* UserWidget = Cast<UClass>(UObjectToken->GetObject().Get()))
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(UserWidget);
		}
	}
#endif
}
void OnMessageLogLinkActivated_Binding(TWeakObjectPtr<const UUserWidget> WeakUserWidget, FGuid BindingId)
{
#if WITH_EDITOR
		if (const UUserWidget* UserWidget = WeakUserWidget.Get())
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(UserWidget->GetClass());
		}
#endif
}

void AddUserWidget(TSharedRef<FTokenizedMessage>& Message, const UUserWidget* UserWidget)
{
#if WITH_EDITOR
	if (UserWidget)
	{
		Message->AddToken(FUObjectToken::Create(UserWidget->GetClass(), FText::FromName(UserWidget->GetFName()))
			->OnMessageTokenActivated(FOnMessageTokenActivated::CreateStatic(&OnMessageLogLinkActivated_UObjectToken))
		);
	}
#else
	if (UserWidget)
	{
		Message->AddToken(FTextToken::Create(FText::FromName(UserWidget->GetFName())));
	}
#endif
}
} //namespace Private

FMessageLog::FMessageLog(const UUserWidget* InUserWidget)
	: ::FMessageLog(LogName)
	, UserWidget(InUserWidget)
{}

TSharedRef<FTokenizedMessage> FMessageLog::Message(EMessageSeverity::Type InSeverity, const FText& InMessage)
{
	TSharedRef<FTokenizedMessage> NewMessage = FTokenizedMessage::Create(InSeverity);
	NewMessage->AddToken(FTextToken::Create(InMessage));
	Private::AddUserWidget(NewMessage, UserWidget.Get());
	AddMessage(NewMessage);
	return NewMessage;
}

TSharedRef<FTokenizedMessage> FMessageLog::Error(const FText& InMessage)
{
	return Message(EMessageSeverity::Error, InMessage);
}

TSharedRef<FTokenizedMessage> FMessageLog::PerformanceWarning(const FText& InMessage)
{
	return Message(EMessageSeverity::PerformanceWarning, InMessage);
}

TSharedRef<FTokenizedMessage> FMessageLog::Warning(const FText& InMessage)
{
	return Message(EMessageSeverity::Warning, InMessage);
}

TSharedRef<FTokenizedMessage> FMessageLog::Info(const FText& InMessage)
{
	return Message(EMessageSeverity::Info, InMessage);
}

void FMessageLog::AddBindingToken(TSharedRef<FTokenizedMessage> NewMessage, const UMVVMViewClass* Class, const FMVVMViewClass_Binding& ClassBinding, FMVVMViewClass_BindingKey Key)
{
#if UE_WITH_MVVM_DEBUGGING && WITH_EDITOR
	NewMessage->AddToken(
		FActionToken::Create(
			LOCTEXT("BindingToken", "binding")
			, FText::FromString(ClassBinding.ToString(Class, FMVVMViewClass_Binding::FToStringArgs::Short()))
			, FOnActionTokenExecuted::CreateStatic(&Private::OnMessageLogLinkActivated_Binding, UserWidget, ClassBinding.GetEditorId())
		)
	);
#else
	NewMessage->AddToken(
		FTextToken::Create(
			FText::Format(LOCTEXT("BindingTokenToken", "binding: {0}"), FText::AsNumber(Key.GetIndex()))
		)
	);
#endif
}

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE