// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMSubsystem.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Blueprint/WidgetTree.h"
#include "Templates/ValueOrError.h"
#include "View/MVVMView.h"
#include "Types/MVVMAvailableBinding.h"
#include "Kismet/KismetMathLibrary.h"

#if WITH_EDITORONLY_DATA
#include "Engine/Blueprint.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMSubsystem)

#define LOCTEXT_NAMESPACE "MVVMSubsystem"

void UMVVMSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
}


void UMVVMSubsystem::Deinitialize()
{
	Super::Deinitialize();
}

bool UMVVMSubsystem::K2_CompareFloatValues(EMVVMConditionOperation Operation, float Value, float CompareValue, float CompareMaxValue)
{
	switch (Operation)
	{
	case EMVVMConditionOperation::Equal:
	{
		return UKismetMathLibrary::NearlyEqual_FloatFloat(Value, CompareValue);
	}
	case EMVVMConditionOperation::NotEqual:
	{
		return !UKismetMathLibrary::NearlyEqual_FloatFloat(Value, CompareValue);
	}
	case EMVVMConditionOperation::MoreThan:
	{
		return UKismetMathLibrary::Greater_DoubleDouble(Value, CompareValue);
	}
	case EMVVMConditionOperation::MoreThanOrEqual:
	{
		return UKismetMathLibrary::GreaterEqual_DoubleDouble(Value, CompareValue);
	}
	case EMVVMConditionOperation::LessThan:
	{
		return UKismetMathLibrary::Less_DoubleDouble(Value, CompareValue);
	}
	case EMVVMConditionOperation::LessThanOrEqual:
	{
		return UKismetMathLibrary::LessEqual_DoubleDouble(Value, CompareValue);
	}
	case EMVVMConditionOperation::BetweenExclusive:
	case EMVVMConditionOperation::BetweenInclusive:
	{
		float Smallest = CompareValue;
		float Largest = CompareMaxValue;
		if (Smallest > Largest)
		{
			Largest = CompareValue;
			Smallest = CompareMaxValue;
		}
		if (Operation == EMVVMConditionOperation::BetweenExclusive)
		{
			return UKismetMathLibrary::Greater_DoubleDouble(Value, Smallest) && UKismetMathLibrary::Less_DoubleDouble(Value, Largest);
		}
		else
		{
			return UKismetMathLibrary::GreaterEqual_DoubleDouble(Value, Smallest) && UKismetMathLibrary::LessEqual_DoubleDouble(Value, Largest);
		}
	}
	default : 
		break;
	}
	return false;
}


UMVVMView* UMVVMSubsystem::K2_GetViewFromUserWidget(const UUserWidget* UserWidget) const
{
	return GetViewFromUserWidget(UserWidget);
}


UMVVMView* UMVVMSubsystem::GetViewFromUserWidget(const UUserWidget* UserWidget)
{
	return UserWidget ? UserWidget->GetExtension<UMVVMView>() : nullptr;
}


bool UMVVMSubsystem::DoesWidgetTreeContainedWidget(const UWidgetTree* WidgetTree, const UWidget* ViewWidget) const
{
	// Test if the View's Widget is valid.
	if (WidgetTree != nullptr && ViewWidget != nullptr)
	{
		TArray<UWidget*> Widgets;
		WidgetTree->GetAllWidgets(Widgets);
		return Widgets.Contains(ViewWidget);
	}
	return false;
}


namespace UE::MVVM::Private
{

	FMVVMAvailableBinding GetAvailableBinding(const UFunction* Function, bool bHasNotify, bool bCanAccessPrivate, bool bCanAccessProtected, const bool bIsBindingToEvent)
	{
		// N.B. A function can be private/protected and can still be use when it's a BlueprintGetter/BlueprintSetter.
		//But we bind to the property not the function.
		static FName Name_AllowPrivateAccess = TEXT("AllowPrivateAccess");
#if WITH_EDITOR
		if (Function->HasAnyFunctionFlags(FUNC_Private) && !bCanAccessPrivate)
		{
			if (!Function->GetBoolMetaData(Name_AllowPrivateAccess))
			{
				return FMVVMAvailableBinding();
			}
		}
		if (Function->HasAnyFunctionFlags(FUNC_Protected) && !bCanAccessProtected)
		{
			if (!Function->GetBoolMetaData(Name_AllowPrivateAccess))
			{
				return FMVVMAvailableBinding();
			}
		}
#endif

		const bool bIsReadable = BindingHelper::IsValidForSourceBinding(Function);
		const bool bIsWritable = BindingHelper::IsValidForDestinationBinding(Function);
		const bool bIsForEvent = BindingHelper::IsValidForEventBinding(Function);
		if (bIsBindingToEvent)
		{
			if (bIsForEvent)
			{
				return FMVVMAvailableBinding(FMVVMBindingName(Function->GetFName()), bIsReadable, true, bHasNotify && bIsReadable);
			}
		}
		else if (bIsReadable || bIsWritable || bHasNotify)
		{
			return FMVVMAvailableBinding(FMVVMBindingName(Function->GetFName()), bIsReadable, bIsWritable, bHasNotify && bIsReadable);
		}

		return FMVVMAvailableBinding();
	}

	FMVVMAvailableBinding GetAvailableBinding(const FProperty* Property, bool bHasNotify, bool bCanAccessPrivate, bool bCanAccessProtected)
	{
		// N.B. Property can be private/protected in cpp but if they are visible in BP that is all that matter.
		//Only property defined in BP can be BP visible and really private.
#if WITH_EDITOR
		static FName NAME_BlueprintPrivate = "BlueprintPrivate";
		if (Property->HasMetaData(NAME_BlueprintPrivate) && !bCanAccessPrivate)
		{
			return FMVVMAvailableBinding();
		}
		static FName NAME_BlueprintProtected = "BlueprintProtected";
		if (Property->HasMetaData(NAME_BlueprintProtected) && !bCanAccessProtected)
		{
			return FMVVMAvailableBinding();
		}
#endif

		const bool bIsReadable = BindingHelper::IsValidForSourceBinding(Property);
		const bool bIsWritable = BindingHelper::IsValidForDestinationBinding(Property);
		if (bIsReadable || bIsWritable || bHasNotify)
		{
			return FMVVMAvailableBinding(FMVVMBindingName(Property->GetFName()), bIsReadable, bIsWritable, bHasNotify && bIsReadable);
		}

		return FMVVMAvailableBinding();
	}

	TArray<FMVVMAvailableBinding> GetAvailableBindings(const UStruct* Container, const UClass* AccessorType, const UE::FieldNotification::IClassDescriptor* ClassDescriptor, const bool bIsBindingToEvent)
	{
		TSet<FName> FieldDescriptors;
		if (ClassDescriptor)
		{
			ClassDescriptor->ForEachField(CastChecked<UClass>(Container), [&FieldDescriptors](UE::FieldNotification::FFieldId FieldId)
			{
				FieldDescriptors.Add(FieldId.GetName());
				return true;
			});
		}

		TArray<FMVVMAvailableBinding> Result;
		Result.Reserve(FieldDescriptors.Num());

		const bool bCanAccessPrivateMember = Container == AccessorType;
		const bool bCanAccessProtectedMember = AccessorType ? AccessorType->IsChildOf(Container) : false;

		if (Cast<UClass>(Container))
		{
			for (TFieldIterator<const UFunction> FunctionIt(Container, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
			{
				const UFunction* Function = *FunctionIt;
				check(Function);

				const bool bHasNotify = FieldDescriptors.Contains(Function->GetFName());
				FMVVMAvailableBinding Binding = GetAvailableBinding(Function, bHasNotify, bCanAccessPrivateMember, bCanAccessProtectedMember, bIsBindingToEvent);
				if (Binding.IsValid())
				{
					Result.Add(Binding);
				}
			}
		}

#if WITH_EDITOR
		FName NAME_BlueprintGetter = "BlueprintGetter";
		FName NAME_BlueprintSetter = "BlueprintSetter";
#endif

		for (TFieldIterator<const FProperty> PropertyIt(Container, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			const FProperty* Property = *PropertyIt;
			check(Property);

			const bool bHasNotify = FieldDescriptors.Contains(Property->GetFName());
			FMVVMAvailableBinding Binding = GetAvailableBinding(Property, bHasNotify, bCanAccessPrivateMember, bCanAccessProtectedMember);
			if (Binding.IsValid())
			{
#if WITH_EDITOR
				// Remove any BlueprintGetter & BlueprintSetter and use the Property instead.
				if (Binding.IsReadable())
				{
					const FString& PropertyGetter = Property->GetMetaData(NAME_BlueprintGetter);
					if (!PropertyGetter.IsEmpty())
					{
						FMVVMBindingName BindingName = FMVVMBindingName(*PropertyGetter);
						int32 BindingIndex = Result.IndexOfByPredicate([BindingName](const FMVVMAvailableBinding& Binding) { return Binding.GetBindingName() == BindingName && Binding.IsReadable() && !Binding.IsWritable() && !Binding.HasNotify(); });
						if (BindingIndex != INDEX_NONE)
						{
							Result.RemoveAtSwap(BindingIndex);
						}
					}
				}
				if (Binding.IsWritable())
				{
					const FString& PropertySetter = Property->GetMetaData(NAME_BlueprintSetter);
					if (!PropertySetter.IsEmpty())
					{
						FMVVMBindingName BindingName = FMVVMBindingName(*PropertySetter);
						int32 BindingIndex = Result.IndexOfByPredicate([BindingName](const FMVVMAvailableBinding& Binding) { return Binding.GetBindingName() == BindingName && Binding.IsWritable() && !Binding.IsReadable(); });
						if (BindingIndex != INDEX_NONE)
						{
							Result.RemoveAtSwap(BindingIndex);
						}
					}
				}
#endif
				Result.Add(Binding);
			}
		}

		return Result;
	}

	TArray<FMVVMAvailableBinding> GetAvailableBindings(const UClass* InSubClass, const UClass* InAccessor, const bool bIsBindingToEvent)
	{
		if (InSubClass && InSubClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
		{
			TScriptInterface<INotifyFieldValueChanged> DefaultObject = InSubClass->GetDefaultObject();
			const UE::FieldNotification::IClassDescriptor& ClassDescriptor = DefaultObject->GetFieldNotificationDescriptor();
			return GetAvailableBindings(InSubClass, InAccessor, &ClassDescriptor, bIsBindingToEvent);
		}
		else
		{
			return GetAvailableBindings(InSubClass, InAccessor, nullptr, bIsBindingToEvent);
		}
	}


	FMVVMAvailableBinding GetAvailableBinding(FMVVMConstFieldVariant FieldVariant, const UClass* AccessorType, const UE::FieldNotification::IClassDescriptor* ClassDescriptor, const bool bIsBindingToEvent)
	{
		bool bHasNotify = false;
		if (ClassDescriptor)
		{
			UE::FieldNotification::FFieldId FieldId = ClassDescriptor->GetField(CastChecked<UClass>(FieldVariant.GetOwner()), FieldVariant.GetName());
			bHasNotify = FieldId.IsValid();
		}

		const bool bCanAccessPrivateMember = FieldVariant.GetOwner() == AccessorType;
		const bool bCanAccessProtectedMember = AccessorType ? AccessorType->IsChildOf(FieldVariant.GetOwner()) : false;

		if (FieldVariant.IsProperty())
		{
			return GetAvailableBinding(FieldVariant.GetProperty(), bHasNotify, bCanAccessPrivateMember, bCanAccessProtectedMember);
		}
		else if (FieldVariant.IsFunction())
		{
			return GetAvailableBinding(FieldVariant.GetFunction(), bHasNotify, bCanAccessPrivateMember, bCanAccessProtectedMember, bIsBindingToEvent);
		}

		return FMVVMAvailableBinding();
	}


	FMVVMAvailableBinding GetAvailableBinding(FMVVMConstFieldVariant FieldVariant, const UClass* InAccessor, const bool bIsBindingToEvent)
	{
		const UClass* OwnerClass = Cast<UClass>(FieldVariant.GetOwner());
		if (OwnerClass && OwnerClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
		{
#if WITH_EDITORONLY_DATA
			// Can't access the CDO while changing layout, fallback to skeleton
			if (OwnerClass->bLayoutChanging)
			{
				const UBlueprint* OwnerBlueprint = Cast<UBlueprint>(OwnerClass->ClassGeneratedBy);
				if (OwnerBlueprint && OwnerBlueprint->SkeletonGeneratedClass)
				{
					OwnerClass = OwnerBlueprint->SkeletonGeneratedClass;
				}
			}
#endif
			TScriptInterface<INotifyFieldValueChanged> DefaultObject = OwnerClass->GetDefaultObject();
			const UE::FieldNotification::IClassDescriptor& ClassDescriptor = DefaultObject->GetFieldNotificationDescriptor();
			return GetAvailableBinding(FieldVariant, InAccessor, &ClassDescriptor, bIsBindingToEvent);
		}
		else
		{
			return GetAvailableBinding(FieldVariant, InAccessor, nullptr, bIsBindingToEvent);
		}
	}

	FMVVMAvailableBinding GetAvailableBinding(FMVVMBindingName BindingName, const UClass* InSubClass, const UClass* InAccessor, const bool bIsBindingToEvent)
	{
		if (!BindingName.IsValid())
		{
			return FMVVMAvailableBinding();
		}

		FMVVMConstFieldVariant FieldVariant = UE::MVVM::BindingHelper::FindFieldByName(InSubClass, BindingName);
		return GetAvailableBinding(FieldVariant, InAccessor, bIsBindingToEvent);
	}
} //namespace


TArray<FMVVMAvailableBinding> UMVVMSubsystem::K2_GetAvailableBindings(const UClass* Class, const UClass* Accessor) const
{
	return GetAvailableBindings(Class, Accessor);
}


TArray<FMVVMAvailableBinding> UMVVMSubsystem::GetAvailableBindings(const UClass* Class, const UClass* Accessor)
{
	return UE::MVVM::Private::GetAvailableBindings(Class, Accessor, false);
}


TArray<FMVVMAvailableBinding> UMVVMSubsystem::GetAvailableBindingsForStruct(const UScriptStruct* Struct)
{
	return UE::MVVM::Private::GetAvailableBindings(Struct, nullptr, nullptr, false);
}

TArray<FMVVMAvailableBinding> UMVVMSubsystem::GetAvailableBindingsForEvent(const UClass* Class, const UClass* Accessor)
{
	return UE::MVVM::Private::GetAvailableBindings(Class, Accessor, true);
}

FMVVMAvailableBinding UMVVMSubsystem::K2_GetAvailableBinding(const UClass* Class, FMVVMBindingName BindingName, const UClass* Accessor) const
{
	return GetAvailableBinding(Class, BindingName, Accessor);
}


FMVVMAvailableBinding UMVVMSubsystem::GetAvailableBinding(const UClass* Class, FMVVMBindingName BindingName, const UClass* Accessor)
{
	return UE::MVVM::Private::GetAvailableBinding(BindingName, Class, Accessor, false);
}


FMVVMAvailableBinding UMVVMSubsystem::GetAvailableBindingForField(UE::MVVM::FMVVMConstFieldVariant FieldVariant, const UClass* Accessor)
{
	return UE::MVVM::Private::GetAvailableBinding(FieldVariant, Accessor, false);
}

FMVVMAvailableBinding UMVVMSubsystem::GetAvailableBindingForEvent(UE::MVVM::FMVVMConstFieldVariant FieldVariant, const UClass* Accessor)
{
	return UE::MVVM::Private::GetAvailableBinding(FieldVariant, Accessor, true);
}

FMVVMAvailableBinding UMVVMSubsystem::GetAvailableBindingForEvent(const UClass* Class, FMVVMBindingName BindingName, const UClass* Accessor)
{
	return UE::MVVM::Private::GetAvailableBinding(BindingName, Class, Accessor, true);
}

UMVVMViewModelCollectionObject* UMVVMSubsystem::GetGlobalViewModelCollection() const
{
	return nullptr;
}


TValueOrError<bool, FText> UMVVMSubsystem::IsBindingValid(FBindingArgs Args) const
{
	if (UE::MVVM::IsForwardBinding(Args.Mode))
	{
		TValueOrError<bool, FText> ForwardDirection = IsBindingValid(Args.ForwardArgs);
		if (ForwardDirection.HasError())
		{
			return ForwardDirection;
		}
	}

	if (UE::MVVM::IsBackwardBinding(Args.Mode))
	{
		TValueOrError<bool, FText> BackwardDirection = IsBindingValid(Args.BackwardArgs);
		if (BackwardDirection.HasError())
		{
			return BackwardDirection;
		}
	}

	return MakeValue(true);
}

TValueOrError<bool, FText> UMVVMSubsystem::IsBindingValid(FDirectionalBindingArgs Args) const
{
	return IsBindingValid(Args.ToConst());
}

TValueOrError<bool, FText> UMVVMSubsystem::IsBindingValid(FConstDirectionalBindingArgs Args) const
{
	const bool bIsSimpleConversionFunction = Args.ConversionFunction && UE::MVVM::BindingHelper::IsValidForSimpleRuntimeConversion(Args.ConversionFunction);
	const bool bIsComplexConversionFunction = Args.ConversionFunction && UE::MVVM::BindingHelper::IsValidForComplexRuntimeConversion(Args.ConversionFunction);
	const bool bIsDelegateSignatureBinding = Args.ConversionFunction && UE::MVVM::BindingHelper::IsValidForDelegateSignatureBinding(Args.ConversionFunction);

	// Test Source
	const FProperty* SourceProperty = nullptr;
	if (!bIsComplexConversionFunction && !bIsDelegateSignatureBinding)
	{
		TValueOrError<const FProperty*, FText> SourceResult = UE::MVVM::BindingHelper::TryGetPropertyTypeForSourceBinding(Args.SourceBinding);
		if (SourceResult.HasError())
		{
			return MakeError(SourceResult.StealError());
		}

		SourceProperty = SourceResult.StealValue();
		if (SourceProperty == nullptr)
		{
			return MakeError(LOCTEXT("NoValueToBindAtSource", "There is no value to bind at the source."));
		}
	}

	// Test Destination
	const FProperty* DestinationProperty = nullptr;
	{
		TValueOrError<const FProperty*, FText> DestinationResult = UE::MVVM::BindingHelper::TryGetPropertyTypeForDestinationBinding(Args.DestinationBinding);
		if (DestinationResult.HasError())
		{
			return MakeError(DestinationResult.StealError());
		}

		DestinationProperty = DestinationResult.StealValue();
		if (DestinationProperty == nullptr)
		{
			return MakeError(LOCTEXT("NoValueToBindAtDestination", "There is no value to bind at the destination."));
		}
	}

	auto GetPropertyType = [](const FProperty* Property)
	{
		if (!Property)
		{
			return FString("");
		}

		FString InnerType;
		FString CppType = Property->GetCPPType(&InnerType);
		if (InnerType.Len())
		{
			CppType.Append(InnerType);
		}
		return CppType;
	};

	// Async events do not need a return property, their graph will handle value setting, simply pass compilation
	if (bIsDelegateSignatureBinding)
	{
		return MakeValue(true);
	}

	// Test the conversion function
	if (Args.ConversionFunction)
	{
		TValueOrError<const FProperty*, FText> ReturnResult = UE::MVVM::BindingHelper::TryGetReturnTypeForConversionFunction(Args.ConversionFunction);
		if (ReturnResult.HasError())
		{
			return MakeError(ReturnResult.StealError());
		}

		if (!bIsSimpleConversionFunction && !bIsComplexConversionFunction)
		{
			return MakeError(FText::Format(LOCTEXT("ConversionFunctionNotValid", "The conversion function '{0}' is not valid as a conversion function"), FText::FromString(Args.ConversionFunction->GetName())));
		}

		// The simple compiled version look like Setter(Conversion(Getter())).
		if (bIsSimpleConversionFunction)
		{
			TValueOrError<TArray<const FProperty*>, FText> ArgumentsResult = UE::MVVM::BindingHelper::TryGetArgumentsForConversionFunction(Args.ConversionFunction);
			if (ArgumentsResult.HasError())
			{
				return MakeError(ArgumentsResult.StealError());
			}
			if (ArgumentsResult.GetValue().Num() != 1)
			{
				return MakeError(LOCTEXT("NumberOfArgumentsWrong", "Internal error. The number of arguments should be 1."));
			}

			bool bAnyCompatible = false;
			const TArray<const FProperty*>& ArgumentProperties = ArgumentsResult.GetValue();
			for (const FProperty* ArgumentProperty : ArgumentProperties)
			{
				if (UE::MVVM::BindingHelper::ArePropertiesCompatible(SourceProperty, ArgumentProperty))
				{
					bAnyCompatible = true;
					break;
				}
			}

			if (!bAnyCompatible)
			{
				TArray<FText> ArgumentTypes;
				ArgumentTypes.Reserve(ArgumentProperties.Num());
				FString InnerType;
				for (const FProperty* ArgProperty : ArgumentProperties)
				{
					ArgumentTypes.Add(FText::FromString(GetPropertyType(ArgProperty)));
				}

				return MakeError(FText::Format(LOCTEXT("SourcePropertyDoesNotMatchArguments", "The source property '{0}' ({1}) does not match any of the argument types of the conversion function ({1})."), 
					FText::FromString(SourceProperty->GetName()), 
					FText::FromString(GetPropertyType(SourceProperty)),
					FText::Join(LOCTEXT("CommaDelim", ", "), ArgumentTypes))
				);
			}
		}
		else
		{
			// The complex compiled version look like Setter(Conversion()) (no argument)
			check(bIsComplexConversionFunction);
			const FProperty* ReturnProperty = ReturnResult.GetValue();
			if (!UE::MVVM::BindingHelper::ArePropertiesCompatible(ReturnProperty, DestinationProperty))
			{
				return MakeError(FText::Format(LOCTEXT("DestinationPropertyDoesNotMatchReturn", "The destination property '{0}' ({1}) does not match the return type of the conversion function ({2})."), 
					FText::FromString(DestinationProperty->GetName()), 
					FText::FromString(GetPropertyType(DestinationProperty)),
					FText::FromString(ReturnProperty->GetCPPType()))
				);
			}
		}
	}
	else if (!UE::MVVM::BindingHelper::ArePropertiesCompatible(SourceProperty, DestinationProperty))
	{
		return MakeError(FText::Format(LOCTEXT("SourcePropertyDoesNotMatchDestination", "The source property '{0}' ({1}) does not match the type of the destination property '{2}' ({3}). A conversion function is required."), 
			FText::FromString(SourceProperty->GetName()),
			FText::FromString(GetPropertyType(SourceProperty)),
			FText::FromString(DestinationProperty->GetName()), 
			FText::FromString(GetPropertyType(DestinationProperty)))
		);
	}

	return MakeValue(true);
}

#undef LOCTEXT_NAMESPACE
