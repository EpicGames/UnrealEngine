// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMViewModelBase.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "Engine/BlueprintGeneratedClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMViewModelBase)

#define LOCTEXT_NAMESPACE "MVVMViewModelBase"

FDelegateHandle UMVVMViewModelBase::AddFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FFieldValueChangedDelegate InNewDelegate)
{
	return NotificationDelegates.AddFieldValueChangedDelegate(this, InFieldId, MoveTemp(InNewDelegate));
}


void UMVVMViewModelBase::K2_AddFieldValueChangedDelegate(FFieldNotificationId InFieldId, FFieldValueChangedDynamicDelegate InDelegate)
{
	if (InFieldId.IsValid())
	{
		const UE::FieldNotification::FFieldId FieldId = GetFieldNotificationDescriptor().GetField(GetClass(), InFieldId.FieldName);
		ensureMsgf(FieldId.IsValid(), TEXT("The field should be compiled correctly."));
		NotificationDelegates.AddFieldValueChangedDelegate(this, FieldId, InDelegate);
	}
}


bool UMVVMViewModelBase::RemoveFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FDelegateHandle InHandle)
{
	return NotificationDelegates.RemoveFieldValueChangedDelegate(this, InFieldId, InHandle);
}


void UMVVMViewModelBase::K2_RemoveFieldValueChangedDelegate(FFieldNotificationId InFieldId, FFieldValueChangedDynamicDelegate InDelegate)
{
	if (InFieldId.IsValid())
	{
		const UE::FieldNotification::FFieldId FieldId = GetFieldNotificationDescriptor().GetField(GetClass(), InFieldId.FieldName);
		ensureMsgf(FieldId.IsValid(), TEXT("The field should be compiled correctly."));
		NotificationDelegates.RemoveFieldValueChangedDelegate(this, FieldId, MoveTemp(InDelegate));
	}
}


int32 UMVVMViewModelBase::RemoveAllFieldValueChangedDelegates(FDelegateUserObjectConst InUserObject)
{
	return NotificationDelegates.RemoveAllFieldValueChangedDelegates(this, InUserObject);
}


int32 UMVVMViewModelBase::RemoveAllFieldValueChangedDelegates(UE::FieldNotification::FFieldId InFieldId, FDelegateUserObjectConst InUserObject)
{
	return NotificationDelegates.RemoveAllFieldValueChangedDelegates(this, InFieldId, InUserObject);
}


TArray<UE::FieldNotification::FFieldMulticastDelegate::FDelegateView> UMVVMViewModelBase::GetNotificationDelegateView() const
{
	return NotificationDelegates.GetView();
}


const UE::FieldNotification::IClassDescriptor& UMVVMViewModelBase::GetFieldNotificationDescriptor() const
{
	static FFieldNotificationClassDescriptor Local;
	return Local;
}


void UMVVMViewModelBase::FFieldNotificationClassDescriptor::ForEachField(const UClass* Class, TFunctionRef<bool(::UE::FieldNotification::FFieldId FieldId)> Callback) const
{
	if (const UBlueprintGeneratedClass* ViewModelBPClass = Cast<const UBlueprintGeneratedClass>(Class))
	{
		ViewModelBPClass->ForEachFieldNotify(Callback, true);
	}
}


void UMVVMViewModelBase::BroadcastFieldValueChanged(UE::FieldNotification::FFieldId InFieldId)
{
	NotificationDelegates.BroadcastFieldValueChanged(this, InFieldId);
}


void UMVVMViewModelBase::K2_BroadcastFieldValueChanged(FFieldNotificationId ViewModelPropertyName)
{
	UE::FieldNotification::FFieldId FieldId = GetFieldNotificationDescriptor().GetField(GetClass(), ViewModelPropertyName.GetFieldName());
	NotificationDelegates.BroadcastFieldValueChanged(this, FieldId);
}


DEFINE_FUNCTION(UMVVMViewModelBase::execK2_SetPropertyValue)
{
	Stack.StepCompiledIn<FProperty>(nullptr);
	FProperty* TargetProperty = Stack.MostRecentProperty;
	void* TargetValuePtr = Stack.MostRecentPropertyAddress;

	Stack.StepCompiledIn<FProperty>(nullptr);
	FProperty* SourceProperty = Stack.MostRecentProperty;
	void* SourceValuePtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	UMVVMViewModelBase* ViewModelContext = Cast<UMVVMViewModelBase>(Context);

	bool bResult = false;
	if (ViewModelContext == nullptr || TargetProperty == nullptr || SourceProperty == nullptr)
	{
		const FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("MissingInputProperty", "Failed to resolve the input parameter for SetPropertyValue.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN

		UE::FieldNotification::FFieldId FieldId = ViewModelContext->GetFieldNotificationDescriptor().GetField(ViewModelContext->GetClass(), TargetProperty->GetFName());
		if (!FieldId.IsValid())
		{
			const FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::AccessViolation,
				LOCTEXT("Bad FieldId", "Failed to find the FieldId that correspond to the set value.")
			);
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}
		else
		{
			bResult = TargetProperty->Identical(TargetValuePtr, SourceValuePtr);
			if (!bResult)
			{
				// Set the value then notify that the value changed.
				TargetProperty->SetValue_InContainer(ViewModelContext, SourceValuePtr);
				ViewModelContext->BroadcastFieldValueChanged(FieldId);
			}
		}

		P_NATIVE_END
	}

	*StaticCast<bool*>(RESULT_PARAM) = (!bResult);
}

#undef LOCTEXT_NAMESPACE

