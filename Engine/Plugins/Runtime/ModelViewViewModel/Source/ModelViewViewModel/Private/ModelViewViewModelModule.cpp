// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelViewViewModelModule.h"

#include "Bindings/ImplicitConverters/MVVMNumericImplicitConverter.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"
#include "Types/MVVMExecutionMode.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#endif

#define LOCTEXT_NAMESPACE "ModelViewViewModelModule"

void FModelViewViewModelModule::StartupModule()
{
	IConsoleVariable* CVarDefaultExecutionMode = IConsoleManager::Get().FindConsoleVariable(TEXT("MVVM.DefaultExecutionMode"));
	if (ensure(CVarDefaultExecutionMode))
	{
		HandleDefaultExecutionModeChanged(CVarDefaultExecutionMode);
		CVarDefaultExecutionMode->OnChangedDelegate().AddRaw(this, &FModelViewViewModelModule::HandleDefaultExecutionModeChanged);
	}
	
	NumericConverter = MakeShared<FMVVMNumericImplicitConverter>();
	RegisterImplicitConverter(NumericConverter.ToSharedRef());

#if WITH_EDITOR
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FModelViewViewModelModule::OnPostEngineInit);
#endif
}

void FModelViewViewModelModule::OnPostEngineInit()
{
#if WITH_EDITOR
	ensure(!OnBlueprintCompiledHandle.IsValid());
	if (GEditor != nullptr)
	{
		OnBlueprintCompiledHandle = GEditor->OnBlueprintCompiled().AddRaw(this, &FModelViewViewModelModule::HandleBlueprintCompiled);
		OnBlueprintPreCompileHandle = GEditor->OnBlueprintPreCompile().AddRaw(this, &FModelViewViewModelModule::HandleBlueprintPreCompile);
	}
#endif
}

void FModelViewViewModelModule::HandleBlueprintCompiled()
{
#if WITH_EDITOR
	OnBlueprintCompiled.Broadcast();
#endif
}

void FModelViewViewModelModule::HandleBlueprintPreCompile(UBlueprint* Blueprint)
{
#if WITH_EDITOR
	OnBlueprintPreCompile.Broadcast(Blueprint);
#endif
}


void FModelViewViewModelModule::ShutdownModule()
{
	if (!IsEngineExitRequested())
	{
		if (IConsoleVariable* CVarDefaultExecutionMode = IConsoleManager::Get().FindConsoleVariable(TEXT("MVVM.DefaultExecutionMode")))
		{
			CVarDefaultExecutionMode->OnChangedDelegate().RemoveAll(this);
		}
	}

	if (NumericConverter.IsValid())
	{
		UnregisterImplicitConverter(NumericConverter.ToSharedRef());
	}

#if WITH_EDITOR
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	if (GEditor != nullptr)
	{
		if (OnBlueprintCompiledHandle.IsValid())
		{
			GEditor->OnBlueprintCompiled().Remove(OnBlueprintCompiledHandle);
		}
		if (OnBlueprintPreCompileHandle.IsValid())
		{
			GEditor->OnBlueprintPreCompile().Remove(OnBlueprintPreCompileHandle);
		}
	}
#endif
}

void FModelViewViewModelModule::RegisterImplicitConverter(const TSharedRef<FMVVMImplicitConverter>& Converter)
{
	ImplicitConverters.AddUnique(Converter);
}

void FModelViewViewModelModule::UnregisterImplicitConverter(const TSharedRef<FMVVMImplicitConverter>& Converter)
{
	ImplicitConverters.Remove(Converter);
}

void FModelViewViewModelModule::HandleDefaultExecutionModeChanged(IConsoleVariable* Variable)
{
	const int32 Value = Variable->GetInt();
	switch(Value)
	{
	case (int32)EMVVMExecutionMode::Delayed:
	case (int32)EMVVMExecutionMode::Immediate:
	case (int32)EMVVMExecutionMode::Tick:
	case (int32)EMVVMExecutionMode::DelayedWhenSharedElseImmediate:
		break;
	default:
		ensureMsgf(false, TEXT("MVVM.DefaultExecutionMode default value is not a valid value."));
		Variable->Set((int32)EMVVMExecutionMode::DelayedWhenSharedElseImmediate, (EConsoleVariableFlags)(Variable->GetFlags() & ECVF_SetByMask));
		break;
	}
}

IMPLEMENT_MODULE(FModelViewViewModelModule, ModelViewViewModel);

#undef LOCTEXT_NAMESPACE
