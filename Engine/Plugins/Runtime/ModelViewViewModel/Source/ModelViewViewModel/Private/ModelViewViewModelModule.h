// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IModelViewViewModelModule.h"
#include "Stats/Stats.h"

class FMVVMNumericImplicitConverter;
class IConsoleVariable;
class UBlueprint;

/** */
DECLARE_STATS_GROUP(TEXT("UMG Viewmodel"), STATGROUP_UMG_Viewmodel, STATCAT_Advanced);

/**
 *
 */
class FModelViewViewModelModule : public IModelViewViewModelModule
{
public:
	FModelViewViewModelModule() = default;

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

#if WITH_EDITOR
	DECLARE_TS_MULTICAST_DELEGATE(FOnBlueprintCompiled)
	FOnBlueprintCompiled OnBlueprintCompiled;

	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnBlueprintPreCompile, UBlueprint*)
	FOnBlueprintPreCompile OnBlueprintPreCompile;
#endif

	virtual void RegisterImplicitConverter(const TSharedRef<FMVVMImplicitConverter>& Converter) override;
	virtual void UnregisterImplicitConverter(const TSharedRef<FMVVMImplicitConverter>& Converter) override;
	TConstArrayView<TSharedPtr<FMVVMImplicitConverter>> GetImplicitConverters() const { return ImplicitConverters; }

private:
	void HandleDefaultExecutionModeChanged(IConsoleVariable* Variable);
	void OnPostEngineInit();

	void HandleBlueprintCompiled();
	void HandleBlueprintPreCompile(UBlueprint* Blueprint);
#if WITH_EDITOR
	FDelegateHandle OnBlueprintCompiledHandle;
	FDelegateHandle OnBlueprintPreCompileHandle;
#endif

	TArray<TSharedPtr<FMVVMImplicitConverter>> ImplicitConverters;
	TSharedPtr<FMVVMNumericImplicitConverter> NumericConverter;
};
