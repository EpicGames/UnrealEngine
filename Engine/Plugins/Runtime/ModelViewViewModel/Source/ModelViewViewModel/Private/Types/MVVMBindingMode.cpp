// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/MVVMBindingMode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBindingMode)


namespace UE::MVVM
{

	bool IsForwardBinding(EMVVMBindingMode Mode)
	{
		//OneTimeToDestination || OneWayToDestination || TwoWay
		return static_cast<uint8>(Mode) <= static_cast<uint8>(EMVVMBindingMode::TwoWay);
	}

	bool IsBackwardBinding(EMVVMBindingMode Mode)
	{
		//OneTimeToSource || OneWayToSource || TwoWay
		return static_cast<uint8>(Mode) >= static_cast<uint8>(EMVVMBindingMode::TwoWay);
	}

	bool IsOneTimeBinding(EMVVMBindingMode Mode)
	{
		return Mode == EMVVMBindingMode::OneTimeToDestination || Mode == EMVVMBindingMode::OneTimeToSource;
	}

} //namespace

