// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/MVVMConditionOperation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMConditionOperation)

#ifndef CASE_ENUM_TO_TEXT
#define CASE_ENUM_TO_TEXT(txt) case txt: return TEXT(#txt);
#endif

const TCHAR* LexToString(EMVVMConditionOperation Enum)
{
	switch (Enum)
	{
		FOREACH_ENUM_EMVVMCONDITIONOPERATION(CASE_ENUM_TO_TEXT)
	}
	return TEXT("<Unknown EMVVMConditionOperation>");
}

