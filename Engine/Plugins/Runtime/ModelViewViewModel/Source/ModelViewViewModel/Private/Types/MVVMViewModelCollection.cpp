// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/MVVMViewModelCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMViewModelCollection)

UMVVMViewModelBase* FMVVMViewModelCollection::FindViewModelInstance(FMVVMViewModelContext Context) const
{
	FMVVMViewModelContextInstance* FoundInstance = ViewModelInstances.FindByPredicate([Context](const FMVVMViewModelContextInstance& Other) { return Other.GetContext() == Context; });
	return FoundInstance ? FoundInstance->GetViewModel() : nullptr;
}

UMVVMViewModelBase* FMVVMViewModelCollection::FindFirstViewModelInstanceOfType(const TSubclassOf<UMVVMViewModelBase>& ViewModelClass) const
{
	FMVVMViewModelContextInstance* FoundInstance = ViewModelInstances.FindByPredicate([ViewModelClass](const FMVVMViewModelContextInstance& Other) { return Other.GetContext().IsCompatibleWith(ViewModelClass); });
	return FoundInstance ? FoundInstance->GetViewModel() : nullptr;
}

bool FMVVMViewModelCollection::AddInstance(FMVVMViewModelContext Context, UMVVMViewModelBase* ViewModel)
{
	FMVVMViewModelContextInstance* FoundInstance = ViewModelInstances.FindByPredicate([Context](const FMVVMViewModelContextInstance& Other) { return Other.GetContext() == Context; });
	if (FoundInstance)
	{
		return false;
	}

	FMVVMViewModelContextInstance Instance { Context, ViewModel };
	if (Instance.IsValid())
	{
		ViewModelInstances.Add(Instance);
		OnCollectionChangedDelegate.Broadcast();
		return true;
	}
	return false;
}


bool FMVVMViewModelCollection::RemoveInstance(FMVVMViewModelContext Context)
{
	int32 FoundInstanceIndex = ViewModelInstances.IndexOfByPredicate([Context](const FMVVMViewModelContextInstance& Other) { return Other.GetContext() == Context; });
	if (FoundInstanceIndex == INDEX_NONE)
	{
		return false;
	}

	ViewModelInstances.RemoveAtSwap(FoundInstanceIndex);
	OnCollectionChangedDelegate.Broadcast();
	return true;
}


int32 FMVVMViewModelCollection::RemoveAllInstances(UMVVMViewModelBase* ViewModel)
{
	const int32 NumRemoved = ViewModelInstances.RemoveAll([ViewModel](const FMVVMViewModelContextInstance& Other) { return Other.GetViewModel() == ViewModel; });
	if (NumRemoved != 0)
	{
		OnCollectionChangedDelegate.Broadcast();
	}
	return NumRemoved;
}


void FMVVMViewModelCollection::Reset()
{
	ViewModelInstances.Reset();
	OnCollectionChangedDelegate.Broadcast();
}

