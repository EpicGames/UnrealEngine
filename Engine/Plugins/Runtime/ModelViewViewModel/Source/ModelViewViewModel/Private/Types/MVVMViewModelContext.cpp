// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/MVVMViewModelContext.h"
#include "MVVMViewModelBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMViewModelContext)

bool FMVVMViewModelContext::IsValid() const
{
	return !ContextName.IsNone() && ContextClass.Get() != nullptr;
}


bool FMVVMViewModelContext::operator== (const FMVVMViewModelContext& Other) const
{
	return Other.ContextName == ContextName
		&& ContextClass == Other.ContextClass;
}


bool FMVVMViewModelContext::IsCompatibleWith(const FMVVMViewModelContext& Other) const
{
	return Other.ContextName == ContextName && IsCompatibleWith(Other.ContextClass);
}

bool FMVVMViewModelContext::IsCompatibleWith(const TSubclassOf<UMVVMViewModelBase>& OtherClass) const
{
	return OtherClass != nullptr
		&& ContextClass.Get() != nullptr
		&& ContextClass.Get()->IsChildOf(OtherClass);
}


bool FMVVMViewModelContext::IsCompatibleWith(const UMVVMViewModelBase* Other) const
{
	return  ContextClass.Get() && Other && Other->GetClass()->IsChildOf(ContextClass.Get());
}

