// Copyright Epic Games, Inc. All Rights Reserved.

#include "View/MVVMBindingSubsystem.h"

#include "Framework/Application/SlateApplication.h"
#include "Misc/MemStack.h"
#include "SlateGlobals.h"
#include "Stats/Stats.h"
#include "View/MVVMView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBindingSubsystem)


DECLARE_CYCLE_STAT(TEXT("MVVM Bindings"), STAT_MVVMBindingTick, STATGROUP_Slate);

namespace UE::MVVM::Private
{
struct FViewAndBinding
{
	FViewAndBinding(const TObjectKey<const UMVVMView>& InView, FMVVMViewClass_BindingKey InBinding)
		: View(InView)
		, Binding(InBinding)
	{
	}
	TObjectKey<const UMVVMView> View;
	FMVVMViewClass_BindingKey Binding;

	bool operator== (const FViewAndBinding& Other) const
	{
		return View == Other.View && Binding == Other.Binding;
	}

	friend uint32 GetTypeHash(const FViewAndBinding& Key)
	{
		uint32 Value1 = GetTypeHash(Key.View);
		uint32 Value2 = GetTypeHash(Key.Binding.GetIndex());
		return HashCombine(Value1, Value2);
	}
};
}

void UMVVMBindingSubsystem::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_MVVMBindingTick);

	if (ViewsWithTickBindings.Num() > 0)
	{
		FMemMark Mark(FMemStack::Get());
		TArray<const UMVVMView*, TMemStackAllocator<>> ToTick;
		ToTick.Reserve(ViewsWithTickBindings.Num());
		for (TWeakObjectPtr<const UMVVMView> View : ViewsWithTickBindings)
		{
			const UMVVMView* ViewPtr = View.Get();
			if (ensure(ViewPtr))
			{
				ToTick.Add(ViewPtr);
			}
		}

		for (const UMVVMView* ViewPtr : ToTick)
		{
			ViewPtr->ExecuteTickBindings();
		}
	}

	if (DelayedBindings.Num() > 0)
	{
		TSet<UE::MVVM::Private::FViewAndBinding> AllDelayedBindingsExecutedThisFrame;
		AllDelayedBindingsExecutedThisFrame.Reserve(DelayedBindings.Num());

		FDelayedBindingMap DelayedBindingsWhileTicking = MoveTemp(DelayedBindings);
		DelayedBindings = FDelayedBindingMap();

		do 
		{
			for (const auto& DelayedBindingsPair : DelayedBindingsWhileTicking)
			{
				if (const UMVVMView* View = DelayedBindingsPair.Key.ResolveObjectPtr())
				{
					ensure(DelayedBindingsPair.Value.Num() > 0);
					for (const FMVVMViewClass_BindingKey& DelayedBinding : DelayedBindingsPair.Value)
					{
						View->ExecuteDelayedBinding(DelayedBinding);
						UE::MVVM::Private::FViewAndBinding ViewAndBinding = UE::MVVM::Private::FViewAndBinding(DelayedBindingsPair.Key, DelayedBinding);
						AllDelayedBindingsExecutedThisFrame.Add(ViewAndBinding);
					}
				}
			}

			DelayedBindingsWhileTicking.Reset();

			// Test new bindings added while executing the latest binding list.
			//If it's a new  binding (not already executed this frame), execute it this frame. Else, execute it next frame.
			for (auto DelayedBindingItt = DelayedBindings.CreateIterator(); DelayedBindingItt; ++DelayedBindingItt)
			{
				FDelayedBindingList* FoundDelayedBindingListPtr = nullptr;
				for (int32 DelayIndex = DelayedBindingItt.Value().Num() - 1; DelayIndex >= 0; --DelayIndex)
				{
					// Was it executed this frame
					const FMVVMViewClass_BindingKey& DelayedBinding = DelayedBindingItt.Value()[DelayIndex];
					UE::MVVM::Private::FViewAndBinding ViewAndBinding = UE::MVVM::Private::FViewAndBinding(DelayedBindingItt.Key(), DelayedBinding);
					if (!AllDelayedBindingsExecutedThisFrame.Find(ViewAndBinding))
					{
						if (!FoundDelayedBindingListPtr)
						{
							FoundDelayedBindingListPtr = &DelayedBindingsWhileTicking.FindOrAdd(DelayedBindingItt.Key());
						}
						FoundDelayedBindingListPtr->AddUnique(DelayedBinding);
						DelayedBindingItt.Value().RemoveAtSwap(DelayIndex);
					}

					// If they were all executed, remove the key from the next frame list.
					if (DelayedBindingItt.Value().Num() == 0)
					{
						DelayedBindingItt.RemoveCurrent();
					}
				}
			}

			//DelayedBindings = FDelayedMap(); // do not reset the array. Bindings could be executed this frame and needs to be re executed next frame

		} while (DelayedBindingsWhileTicking.Num() > 0);
	}
}

TStatId UMVVMBindingSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UMVVMBindingSubsystem, STATGROUP_Tickables);
}

void UMVVMBindingSubsystem::AddViewWithTickBinding(const UMVVMView* InView)
{
	check(!ViewsWithTickBindings.Contains(InView));
	ensureMsgf(FSlateApplication::IsInitialized(), TEXT("The Slate Application is not initialized. This is probably because you are running a server. The Delayed and Tick binding will not execute."));

	ViewsWithTickBindings.Add(InView);
}

void UMVVMBindingSubsystem::RemoveViewWithTickBinding(const UMVVMView* InView)
{
	ViewsWithTickBindings.RemoveSingleSwap(InView);
}

void UMVVMBindingSubsystem::AddDelayedBinding(const UMVVMView* View, FMVVMViewClass_BindingKey InCompiledBinding)
{
	ensureMsgf(FSlateApplication::IsInitialized(), TEXT("The Slate Application is not initialized. This is probably because you are running a server. The Delayed and Tick binding will not execute."));
	DelayedBindings.FindOrAdd(View).AddUnique(InCompiledBinding);
}

void UMVVMBindingSubsystem::RemoveDelayedBindings(const UMVVMView* View)
{
	ensureMsgf(FSlateApplication::IsInitialized(), TEXT("The Slate Application is not initialized. This is probably because you are running a server. The Delayed and Tick binding will not execute."));
	DelayedBindings.Remove(View);
}

void UMVVMBindingSubsystem::RemoveDelayedBindings(const UMVVMView* View, FMVVMViewClass_SourceKey SourceKey)
{
	ensureMsgf(FSlateApplication::IsInitialized(), TEXT("The Slate Application is not initialized. This is probably because you are running a server. The Delayed and Tick binding will not execute."));
	if (FDelayedBindingList* FoundView = DelayedBindings.Find(View))
	{
		const UMVVMViewClass* ViewClass = View->GetViewClass();
		for (int32 Index = FoundView->Num() - 1; Index >= 0; --Index)
		{
			const FMVVMViewClass_Binding& Binding = ViewClass->GetBinding((*FoundView)[Index]);
			if ((Binding.GetSources() & SourceKey.GetBit()) != 0)
			{
				(*FoundView).RemoveAtSwap(Index);
			}
		}

		if (FoundView->Num() == 0)
		{
			DelayedBindings.Remove(View);
		}
	}
}
