// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "Tickable.h"
#include "UObject/ObjectKey.h"
#include "View/MVVMViewClass.h"

#include "MVVMBindingSubsystem.generated.h"

class UMVVMView;

/** */
UCLASS(NotBlueprintable, Hidden)
class UMVVMBindingSubsystem : public UEngineSubsystem, public FTickableGameObject
{
	GENERATED_BODY()

public:
	void AddViewWithTickBinding(const UMVVMView* View);
	void RemoveViewWithTickBinding(const UMVVMView* View);

	void AddDelayedBinding(const UMVVMView* View, FMVVMViewClass_BindingKey BindingKey);
	void RemoveDelayedBindings(const UMVVMView* View);
	void RemoveDelayedBindings(const UMVVMView* View, FMVVMViewClass_SourceKey SourceKey);

	//~ FTickableGameObject
	virtual bool IsTickableWhenPaused() const override
	{
		return true;
	}
	virtual bool IsTickableInEditor() const override
	{
		return true;
	}
	virtual ETickableTickType GetTickableTickType() const override
	{
		return (IsTemplate() ? ETickableTickType::Never : ETickableTickType::Always);
	}

	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;

private:
	using FDelayedBindingList = TArray<FMVVMViewClass_BindingKey, TInlineAllocator<8>>;
	using FDelayedBindingMap = TMap<TObjectKey<const UMVVMView>, FDelayedBindingList>;
	FDelayedBindingMap DelayedBindings;
	TArray<TWeakObjectPtr<const UMVVMView>> ViewsWithTickBindings;
};
