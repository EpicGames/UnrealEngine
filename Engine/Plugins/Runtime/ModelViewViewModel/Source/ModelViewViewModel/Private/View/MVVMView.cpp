// Copyright Epic Games, Inc. All Rights Reserved.

#include "View/MVVMView.h"
#include "FieldNotificationDelegate.h"
#include "View/MVVMViewClass.h"
#include "Misc/MemStack.h"
#include "View/MVVMBindingSubsystem.h"

#include "Blueprint/UserWidget.h"
#include "Debugging/MVVMDebugging.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Extensions/MVVMViewClassExtension.h"
#include "MVVMGameSubsystem.h"
#include "MVVMMessageLog.h"
#include "ModelViewViewModelModule.h"
#include "Modules/ModuleManager.h"
#include "Templates/ValueOrError.h"
#include "Types/MVVMFieldContext.h"
#include "Types/MVVMViewModelCollection.h"
#include "Bindings/MVVMBindingHelper.h"
#if WITH_EDITOR
#include "Engine/Blueprint.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMView)

DECLARE_CYCLE_STAT(TEXT("InitializeSources"), STAT_UMG_Viewmodel_InitializeSources, STATGROUP_UMG_Viewmodel);
DECLARE_CYCLE_STAT(TEXT("UninitializeSources"), STAT_UMG_Viewmodel_UninitializeSources, STATGROUP_UMG_Viewmodel);
DECLARE_CYCLE_STAT(TEXT("InitializeBindings"), STAT_UMG_Viewmodel_InitializeBindings, STATGROUP_UMG_Viewmodel);
DECLARE_CYCLE_STAT(TEXT("UninitializeBindings"), STAT_UMG_Viewmodel_UninitializeBindings, STATGROUP_UMG_Viewmodel);
DECLARE_CYCLE_STAT(TEXT("SetSource"), STAT_UMG_Viewmodel_SetSource, STATGROUP_UMG_Viewmodel);
DECLARE_CYCLE_STAT(TEXT("ExecuteViewModel"), STAT_UMG_Viewmodel_ExecuteViewModel, STATGROUP_UMG_Viewmodel);
DECLARE_CYCLE_STAT(TEXT("ExecuteBinding ValueChanged"), STAT_UMG_Viewmodel_ExecuteBinding_ValueChanged, STATGROUP_UMG_Viewmodel);
DECLARE_CYCLE_STAT(TEXT("ExecuteBinding Delayed"), STAT_UMG_Viewmodel_ExecuteBinding_Delayed, STATGROUP_UMG_Viewmodel);
DECLARE_CYCLE_STAT(TEXT("ExecuteBinding Tick"), STAT_UMG_Viewmodel_ExecuteBinding_Tick, STATGROUP_UMG_Viewmodel);

#define LOCTEXT_NAMESPACE "MVVMView"

///////////////////////////////////////////////////////////////////////
// 
///////////////////////////////////////////////////////////////////////

namespace UE::MVVM::Private
{
UMVVMViewModelCollectionObject* GetGlobalCollection(UUserWidget* UserWidget)
{
	if (const UWorld* World = UserWidget->GetWorld())
	{
		if (const UGameInstance* GameInstance = World->GetGameInstance())
		{
			return GameInstance->GetSubsystem<UMVVMGameSubsystem>()->GetViewModelCollection();
		}
	}
	return nullptr;
}

using FRecursiveDetctionElement = TTuple<const UObject*, FMVVMViewClass_BindingKey>;
TArray<FRecursiveDetctionElement> RecursiveDetector;
} // namespace

void UMVVMView::ConstructView(const UMVVMViewClass* InGeneratedViewClass)
{
	ensure(GeneratedViewClass == nullptr);
	GeneratedViewClass = InGeneratedViewClass;

	check(Sources.Num() == 0);
	int32 SourceNum = GeneratedViewClass->GetSources().Num();
	Sources.SetNum(SourceNum);
	for (int32 Index = 0; Index < SourceNum; ++Index)
	{
		FMVVMView_Source& Source = Sources[Index];
		Source.ClassKey = FMVVMViewClass_SourceKey(Index);
	}
}


void UMVVMView::Construct()
{
	check(GeneratedViewClass);
	check(bConstructed == false);

	// Create an independent copy of extensions per view.
	check(Extensions.IsEmpty());
	Extensions.Reset(GeneratedViewClass->GetViewClassExtensions().Num());
	for (UMVVMViewClassExtension* Extension : GeneratedViewClass->GetViewClassExtensions())
	{
		Extensions.Add(Extension->ViewConstructed(GetUserWidget(), this));
	}

	if (GeneratedViewClass->DoesInitializeSourcesOnConstruct())
	{
		InitializeSources();
	}

	if (GeneratedViewClass->DoesInitializeEventsOnConstruct())
	{
		InitializeEvents();
	}

	bConstructed = true;

#if WITH_EDITOR
	if (GEditor)
	{
		if (GetWorld() && GetWorld()->WorldType == EWorldType::Editor)
		{
			FModelViewViewModelModule& Module = FModuleManager::GetModuleChecked<FModelViewViewModelModule>("ModelViewViewModel");
			BlueprintPreCompileHandle = Module.OnBlueprintPreCompile.AddUObject(this, &UMVVMView::HandleBlueprintPreCompile);
		}
	}
#endif

#if UE_WITH_MVVM_DEBUGGING
	UE::MVVM::FDebugging::BroadcastViewConstructed(this);
#endif
}


void UMVVMView::Destruct()
{
#if WITH_EDITOR
	if (GEditor && BlueprintPreCompileHandle.IsValid())
	{
		FModelViewViewModelModule& Module = FModuleManager::GetModuleChecked<FModelViewViewModelModule>("ModelViewViewModel");
		Module.OnBlueprintPreCompile.Remove(BlueprintPreCompileHandle);
		BlueprintPreCompileHandle.Reset();
	}

	if (!GeneratedViewClass && (!GetWorld() || GetWorld()->WorldType == EWorldType::Editor))
	{
		return;
	}
#endif

	check(bConstructed == true);
	bConstructed = false;

#if UE_WITH_MVVM_DEBUGGING
	UE::MVVM::FDebugging::BroadcastViewBeginDestruction(this);
#endif

	UninitializeEvents();
	UninitializeSources(); // and bindings

	const TArrayView<const TObjectPtr<UMVVMViewClassExtension>> ClassExtensions = GeneratedViewClass->GetViewClassExtensions();
	check(ClassExtensions.Num() == Extensions.Num());
	for (int32 Index = ClassExtensions.Num() - 1; Index >= 0; --Index)
	{
		ClassExtensions[Index]->OnViewDestructed(GetUserWidget(), this, Extensions[Index]);
		Extensions.RemoveAtSwap(Index);
	}
}


void UMVVMView::InitializeSources()
{
	if (bSourcesInitialized || GeneratedViewClass == nullptr)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_UMG_Viewmodel_InitializeSources);

	ensure(bHasDefaultTickBinding == false && NumberOfSourceWithTickBinding == 0);

	// Init Sources/ViewModel instances
	const int32 NumberOfSources = GeneratedViewClass->GetSources().Num();
	check(Sources.Num() == NumberOfSources);
	check(NumberOfSources <= 64); // the max number of source the bitfield can hold.

	if (GeneratedViewClass->DoesListenToViewModelCollectionChanged())
	{
		if (UMVVMViewModelCollectionObject* Collection = UE::MVVM::Private::GetGlobalCollection(GetUserWidget()))
		{
			Collection->OnCollectionChanged().AddUObject(this, &UMVVMView::HandleViewModelCollectionChanged);
		}
	}

	for (int32 Index = 0; Index < NumberOfSources; ++Index)
	{
		InitializeSource(FMVVMView_SourceKey(Index));

		// If DoesInitializeSourcesOnConstruct false but DoesInitializeBindingsOnConstruct, then we need to initialized the bindings when the source are initialized
		if (GeneratedViewClass->DoesInitializeBindingsOnConstruct())
		{
			//note: Run the bindings. The binding from Source A can influence how the Source B is initialized/evaluated.
			InitializeSourceBindings(FMVVMView_SourceKey(Index), false);
		}

		InitializeSourceParentBinding(FMVVMView_SourceKey(Index));
	}

	if (GeneratedViewClass->DoesInitializeBindingsOnConstruct())
	{
		InitializeSourceBindingsCommon();
	}

	const TArrayView<const TObjectPtr<UMVVMViewClassExtension>> ClassExtensions = GeneratedViewClass->GetViewClassExtensions();
	check(ClassExtensions.Num() == Extensions.Num());
	for (int32 Index = 0; Index < ClassExtensions.Num(); ++Index)
	{
		ClassExtensions[Index]->OnSourcesInitialized(GetUserWidget(), this, Extensions[Index]);
	}

	if (GeneratedViewClass->DoesInitializeBindingsOnConstruct())
	{
		for (int32 Index = 0; Index < ClassExtensions.Num(); ++Index)
		{
			ClassExtensions[Index]->OnBindingsInitialized(GetUserWidget(), this, Extensions[Index]);
		}
	}

	bSourcesInitialized = true;
}


void UMVVMView::UninitializeSources()
{
	if (!bSourcesInitialized)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_UMG_Viewmodel_UninitializeSources);

	if (bBindingsInitialized)
	{
		UninitializeBindings();
	}

	const TArrayView<const TObjectPtr<UMVVMViewClassExtension>> ClassExtensions = GeneratedViewClass->GetViewClassExtensions();
	check(ClassExtensions.Num() == Extensions.Num());
	for (int32 Index = 0; Index < ClassExtensions.Num(); ++Index)
	{
		ClassExtensions[Index]->OnSourcesUninitialized(GetUserWidget(), this, Extensions[Index]);
	}

	if (GeneratedViewClass->DoesListenToViewModelCollectionChanged())
	{
		if (UMVVMViewModelCollectionObject* Collection = UE::MVVM::Private::GetGlobalCollection(GetUserWidget()))
		{
			Collection->OnCollectionChanged().RemoveAll(this);
		}
	}

	bSourcesInitialized = false;

	const int32 NumberOfSources = Sources.Num();
	for (int32 Index = 0; Index < NumberOfSources; ++Index)
	{
		UninitializeSource(FMVVMView_SourceKey(Index));
		UninitializeSourceParentBinding(FMVVMView_SourceKey(Index));
	}
}


void UMVVMView::InitializeSource(FMVVMView_SourceKey SourceKey)
{
	FMVVMView_Source& ViewSource = Sources[SourceKey.GetIndex()];
	const FMVVMViewClass_Source& ClassSource = GeneratedViewClass->GetSource(ViewSource.ClassKey);

	check(ViewSource.bSourceInitialized == false);

	// Is it already set by something external before we had the change to set it.
	if (!ViewSource.bSetManually)
	{
		UUserWidget* UserWidget = GetUserWidget();
		UObject* NewSource = ClassSource.GetOrCreateInstance(GeneratedViewClass, this, UserWidget);
		InitializeSourceInternal(NewSource, ViewSource.ClassKey, ClassSource, ViewSource);

#if UE_WITH_MVVM_DEBUGGING
		UE::MVVM::FDebugging::BroadcastViewSourceValueChanged(this, ViewSource.ClassKey, SourceKey);
#endif
	}

	ViewSource.bSourceInitialized = true;
}


void UMVVMView::InitializeSourceInternal(UObject* NewSource, FMVVMViewClass_SourceKey SourceKey, const FMVVMViewClass_Source& ClassSource, FMVVMView_Source& ViewSource)
{
	if (NewSource)
	{
		ValidSources |= SourceKey.GetBit();
	}
	else
	{
		ValidSources &= ~SourceKey.GetBit();
	}

	if (ClassSource.RequireSettingUserWidgetProperty())
	{
		UUserWidget* UserWidget = GetUserWidget();
		FObjectPropertyBase* FoundObjectProperty = FindFProperty<FObjectPropertyBase>(UserWidget->GetClass(), ClassSource.GetUserWidgetPropertyName());
		if (ensureAlwaysMsgf(FoundObjectProperty, TEXT("The compiler should have added the property")))
		{
			if (NewSource == nullptr || ensure(NewSource->GetClass()->IsChildOf(FoundObjectProperty->PropertyClass)))
			{
				FoundObjectProperty->SetObjectPropertyValue_InContainer(UserWidget, NewSource);
				ViewSource.bAssignedToUserWidgetProperty = true;
			}
		}
	}

	ViewSource.Source = NewSource;

	// See comment in UMVVMView::InitializeSourceParentBinding for why we do this
	GetUserWidget()->BroadcastFieldValueChanged(UE::FieldNotification::FFieldId(ClassSource.GetName(), 0));
}


void UMVVMView::UninitializeSource(FMVVMView_SourceKey SourceKey)
{
	FMVVMView_Source& ViewSource = Sources[SourceKey.GetIndex()];
	const FMVVMViewClass_Source& ClassSource = GeneratedViewClass->GetSource(ViewSource.ClassKey);

	if (ViewSource.bSourceInitialized)
	{
		ViewSource.bSourceInitialized = false;
		if (!ViewSource.bSetManually)
		{
			ClassSource.ReleaseInstance(ViewSource.Source, this);
			ViewSource.bSourceInitialized = false;

			ViewSource.Source = nullptr;
			if (ViewSource.bAssignedToUserWidgetProperty)
			{
				UUserWidget* UserWidget = GetUserWidget();
				FObjectPropertyBase* FoundObjectProperty = FindFProperty<FObjectPropertyBase>(UserWidget->GetClass(), ClassSource.GetUserWidgetPropertyName());
				if (ensureAlwaysMsgf(FoundObjectProperty, TEXT("The compiler should have added the property")))
				{
					FoundObjectProperty->SetObjectPropertyValue_InContainer(UserWidget, nullptr);
				}
				ViewSource.bAssignedToUserWidgetProperty = false;
			}

			ValidSources &= ~ViewSource.ClassKey.GetBit();

#if UE_WITH_MVVM_DEBUGGING
			UE::MVVM::FDebugging::BroadcastViewSourceValueChanged(this, ViewSource.ClassKey, SourceKey);
#endif
		}
	}
}

void UMVVMView::InitializeSourceParentBinding(FMVVMView_SourceKey SourceKey)
{
	FMVVMView_Source& ViewSource = Sources[SourceKey.GetIndex()];
	const FMVVMViewClass_Source& ClassSource = GeneratedViewClass->GetSource(ViewSource.ClassKey);

	if (ClassSource.IsParentContextReference())
	{
		FMVVMViewClass_Source::FParentSourceInfo ParentInfo = ClassSource.GetParentSupplyingSourceClass(GetUserWidget());
		if (ParentInfo.ParentView && ParentInfo.ParentUserWidget)
		{
			const FMVVMView_SourceKey ParentSourceKey = ParentInfo.ParentSourceKey;
			const FMVVMViewClass_Source& ParentClassSource = ParentInfo.ParentView->GetViewClass()->GetSource(ParentInfo.ParentSourceKey.GetIndex());

			TWeakObjectPtr<const UMVVMView> WeakView = ParentInfo.ParentView;
			UE::FieldNotification::FFieldMulticastDelegate::FDelegate Delegate = UE::FieldNotification::FFieldMulticastDelegate::FDelegate::CreateWeakLambda(this, [this, SourceKey, ParentSourceKey, WeakView](UObject*, UE::FieldNotification::FFieldId)
				{
					if (const UMVVMView* ParentView = WeakView.Get())
					{
						SetSourceInternal(FMVVMViewClass_SourceKey(SourceKey.GetIndex()), ParentView->GetSource(ParentSourceKey).Source, true);
					}
				});

			ParentInfo.ParentUserWidget->GetFieldNotificationDescriptor();

			// This is a big of a hack. The VM variable is not marked as field notify and we don't want it to be, but we also don't want to add bespoke
			// notification logic. We always know that there will be at least one field notify in a user widget (UWidget has a few) and field notifies
			// use the index to enable callbacks then the names to filter them. Adding this will enable all calbacks bound to index 0 (which may be multiple)
			// and get called for any callbacks that share the name, regardless of index. If there happens to be a duplicate name in the delegates, then
			// it likely just means that some callbacks are firing off more than they should, i.e. being notified of a change when there wasn't any,
			// so there should be very low risk to this.
			//
			// If any of this changes make sure to also update UMVVMView::InitializeSourceInternal and UMVVMView::UninitializeSourceParentBinding
			ParentInfo.ParentUserWidget->AddFieldValueChangedDelegate(UE::FieldNotification::FFieldId(ParentClassSource.GetName(), 0), MoveTemp(Delegate));
		}
	}
}

void UMVVMView::UninitializeSourceParentBinding(FMVVMView_SourceKey SourceKey)
{
	FMVVMView_Source& ViewSource = Sources[SourceKey.GetIndex()];
	const FMVVMViewClass_Source& ClassSource = GeneratedViewClass->GetSource(ViewSource.ClassKey);

	if (ClassSource.IsParentContextReference())
	{
		// See comment in UMVVMView::InitializeSourceParentBinding for why we do this
		FMVVMViewClass_Source::FParentSourceInfo ParentInfo = ClassSource.GetParentSupplyingSourceClass(GetUserWidget());
		if (ParentInfo.ParentView && ParentInfo.ParentUserWidget)
		{
			ParentInfo.ParentUserWidget->RemoveAllFieldValueChangedDelegates(this);
		}
	}
}

void UMVVMView::InitializeBindings()
{
	if (!bSourcesInitialized)
	{
		InitializeSources();
	}

	if (bBindingsInitialized)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_UMG_Viewmodel_InitializeBindings);

	check(NumberOfSourceWithTickBinding == 0);

	for (int32 SourceIndex = 0; SourceIndex < Sources.Num(); ++SourceIndex)
	{
		InitializeSourceBindings(FMVVMView_SourceKey(SourceIndex), false);
	}

	InitializeSourceBindingsCommon();

	const TArrayView<const TObjectPtr<UMVVMViewClassExtension>> ClassExtensions = GeneratedViewClass->GetViewClassExtensions();
	check(ClassExtensions.Num() == Extensions.Num());
	for (int32 Index = 0; Index < ClassExtensions.Num(); ++Index)
	{
		ClassExtensions[Index]->OnBindingsInitialized(GetUserWidget(), this, Extensions[Index]);
	}
}


void UMVVMView::InitializeSourceBindingsCommon()
{
	ensure(bHasDefaultTickBinding == false);

	if (ValidSources != 0) // If one source is property initialized, we assumed that at least one binding was registered.
	{
		static IConsoleVariable* CVarDefaultExecutionMode = IConsoleManager::Get().FindConsoleVariable(TEXT("MVVM.DefaultExecutionMode"));
		bHasDefaultTickBinding = ensure(CVarDefaultExecutionMode) ? (EMVVMExecutionMode)CVarDefaultExecutionMode->GetInt() == EMVVMExecutionMode::Tick : false;
	}

	if (NumberOfSourceWithTickBinding == 0 && bHasDefaultTickBinding)
	{
		GEngine->GetEngineSubsystem<UMVVMBindingSubsystem>()->AddViewWithTickBinding(this);
	}

	bBindingsInitialized = true;
}


void UMVVMView::UninitializeBindings()
{
	if (!bBindingsInitialized)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_UMG_Viewmodel_UninitializeBindings);

	bBindingsInitialized = false;

	const TArrayView<const TObjectPtr<UMVVMViewClassExtension>> ClassExtensions = GeneratedViewClass->GetViewClassExtensions();
	check(ClassExtensions.Num() == Extensions.Num());
	for (int32 Index = 0; Index < ClassExtensions.Num(); ++Index)
	{
		ClassExtensions[Index]->OnBindingsUninitialized(GetUserWidget(), this, Extensions[Index]);
	}

	const TArrayView<const FMVVMViewClass_Source> ClassSources = GeneratedViewClass->GetSources();
	for (int32 SourceIndex = 0; SourceIndex < ClassSources.Num(); ++SourceIndex)
	{
		const FMVVMViewClass_Source& ClassSource = ClassSources[SourceIndex];
		FMVVMView_Source& ViewSource = Sources[SourceIndex];
		UninitializeSourceBindings(FMVVMViewClass_SourceKey(SourceIndex), ClassSource, ViewSource);
	}

	// Remove all delayed bindings from the 
	UMVVMBindingSubsystem* BindingSubsystem = GEngine->GetEngineSubsystem<UMVVMBindingSubsystem>();
	if (BindingSubsystem)
	{
		BindingSubsystem->RemoveDelayedBindings(this);

		if (NumberOfSourceWithTickBinding == 0 && bHasDefaultTickBinding)
		{
			BindingSubsystem->RemoveViewWithTickBinding(this);
		}
	}
	bHasDefaultTickBinding = false;
}


void UMVVMView::InitializeSourceBindings(FMVVMView_SourceKey SourceKey, bool bRunAllBindings)
{
	FMVVMView_Source& ViewSource = Sources[SourceKey.GetIndex()];
	const FMVVMViewClass_Source& ClassSource = GeneratedViewClass->GetSource(ViewSource.ClassKey);

	const bool bIsPointerValid = ViewSource.Source != nullptr;
	const bool bIsBitfieldValid = (ViewSource.ClassKey.GetBit() & ValidSources) != 0;
	ensureMsgf(bIsPointerValid == bIsBitfieldValid, TEXT("The source %s should be valid."), *ClassSource.GetName().ToString());
	if (bIsPointerValid)
	{
		// Register the delegates
		if (ClassSource.GetFieldIds().Num() > 0)
		{
			TScriptInterface<INotifyFieldValueChanged> NotifyFieldValueChanged = ViewSource.Source->Implements<UNotifyFieldValueChanged>() ? TScriptInterface<INotifyFieldValueChanged>(ViewSource.Source) : TScriptInterface<INotifyFieldValueChanged>();
			if (NotifyFieldValueChanged.GetInterface() != nullptr)
			{
				int32 HandleIndex = 0;
				for (const FMVVMViewClass_FieldId& FieldId : ClassSource.GetFieldIds())
				{
					if (ensureMsgf(FieldId.IsValid(), TEXT("Invalid field. It failed somewhere in the compiler.")))
					{
						UE::FieldNotification::FFieldMulticastDelegate::FDelegate Delegate = UE::FieldNotification::FFieldMulticastDelegate::FDelegate::CreateUObject(this, &UMVVMView::HandledLibraryBindingValueChanged, SourceKey);
						FDelegateHandle Handle = NotifyFieldValueChanged->AddFieldValueChangedDelegate(FieldId.GetFieldId(), MoveTemp(Delegate));
						ensure(Handle.IsValid());
						RegisteredBindings.Add( { SourceKey.GetIndex(), HandleIndex }, Handle);
						++ViewSource.RegisteredCount;
					}
					++HandleIndex;
				}
			}
		}

		// Run the bindings
		if (bRunAllBindings)
		{
			// Run all the bindings from this source but only once.
			TArray<FMVVMViewClass_BindingKey, TInlineAllocator<8>> ExecutedBindings;
			for (const FMVVMViewClass_SourceBinding& SourceBinding : ClassSource.GetBindings())
			{
				if (!ExecutedBindings.Contains(SourceBinding.GetBindingKey()))
				{
					ExecutedBindings.Add(SourceBinding.GetBindingKey());
					const FMVVMViewClass_Binding& ClassBinding = GeneratedViewClass->GetBinding(SourceBinding.GetBindingKey());
					ExecuteBindingImmediately(ClassBinding, SourceBinding.GetBindingKey());
				}
			}

			TArray<FMVVMViewClass_ConditionKey, TInlineAllocator<8>> ExecutedConditions;
			for (const FMVVMViewClass_SourceCondition& SourceCondition : ClassSource.GetConditions())
			{
				if (!ExecutedConditions.Contains(SourceCondition.GetConditionKey()))
				{
					ExecutedConditions.Add(SourceCondition.GetConditionKey());
					const FMVVMViewClass_Condition& ClassCondition = GeneratedViewClass->GetCondition(SourceCondition.GetConditionKey());
					ExecuteConditionInternal(SourceCondition);
				}
			}

		}
		else
		{
			for (const FMVVMViewClass_SourceBinding& SourceBinding : ClassSource.GetBindings())
			{
				if (SourceBinding.ExecuteAtInitialization())
				{
					const FMVVMViewClass_Binding& ClassBinding = GeneratedViewClass->GetBinding(SourceBinding.GetBindingKey());
					ExecuteBindingImmediately(ClassBinding, SourceBinding.GetBindingKey());
				}
			}

			for (const FMVVMViewClass_SourceCondition& SourceCondition : ClassSource.GetConditions())
			{
				const FMVVMViewClass_Condition& ClassCondition = GeneratedViewClass->GetCondition(SourceCondition.GetConditionKey());
				ExecuteConditionInternal(SourceCondition);
			}
		}

		if (ClassSource.HasTickBindings())
		{
			++NumberOfSourceWithTickBinding;
			if (NumberOfSourceWithTickBinding == 1 && !bHasDefaultTickBinding)
			{
				GEngine->GetEngineSubsystem<UMVVMBindingSubsystem>()->AddViewWithTickBinding(this);
			}
		}
	}
	else if (!ClassSource.IsOptional())
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(FText::Format(LOCTEXT("InitializeBindingFailInvalidSource", "The source '{0}' is invalid and could not initialize the bindings.")
			, FText::FromName(ClassSource.GetName())
		));
	}
	ViewSource.bBindingsInitialized = true;
}


void UMVVMView::UninitializeSourceBindings(FMVVMViewClass_SourceKey SourceKey, const FMVVMViewClass_Source& ClassSource, FMVVMView_Source& ViewSource)
{
	bool bResult = false;

	// The view can be bBindingsInitialized but the pointer could invalid then and not actual bindgins was registered.
	if (ViewSource.bBindingsInitialized && ViewSource.RegisteredCount > 0)
	{
		if (ViewSource.Source)
		{
			TScriptInterface<INotifyFieldValueChanged> SourceAsInterface = ViewSource.Source;
			checkf(SourceAsInterface.GetInterface(), TEXT("It was added as a INotifyFieldValueChanged. It should still be."));

			int32 HandleIndex = 0;
			int32 HandlesRemoved = 0;
			for (const FMVVMViewClass_FieldId& FieldId : ClassSource.GetFieldIds())
			{
				if (FieldId.IsValid())
				{
					FDelegateHandle Handle;
					ensure(RegisteredBindings.RemoveAndCopyValue({ SourceKey.GetIndex(), HandleIndex }, Handle));
					ensure(SourceAsInterface->RemoveFieldValueChangedDelegate(FieldId.GetFieldId(), Handle));
					++HandlesRemoved;
				}

				++HandleIndex;
			}
			ensure(HandlesRemoved == ViewSource.RegisteredCount);
			ViewSource.RegisteredCount = 0;
		}
		else
		{
			UE::MVVM::FMessageLog Log(GetUserWidget());
			Log.Error(FText::Format(LOCTEXT("UninitializeBindingFailInvalidSource", "The source '{0}' is invalid and could not uninitialize the bindings.")
				, FText::FromName(ClassSource.GetName())
			));
		}

		if (ClassSource.HasTickBindings())
		{
			--NumberOfSourceWithTickBinding;
			check(NumberOfSourceWithTickBinding >= 0);
			if (NumberOfSourceWithTickBinding == 0 && !bHasDefaultTickBinding)
			{
				if (UMVVMBindingSubsystem* BindingSubsystem = GEngine->GetEngineSubsystem<UMVVMBindingSubsystem>())
				{
					BindingSubsystem->RemoveViewWithTickBinding(this);
				}
			}
		}
	}

	ViewSource.bBindingsInitialized = false;
}


void UMVVMView::ExecuteBindingImmediately(const FMVVMViewClass_Binding& ClassBinding, FMVVMViewClass_BindingKey KeyForLog) const
{
	check(GeneratedViewClass);
	check(GetUserWidget())
	if ((ClassBinding.GetSources() & ValidSources) == ClassBinding.GetSources())
	{
	
		// All the source are valid. Run the binding.
		FMVVMCompiledBindingLibrary::EConversionFunctionType FunctionType = ClassBinding.GetBinding().HasComplexConversionFunction() ? FMVVMCompiledBindingLibrary::EConversionFunctionType::Complex : FMVVMCompiledBindingLibrary::EConversionFunctionType::Simple;
		TValueOrError<void, FMVVMCompiledBindingLibrary::EExecutionFailingReason> ExecutionResult = GeneratedViewClass->GetBindingLibrary().Execute(GetUserWidget(), ClassBinding.GetBinding(), FunctionType);

#if UE_WITH_MVVM_DEBUGGING
		if (ExecutionResult.HasError())
		{
			UE::MVVM::FMessageLog Log(GetUserWidget());
			Log.Error(FText::Format(LOCTEXT("ExecuteBindingFailGeneric", "The binding '{0}' was not executed. {1}.")
				, FText::FromString(ClassBinding.ToString(GeneratedViewClass, FMVVMViewClass_Binding::FToStringArgs::Short()))
				, FMVVMCompiledBindingLibrary::LexToText(ExecutionResult.GetError())
			));

			UE::MVVM::FDebugging::BroadcastLibraryBindingExecuted(this, KeyForLog, ExecutionResult.GetError());
		}
		else
		{
			if (bLogBinding)
			{
				UE::MVVM::FMessageLog Log(GetUserWidget());
				Log.Info(FText::Format(LOCTEXT("ExecuteBindingGeneric", "Execute binding '{0}'.")
					, FText::FromString(ClassBinding.ToString(GeneratedViewClass, FMVVMViewClass_Binding::FToStringArgs::All()))
				));
			}
			UE::MVVM::FDebugging::BroadcastLibraryBindingExecuted(this, KeyForLog);
		}
#else
		if (ExecutionResult.HasError())
		{
			UE::MVVM::FMessageLog Log(GetUserWidget());
			Log.Error(FText::Format(LOCTEXT("ExecuteBindingFailGeneric", "The binding '{0}' was not executed. {1}.")
				, FText::AsNumber(KeyForLog.GetIndex())
				, FMVVMCompiledBindingLibrary::LexToText(ExecutionResult.GetError())
			));
		}
		else if (bLogBinding)
		{
			UE::MVVM::FMessageLog Log(GetUserWidget());
			Log.Info(FText::Format(LOCTEXT("ExecuteBindingGeneric", "Execute binding '{0}'.")
				, FText::AsNumber(KeyForLog.GetIndex())
			));
		}
#endif
	}
	else
	{
		const uint64 MissingSources = ClassBinding.GetSources() & (~ValidSources);
		if ((MissingSources & GeneratedViewClass->GetOptionalSources()) != MissingSources)
		{
#if UE_WITH_MVVM_DEBUGGING
			UE::MVVM::FMessageLog Log(GetUserWidget());
			Log.Error(FText::Format(LOCTEXT("ExecuteBindingFailInvalidSource", "The binding '{0}' was not executed. There are invalid sources.")
				, FText::FromString(ClassBinding.ToString(GeneratedViewClass, FMVVMViewClass_Binding::FToStringArgs::Short()))
			));
			UE::MVVM::FDebugging::BroadcastLibraryBindingExecuted(this, KeyForLog, FMVVMCompiledBindingLibrary::EExecutionFailingReason::InvalidSource);
#else
			UE::MVVM::FMessageLog Log(GetUserWidget());
			Log.Error(FText::Format(LOCTEXT("ExecuteBindingFailInvalidSource", "The binding '{0}' was not executed. There are invalid sources.")
				, FText::AsNumber(KeyForLog.GetIndex())
			));
#endif
		}
	}
}


bool UMVVMView::EvaluateSource(FMVVMViewClass_SourceKey SourceKey)
{
	if (!SourceKey.IsValid())
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(FText::Format(LOCTEXT("ReevaluateDynamicSourceViewModelNameNotFound", "Internal error. The dynamic source {0} cound not be found."), FText::AsNumber(SourceKey.GetIndex())));
		return false;
	}

	UUserWidget* UserWidget = GetUserWidget();
	const FMVVMViewClass_Source& ClassSource = GeneratedViewClass->GetSource(SourceKey);
	UObject* NewSource = ClassSource.GetOrCreateInstance(GeneratedViewClass, this, UserWidget);
	ensureMsgf((NewSource == nullptr || NewSource->GetClass()->ImplementsInterface(UNotifyFieldValueChanged::StaticClass())), TEXT("The source has implement the interface. It should be check at compile time."));
	bool bResult = SetSourceInternal(SourceKey, NewSource, true);
	if (!bResult)
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(FText::Format(LOCTEXT("ExecuteBindingFailEvaluate", "The evaluate source '{0}' fail execution.")
			, FText::FromName(ClassSource.GetName())));
	}
	return bResult;
}


void UMVVMView::HandledLibraryBindingValueChanged(UObject* InSource, UE::FieldNotification::FFieldId InFieldId, FMVVMView_SourceKey ViewSourceKey)
{
	SCOPE_CYCLE_COUNTER(STAT_UMG_Viewmodel_ExecuteBinding_ValueChanged);

	check(InSource);
	check(InFieldId.IsValid());
	check(ViewSourceKey.IsValid());

	if (ensure(GeneratedViewClass) && ensure(Sources.IsValidIndex(ViewSourceKey.GetIndex())))
	{
		const FMVVMView_Source& ViewSource = Sources[ViewSourceKey.GetIndex()];
		if (!ViewSource.bBindingsInitialized || !ViewSource.bSourceInitialized)
		{
			// we do not want to run a binding while we are initializing the bindings.
			UE::MVVM::FMessageLog Log(GetUserWidget());
			Log.Warning(FText::Format(LOCTEXT("ExecuteBindingFailWhileInitializing", "The Field '{0}' could not execute the bindings while initializing."), FText::FromName(InFieldId.GetName())));
			return;
		}

		const FMVVMViewClass_SourceKey ClassSourceKey = ViewSource.ClassKey;
		const FMVVMViewClass_Source& ClassSource = GeneratedViewClass->GetSource(ClassSourceKey);

		// Run all evaluates
		if (ClassSource.HasEvaluateBindings())
		{
			for (const FMVVMViewClass_EvaluateSource& ClassEvaluate : GeneratedViewClass->GetEvaluateSources())
			{
				if (ClassEvaluate.GetParentSource() == ClassSourceKey && ClassEvaluate.GetFieldId().GetFieldName() == InFieldId.GetName())
				{
					EvaluateSource(ClassEvaluate.GetSource());
				}
			}
		}

		// Run all bindings
		for (const FMVVMViewClass_SourceBinding& SourceBinding : ClassSource.GetBindings())
		{
			if (SourceBinding.GetFieldId().GetFieldName() == InFieldId.GetName())
			{
				ExecuteBindingInternal(SourceBinding);
			}
		}

		// Run all conditions
		for (const FMVVMViewClass_SourceCondition& SourceCondition : ClassSource.GetConditions())
		{
			if (SourceCondition.GetFieldId().GetFieldName() == InFieldId.GetName())
			{
				ExecuteConditionInternal(SourceCondition);
			}
		}
	}
}

void UMVVMView::ExecuteConditionInternal(const FMVVMViewClass_SourceCondition& SourceCondition) const
{
	const FMVVMViewClass_Condition& ClassCondition = GeneratedViewClass->GetCondition(SourceCondition.GetConditionKey());

	UUserWidget* UserWidget = GetUserWidget();
	UFunction* FunctionToBind = UserWidget->GetClass()->FindFunctionByName(ClassCondition.GetUserWidgetFunctionName());

	UE::MVVM::BindingHelper::ExecuteFunction_NoReturnValue(FunctionToBind, UserWidget);
}


void UMVVMView::ExecuteBindingInternal(const FMVVMViewClass_SourceBinding& SourceBinding) const
{
	const FMVVMViewClass_Binding& ClassBinding = GeneratedViewClass->GetBinding(SourceBinding.GetBindingKey());
	if (ensure(ClassBinding.IsOneWay()))
	{
		const EMVVMExecutionMode ExecutionMode = ClassBinding.GetExecuteMode();
		if (ExecutionMode == EMVVMExecutionMode::Immediate)
		{
			// Test for recursivity
			const UMVVMView* Self = this;
			if (UE::MVVM::Private::RecursiveDetector.FindByPredicate([Self, InCompiledBindingIndex = SourceBinding.GetBindingKey()](const UE::MVVM::Private::FRecursiveDetctionElement& Element)
				{
					return Element.Get<0>() == Self && Element.Get<1>() == InCompiledBindingIndex;
				}) != nullptr)
			{
				ensureAlwaysMsgf(false, TEXT("Recursive binding detected"));
				//Todo add more infos. Callstack maybe? Log the chain?
				UE::MVVM::FMessageLog Log(Self->GetUserWidget());
				Log.Warning(LOCTEXT("RecursionDetected", "A recursive binding was detected (ie. A->B->C->A->B->C) at runtime."));
				return;
			}

				{
					UE::MVVM::Private::RecursiveDetector.Emplace(this, SourceBinding.GetBindingKey());
					ExecuteBindingImmediately(ClassBinding, SourceBinding.GetBindingKey());
					UE::MVVM::Private::RecursiveDetector.Pop();
				}
		}
		else if (ExecutionMode == EMVVMExecutionMode::Delayed)
		{
			GEngine->GetEngineSubsystem<UMVVMBindingSubsystem>()->AddDelayedBinding(this, SourceBinding.GetBindingKey());
		}
	}
}


void UMVVMView::ExecuteDelayedBinding(const FMVVMViewClass_BindingKey& DelayedBinding) const
{
	SCOPE_CYCLE_COUNTER(STAT_UMG_Viewmodel_ExecuteBinding_Delayed);

	if (ensure(GeneratedViewClass) && bBindingsInitialized)
	{
		ensure(bSourcesInitialized);
		if (ensure(GeneratedViewClass->GetBindings().IsValidIndex(DelayedBinding.GetIndex())))
		{
			// Test for recursivity
			const UMVVMView* Self = this;
			if (UE::MVVM::Private::RecursiveDetector.FindByPredicate([Self, InCompiledBindingIndex = DelayedBinding](const UE::MVVM::Private::FRecursiveDetctionElement& Element)
				{
					return Element.Get<0>() == Self && Element.Get<1>() == InCompiledBindingIndex;
				}) != nullptr)
			{
				ensureAlwaysMsgf(false, TEXT("Recursive binding detected"));
				//Todo add more infos. Callstack maybe? Log the chain?
				UE::MVVM::FMessageLog Log(Self->GetUserWidget());
				Log.Warning(LOCTEXT("RecursionDetected", "A recursive binding was detected (ie. A->B->C->A->B->C) at runtime."));
				return;
			}

			{
				UE::MVVM::Private::RecursiveDetector.Emplace(this, DelayedBinding);
				const FMVVMViewClass_Binding& ClassBinding = GeneratedViewClass->GetBinding(DelayedBinding);
				ExecuteBindingImmediately(ClassBinding, DelayedBinding);
				UE::MVVM::Private::RecursiveDetector.Pop();
			}
		}
	}
}


void UMVVMView::ExecuteTickBindings() const
{
	if (ensure(GeneratedViewClass))
	{
		ensure(NumberOfSourceWithTickBinding > 0 || bHasDefaultTickBinding);

		bool bAtLeastOneBindingWasExecuted = false;
		const TArrayView<const FMVVMViewClass_Binding> ClassBindings = GeneratedViewClass->GetBindings();
		for (int32 BindingIndex = 0; BindingIndex < ClassBindings.Num(); ++BindingIndex)
		{
			const FMVVMViewClass_Binding& ClassBinding = ClassBindings[BindingIndex];
			const EMVVMExecutionMode ExecutionMode = ClassBinding.GetExecuteMode();
			if (ExecutionMode == EMVVMExecutionMode::Tick)
			{
				ExecuteBindingImmediately(ClassBinding, FMVVMViewClass_BindingKey(BindingIndex));
				bAtLeastOneBindingWasExecuted = true;
			}
		}

		if (!bAtLeastOneBindingWasExecuted)
		{
			ensureMsgf(false, TEXT("No tick binding was found but the flags was true."));
		}
	}
}


TScriptInterface<INotifyFieldValueChanged> UMVVMView::GetViewModel(FName ViewModelName) const
{
	int32 FoundIndex = GeneratedViewClass->GetSources().IndexOfByPredicate([ViewModelName](const FMVVMViewClass_Source& Other)
	{
		return Other.GetName() == ViewModelName && Other.IsViewModel();
	});
	return Sources.IsValidIndex(FoundIndex) ? Sources[FoundIndex].Source : nullptr;
}


bool UMVVMView::SetViewModel(FName ViewModelName, TScriptInterface<INotifyFieldValueChanged> NewValue)
{
	if (ViewModelName.IsNone())
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(LOCTEXT("SetViewModelInvalidName", "The viewmodel name is empty."));
		return false;
	}

	if (GeneratedViewClass == nullptr)
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(LOCTEXT("SetViewModelInvalidClass", "The view is not constructed."));
		return false;
	}

	const int32 ClassSourceIndex = GeneratedViewClass->GetSources().IndexOfByPredicate([ViewModelName](const FMVVMViewClass_Source& Other)
		{
			return Other.GetName() == ViewModelName;
		});

	if (ClassSourceIndex == INDEX_NONE)
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(LOCTEXT("SetViewModelViewModelNameNotFound", "The viewmodel name could not be found."));
		return false;
	}
	return SetSourceInternal(FMVVMViewClass_SourceKey(ClassSourceIndex), NewValue, false);
}


bool UMVVMView::SetViewModelByClass(TScriptInterface<INotifyFieldValueChanged> NewValue)
{
	if (!NewValue.GetObject())
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(LOCTEXT("SetViewModelInvalidObject", "The new viewmodel is null."));
		return false;
	}

	if (GeneratedViewClass == nullptr)
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(LOCTEXT("SetViewModelInvalidGeneratedViewClass", "The view is not initialized."));
		return false;
	}

	int32 FoundSourceIndex = INDEX_NONE;
	int32 SourceIndex = 0;
	for (const FMVVMViewClass_Source& ClassSource : GeneratedViewClass->GetSources())
	{
		if (NewValue.GetObject()->GetClass()->IsChildOf(ClassSource.GetSourceClass()))
		{
			if (FoundSourceIndex == INDEX_NONE)
			{
				FoundSourceIndex = SourceIndex;
			}
			else
			{
				UE::MVVM::FMessageLog Log(GetUserWidget());
				Log.Error(FText::Format(LOCTEXT("SetViewModelViewModelNotUnique", "More than one viewmodel match the type of the passed instance. Make sure there exists only one viewmodel of type {0} in the widget blueprint, or try SetViewModel by name."), FText::FromName(NewValue.GetObject()->GetClass()->GetFName())));
				return false;
			}
		}
		++SourceIndex;
	}

	if (FoundSourceIndex == INDEX_NONE)
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(LOCTEXT("SetViewModelViewModelNotFound", "A created viewmodel matching the class of the passed instance could not be found."));
		return false;
	}

	return SetSourceInternal(FMVVMViewClass_SourceKey(FoundSourceIndex), NewValue, false);
}


bool UMVVMView::SetSourceInternal(FMVVMViewClass_SourceKey ClassSourceKey, TScriptInterface<INotifyFieldValueChanged> NewValue, bool bForDynamicSource)
{
	SCOPE_CYCLE_COUNTER(STAT_UMG_Viewmodel_SetSource);

	check(Sources.IsValidIndex(ClassSourceKey.GetIndex()));
	FMVVMView_SourceKey ViewSourceKey = FMVVMView_SourceKey(ClassSourceKey.GetIndex());

	const FMVVMViewClass_Source& ClassSource = GeneratedViewClass->GetSource(ClassSourceKey);
	FMVVMView_Source& ViewSource = Sources[ViewSourceKey.GetIndex()];

	if (bForDynamicSource && !ClassSource.CanBeEvaluated())
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(FText::Format(LOCTEXT("SetViewModelCannotBeEvalaute", "The new viewmodel {0} cannot be evaluated again."), FText::FromName(ClassSource.GetName())));
		return false;
	}
	else if (!bForDynamicSource && !ClassSource.CanBeSet())
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(FText::Format(LOCTEXT("SetViewModelCannotBeSet", "The new viewmodel {0} cannot be set."), FText::FromName(ClassSource.GetName())));
		return false;
	}
			
	if (NewValue.GetObject()
		&& !NewValue.GetObject()->GetClass()->IsChildOf(ClassSource.GetSourceClass()))
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(LOCTEXT("SetViewModelInvalidValueType", "The new viewmodel is not of the expected type."));
		UE_LOGF(LogMVVM, Error, "The viewmodel name '%ls' is invalid for the view '%ls'", *ClassSource.GetName().ToString(), *GetFullName());
		return false;
	}

	UObject* PreviousValue = ViewSource.Source;
	{
		// Sanity check. Test if the bitfield matches the cached value.
		const bool bIsValid = PreviousValue != nullptr;
		const bool bIsBitfieldValid = (ClassSourceKey.GetBit() & ValidSources) != 0;
		ensureMsgf(bIsValid == bIsBitfieldValid, TEXT("The source %s should be valid."), *ClassSource.GetName().ToString());
	}

	if (PreviousValue != NewValue.GetObject())
	{
		bool bPreviousBindingsInitialized = ViewSource.bBindingsInitialized;
		bool bPreviousSourceInitialized = ViewSource.bSourceInitialized;

		if (ViewSource.bBindingsInitialized)
		{
			// if we had bindings init, then the vm gets set to null, then the vm gets set to a valid ptr, we want to update the binding values as it gets set
			// the only issue is that UninitializeSourceBindings expect the source to be non null
			// if the previous source was null, we don't need to do the work anyway
			if (ViewSource.Source != nullptr)
			{
				UninitializeSourceBindings(ClassSourceKey, ClassSource, ViewSource);
			}
			GEngine->GetEngineSubsystem<UMVVMBindingSubsystem>()->RemoveDelayedBindings(this, ClassSourceKey);
		}
		UninitializeSource(ViewSourceKey);

		InitializeSourceInternal(NewValue.GetObject(), ClassSourceKey, ClassSource, ViewSource);
		ViewSource.bSetManually = !bForDynamicSource;
		ViewSource.bSourceInitialized = bPreviousSourceInitialized;

		// remove events, re-add the events.
		if (bEventsInitialized)
		{
			ReinitializeEvents(ClassSourceKey, PreviousValue, NewValue.GetObject());
		}

		if (bPreviousBindingsInitialized)
		{
			// If binding is A.B.Property, and A is set, we need to evaluate/update B
			if (ClassSource.HasEvaluateBindings())
			{
				for (const FMVVMViewClass_EvaluateSource& ClassEvaluate : GeneratedViewClass->GetEvaluateSources())
				{
					if (ClassEvaluate.GetParentSource() == ClassSourceKey)
					{
						EvaluateSource(ClassEvaluate.GetSource());
					}
				}
			}

			// Initialize bindings and execute them without delay.
			//A binding might have different source dependencies. The binding might not have the "ExecAtInit" flag for this source.
			//Force the execution of all the bindings for this source.
			InitializeSourceBindings(ViewSourceKey, true);
		}

#if UE_WITH_MVVM_DEBUGGING
		UE::MVVM::FDebugging::BroadcastViewSourceValueChanged(this, ClassSourceKey, ViewSourceKey);
#endif
	}
	else if (ClassSource.AlwaysExecuteBindingsOnSetSource() && ViewSource.bBindingsInitialized)
	{
		ExecuteViewModelBindingsInternal(ClassSourceKey);
	}

	return true;
}


bool UMVVMView::ExecuteViewModelBindings(FName ViewModelName)
{
	if (ViewModelName.IsNone())
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(LOCTEXT("SetViewModelInvalidName", "The viewmodel name is empty."));
		return false;
	}

	if (GeneratedViewClass == nullptr)
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(LOCTEXT("SetViewModelInvalidClass", "The view is not constructed."));
		return false;
	}

	const int32 ClassSourceIndex = GeneratedViewClass->GetSources().IndexOfByPredicate([ViewModelName](const FMVVMViewClass_Source& Other)
		{
			return Other.GetName() == ViewModelName;
		});

	if (ClassSourceIndex == INDEX_NONE)
	{
		UE::MVVM::FMessageLog Log(GetUserWidget());
		Log.Error(LOCTEXT("SetViewModelViewModelNameNotFound", "The viewmodel name could not be found."));
		return false;
	}

	ExecuteViewModelBindingsInternal(FMVVMViewClass_SourceKey(ClassSourceIndex));
	return true;
}


void UMVVMView::ExecuteViewModelBindingsInternal(FMVVMViewClass_SourceKey ClassSourceKey)
{
	SCOPE_CYCLE_COUNTER(STAT_UMG_Viewmodel_ExecuteViewModel);

	const FMVVMViewClass_Source& ClassSource = GeneratedViewClass->GetSource(ClassSourceKey);

	// Run all evaluates
	if (ClassSource.HasEvaluateBindings())
	{
		for (const FMVVMViewClass_EvaluateSource& ClassEvaluate : GeneratedViewClass->GetEvaluateSources())
		{
			if (ClassEvaluate.GetParentSource() == ClassSourceKey)
			{
				EvaluateSource(ClassEvaluate.GetSource());
			}
		}
	}

	// Run the bindings
	for (const FMVVMViewClass_SourceBinding& SourceBinding : ClassSource.GetBindings())
	{
		ExecuteBindingInternal(SourceBinding);
	}
	// Run all conditions
	for (const FMVVMViewClass_SourceCondition& SourceCondition : ClassSource.GetConditions())
	{
		ExecuteConditionInternal(SourceCondition);
	}
}


void UMVVMView::HandleViewModelCollectionChanged()
{
	if (ensure(GeneratedViewClass))
	{
		const int32 NumberOfSources = GeneratedViewClass->GetSources().Num();
		for (int32 Index = 0; Index < NumberOfSources; ++Index)
		{
			FMVVMView_SourceKey SourceKey(Index);
			FMVVMView_Source& ViewSource = Sources[SourceKey.GetIndex()];
			const FMVVMViewClass_Source& ClassSource = GeneratedViewClass->GetSource(ViewSource.ClassKey);
			if (ViewSource.bSourceInitialized && !ViewSource.bSetManually && ClassSource.RequireGlobalViewModelCollectionUpdate())
			{
				TScriptInterface<INotifyFieldValueChanged> NewValue = ClassSource.GetGlobalCollectionViewModel(GetUserWidget());
				if (NewValue.GetObject() != ViewSource.Source)
				{
					SetSourceInternal(ViewSource.ClassKey, NewValue, true);
				}
			}
		}
	}
}


void UMVVMView::InitializeEvents()
{
	if (bEventsInitialized)
	{
		return;
	}

	UUserWidget* UserWidget = GetUserWidget();
	check(UserWidget);
	check(GeneratedViewClass);

	const TArrayView<const FMVVMViewClass_Event>& ClassEvents = GeneratedViewClass->GetEvents();

	ensure(BoundEvents.IsEmpty());
	BoundEvents.Reset(ClassEvents.Num());
	for (int32 Index = 0; Index < ClassEvents.Num(); ++Index)
	{
		const FMVVMViewClass_Event& ClassEvent = ClassEvents[Index];
		BindEvent(ClassEvent, FMVVMViewClass_EventKey(Index));
	}

	const TArrayView<const TObjectPtr<UMVVMViewClassExtension>> ClassExtensions = GeneratedViewClass->GetViewClassExtensions();
	check(ClassExtensions.Num() == Extensions.Num());
	for (int32 Index = 0; Index < ClassExtensions.Num(); ++Index)
	{
		ClassExtensions[Index]->OnEventsInitialized(GetUserWidget(), this, Extensions[Index]);
	}

	bEventsInitialized = true;
}


void UMVVMView::UninitializeEvents()
{
	if (!bEventsInitialized)
	{
		return;
	}

	check(GeneratedViewClass);
	
	bEventsInitialized = false;

	const TArrayView<const TObjectPtr<UMVVMViewClassExtension>> ClassExtensions = GeneratedViewClass->GetViewClassExtensions();
	check(ClassExtensions.Num() == Extensions.Num());
	for (int32 Index = 0; Index < ClassExtensions.Num(); ++Index)
	{
		ClassExtensions[Index]->OnEventsUninitialized(GetUserWidget(), this, Extensions[Index]);
	}

	for (int32 Index = BoundEvents.Num() - 1; Index >= 0; --Index)
	{
		UnbindEvent(Index);
	}
}


void UMVVMView::BindEvent(const FMVVMViewClass_Event& ClassEvent, FMVVMViewClass_EventKey KeyForLog)
{
	UUserWidget* UserWidget = GetUserWidget();
	check(UserWidget);

	auto LogMessage = [Self=this, UserWidget, &ClassEvent, KeyForLog](const FText& Message)
		{
			UE::MVVM::FMessageLog Log(UserWidget);
			Log.Error(FText::Format(LOCTEXT("EnableLibraryEventFailed_Format", "Widget '{0}' can't register event '{1}'. {2}")
				, FText::FromString(Self->GetFullName())
#if UE_WITH_MVVM_DEBUGGING
				, FText::FromString(ClassEvent.ToString(Self->GeneratedViewClass, FMVVMViewClass_Event::FToStringArgs()))
#else
				, FText::AsNumber(KeyForLog.GetIndex())
#endif
				, Message
			));
		};

	const UFunction* FunctionToBind = UserWidget->GetClass()->FindFunctionByName(ClassEvent.GetUserWidgetFunctionName());
	if (FunctionToBind == nullptr)
	{
		LogMessage(LOCTEXT("EnableLibraryEventFailed_FunctionNotFound", "Function not found"));
		return;
	}

	const FMVVMCompiledBindingLibrary& Library = GeneratedViewClass->GetBindingLibrary();
	TValueOrError<UE::MVVM::FFieldContext, void> FieldPathResult = Library.EvaluateFieldPath(UserWidget, ClassEvent.GetMulticastDelegatePath());
	if (FieldPathResult.HasError())
	{
		bool bIsOptional = ClassEvent.GetSourceKey().IsValid() && (GeneratedViewClass->GetOptionalSources() & ClassEvent.GetSourceKey().GetBit()) != 0;
		if (!bIsOptional)
		{
			// Problem. The path can be long Viewmodel.ObjectA.ObjectB.Multicast.
			//It should warn but will not if the Viewmodel is valid but ObjectA is invalid.
			LogMessage(LOCTEXT("EnableLibraryEventFailed_CantEvaluateBindingPath", "The binding path couldn't be evaluated."));
		}
		return;
	}

	UE::MVVM::FFieldContext& FieldContext = FieldPathResult.GetValue();
	if (FieldContext.GetObjectVariant().IsNull() || !FieldContext.GetObjectVariant().IsUObject())
	{
		bool bIsOptional = ClassEvent.GetSourceKey().IsValid() && (GeneratedViewClass->GetOptionalSources() & ClassEvent.GetSourceKey().GetBit()) != 0;
		if (!bIsOptional)
		{
			// Problem. The path can be long Viewmodel.ObjectA.ObjectB.Multicast.
			//It should warn but will not if the Viewmodel is valid but ObjectA is invalid.
			LogMessage(LOCTEXT("EnableLibraryEventFailed_OwnerInvalid", "The owner is invalid."));
		}
		return;
	}


	FMulticastDelegateProperty* MulticastDelegateProp = FieldContext.GetFieldVariant().IsProperty() ? CastField<FMulticastDelegateProperty>(FieldContext.GetFieldVariant().GetProperty()) : nullptr;
	if (!MulticastDelegateProp)
	{
		LogMessage(LOCTEXT("EnableLibraryEventFailed_NotAProperty", "The path doesn't point to a multicast delegate property."));
		return;
	}

	UObject* EventObject = FieldContext.GetObjectVariant().GetUObject();
	FBoundEvent& Result = BoundEvents.AddDefaulted_GetRef();
	Result.Object = EventObject;
	Result.PropertyName = MulticastDelegateProp->GetFName();
	Result.EventKey = KeyForLog;

	FScriptDelegate Delegate;
	Delegate.BindUFunction(UserWidget, ClassEvent.GetUserWidgetFunctionName());
	MulticastDelegateProp->AddDelegate(MoveTemp(Delegate), EventObject);
}


void UMVVMView::UnbindEvent(int32 Index)
{
	const FBoundEvent& BoundEvent = BoundEvents[Index];

	UObject* EventObject= BoundEvent.Object.Get();
	if (EventObject == nullptr)
	{
		BoundEvents.RemoveAtSwap(Index);
		return;
	}

	FMulticastDelegateProperty* MulticastDelegateProp = CastField<FMulticastDelegateProperty>(EventObject->GetClass()->FindPropertyByName(BoundEvent.PropertyName));
	if (!MulticastDelegateProp)
	{
		BoundEvents.RemoveAtSwap(Index);
		return;
	}

	UUserWidget* UserWidget = GetUserWidget();
	check(UserWidget);

	FScriptDelegate Delegate;
	Delegate.BindUFunction(UserWidget, GeneratedViewClass->GetEvent(BoundEvent.EventKey).GetUserWidgetFunctionName());
	MulticastDelegateProp->RemoveDelegate(Delegate, EventObject);

	BoundEvents.RemoveAtSwap(Index);
}


void UMVVMView::ReinitializeEvents(FMVVMViewClass_SourceKey SourceKey, UObject* PreviousValue, UObject* NewValue)
{
	if (BoundEvents.Num() > 0 && PreviousValue)
	{
		FWeakObjectPtr WeakPreviousValue = PreviousValue;
		for (int32 Index = BoundEvents.Num() - 1; Index >= 0; --Index)
		{
			const FBoundEvent& BoundEvent = BoundEvents[Index];
			if (BoundEvent.Object == WeakPreviousValue)
			{
				UnbindEvent(Index);
			}
		}
	}

	const TArrayView<const FMVVMViewClass_Event>& ClassEvents = GeneratedViewClass->GetEvents();
	for (int32 Index = 0; Index < ClassEvents.Num(); ++Index)
	{
		const FMVVMViewClass_Event& ClassEvent = ClassEvents[Index];
		if (ClassEvent.GetSourceKey() == SourceKey)
		{
			BindEvent(ClassEvent, FMVVMViewClass_EventKey(Index));
		}
	}
}


bool UMVVMView::AreSourcesValidForEvent(int32 EventKeyIndex) const
{
	FMVVMViewClass_EventKey EventKey = FMVVMViewClass_EventKey(EventKeyIndex);
	if (GeneratedViewClass)
	{
		const FMVVMViewClass_Event& ClassEvent = GeneratedViewClass->GetEvent(EventKey);
		uint64 EventSources = ClassEvent.GetSources();
		if ((EventSources & ValidSources) == EventSources)
		{
			return true;
		}
		
		const uint64 MissingSources = EventSources & (~ValidSources);
		if ((MissingSources & GeneratedViewClass->GetOptionalSources()) != MissingSources)
		{
#if UE_WITH_MVVM_DEBUGGING
			UE::MVVM::FMessageLog Log(GetUserWidget());
			Log.Error(FText::Format(LOCTEXT("ExecuteEventFailInvalidSource", "The event '{0}' was not executed. There are invalid sources.")
				, FText::FromString(ClassEvent.ToString(GeneratedViewClass, FMVVMViewClass_Event::FToStringArgs::Short()))
			));
#else
			UE::MVVM::FMessageLog Log(GetUserWidget());
			Log.Error(FText::Format(LOCTEXT("ExecuteEventFailInvalidSource", "The event '{0}' was not executed. There are invalid sources.")
				, FText::AsNumber(EventKey.GetIndex())
			));
#endif
		}
	}
	return false;
}


bool UMVVMView::AreSourcesValidForBinding(int32 InBindingKey) const
{
	FMVVMViewClass_BindingKey BindingKey = FMVVMViewClass_BindingKey(InBindingKey);
	if (GeneratedViewClass)
	{
		const FMVVMViewClass_Binding& ClassBinding = GeneratedViewClass->GetBinding(BindingKey);
		uint64 BindingSources = ClassBinding.GetSources();
		if ((BindingSources & ValidSources) == BindingSources)
		{
			return true;
		}
		
		const uint64 MissingSources = BindingSources & (~ValidSources);
		if ((MissingSources & GeneratedViewClass->GetOptionalSources()) != MissingSources)
		{
			// Only log invalid bindings as info, since it is possible expected behavior for async functions
#if UE_WITH_MVVM_DEBUGGING
			UE::MVVM::FMessageLog Log(GetUserWidget());
			Log.Info(FText::Format(LOCTEXT("ExecuteGraphBindingFailInvalidSource", "The binding '{0}' was not executed. There are invalid sources.")
				, FText::FromString(ClassBinding.ToString(GeneratedViewClass, FMVVMViewClass_Binding::FToStringArgs::Short()))
			));
#else
			UE::MVVM::FMessageLog Log(GetUserWidget());
			Log.Info(FText::Format(LOCTEXT("ExecuteGraphBindingFailInvalidSource", "The binding '{0}' was not executed. There are invalid sources.")
				, FText::AsNumber(BindingKey.GetIndex())
			));
#endif
		}
	}
	return false;
}


#if WITH_EDITOR
void UMVVMView::HandleBlueprintPreCompile(UBlueprint* Blueprint)
{
	if (Blueprint && GeneratedViewClass->GetOuter() == Blueprint->GeneratedClass)
	{
		Destruct();
	}
}
#endif

#undef LOCTEXT_NAMESPACE
