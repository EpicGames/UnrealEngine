// Copyright Epic Games, Inc. All Rights Reserved.

#include "View/MVVMViewClass.h"
#include "Types/MVVMFieldContext.h"
#include "View/MVVMView.h"
#include "View/MVVMViewModelContextResolver.h"
#include "Modules/ModuleManager.h"
#include "ModelViewViewModelModule.h"

#include "Bindings/MVVMFieldPathHelper.h"
#include "Blueprint/UserWidget.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"
#include "Extensions/ContextDataWidgetExtension.h"
#include "MVVMGameSubsystem.h"
#include "MVVMMessageLog.h"
#include "MVVMSubsystem.h"

#include <limits>

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMViewClass)


#define LOCTEXT_NAMESPACE "MVVMViewClass"

namespace UE::MVVM::Private
{
UMVVMViewModelBase* GetGlobalCollectionViewModel(UUserWidget* InUserWidget, const FMVVMViewModelContext& GlobalViewModelInstance, UMVVMViewModelCollectionObject*& Collection)
{
	Collection = nullptr;
	UMVVMViewModelBase* FoundViewModelInstance = nullptr;
	if (GlobalViewModelInstance.IsValid())
	{
		if (const UWorld* World = InUserWidget->GetWorld())
		{
			if (const UGameInstance* GameInstance = World->GetGameInstance())
			{
				Collection = GameInstance->GetSubsystem<UMVVMGameSubsystem>()->GetViewModelCollection();
				if (Collection)
				{
					FoundViewModelInstance = Collection->FindViewModelInstance(GlobalViewModelInstance);
				}
			}
		}
	}

	if (FoundViewModelInstance != nullptr)
	{
		ensureMsgf(FoundViewModelInstance->IsA(GlobalViewModelInstance.ContextClass), TEXT("The Global View Model Instance is not of the expected type."));
	}
	return FoundViewModelInstance;
}
} // namespace

///////////////////////////////////////////////////////////////////////
// 
///////////////////////////////////////////////////////////////////////
UObject* FMVVMViewClass_Source::GetOrCreateInstance(const UMVVMViewClass* InViewClass, UMVVMView* InView, UUserWidget* InUserWidget) const
{
	check(InViewClass);
	check(InView);
	check(InUserWidget);

	UObject* Result = nullptr;
	const bool bOptional = IsOptional();

	auto GetInstanceObjectFromProperty = [&]() -> bool
	{
		TValueOrError<UE::MVVM::FFieldContext, void> FieldPathResult = InViewClass->GetBindingLibrary().EvaluateFieldPath(InUserWidget, FieldPath);
		if (FieldPathResult.HasValue())
		{
			TValueOrError<UObject*, void> ObjectResult = UE::MVVM::FieldPathHelper::EvaluateObjectProperty(FieldPathResult.GetValue());
			if (ObjectResult.HasValue() && ObjectResult.GetValue() != nullptr)
			{
				Result = ObjectResult.GetValue();
			}
		}

		return Result != nullptr || bOptional;
	};

	if ((Flags & (uint16)EFlags::TypeCreateInstance) != 0)
	{
		if (ExpectedSourceType.Get() != nullptr)
		{
			// We fetch the instance that was exposed through the editor if it exists, otherwise we create a fresh one.
			if ((Flags & (uint16)EFlags::IsViewModelInstanceExposed) != 0)
			{
				if (ensure(FieldPath.IsValid()))
				{
					if (!GetInstanceObjectFromProperty())
					{
						UE::MVVM::FMessageLog Log(InUserWidget);
						Log.Error(FText::Format(LOCTEXT("CreateInstanceInvalidStaticInstance", "The source '{0}' was not found for the static view model at initialization."), FText::FromName(PropertyName)));
					}
				}
			}
			else
			{
				Result = NewObject<UObject>(InUserWidget, ExpectedSourceType.Get(), NAME_None, RF_Transient);
			}
		}
		else if (!bOptional)
		{
			UE::MVVM::FMessageLog Log(InUserWidget);
			Log.Error(FText::Format(LOCTEXT("CreateInstanceCreateInstance", "The source '{0}' could not be created. The class is not loaded."), FText::FromName(PropertyName)));
		}
	}
	else if (IsParentContextReference())
	{
		UE::MVVM::FMessageLog Log(InUserWidget);

		if (ExpectedSourceType.Get() != nullptr)
		{
			const FParentSourceInfo ParentInfo = GetParentSupplyingSourceClass(InUserWidget);
			if (ParentInfo.ParentView)
			{
				Result = ParentInfo.Source;

				Log.Info(FText::Format(LOCTEXT("CreateInstanceContextParentFound", "The source '{0}' was found on view from parent widget '{1}'"), FText::FromName(PropertyName), FText::FromString(GetNameSafe(ParentInfo.ParentUserWidget))));
			}
			else if (ParentInfo.Source)
			{
				Result = ParentInfo.Source;

				Log.Info(FText::Format(LOCTEXT("CreateInstanceContextDataFound", "The source '{0}' was found on context data from widget '{1}'"), FText::FromName(PropertyName), FText::FromString(GetNameSafe(ParentInfo.ParentUserWidget))));
			}
			else
			{
				Log.Error(FText::Format(LOCTEXT("CreateInstanceContextParentMissing", "The source '{0}' could not be created. No parent widget found that provides an instance of '{1}'"), FText::FromName(PropertyName), FText::FromString(GetNameSafe(ExpectedSourceType))));
			}		
		}
		else if (!bOptional)
		{
			Log.Error(FText::Format(LOCTEXT("CreateInstanceContextClassMissing", "The source '{0}' could not be created. The class is not loaded."), FText::FromName(PropertyName)));
		}
	}
	else if (IsUserWidget())
	{ 
		Result = InUserWidget;
	}
	else if (Resolver)
	{
		Result = Resolver->CreateInstance(ExpectedSourceType.Get(), InUserWidget, InView);
		if (!Result && !bOptional)
		{
			UE::MVVM::FMessageLog Log(InUserWidget);
			Log.Error(FText::Format(LOCTEXT("CreateInstanceFailResolver", "The source '{0}' could not be created. Resolver returned an invalid value."), FText::FromName(PropertyName)));
		}
		if (Result && !Result->IsA(ExpectedSourceType.Get()))
		{
			Result = nullptr;

			UE::MVVM::FMessageLog Log(InUserWidget);
			Log.Error(FText::Format(LOCTEXT("CreateInstanceFailResolverExpected", "The source '{0}' could not be created. Resolver returned viewodel of an unexpected type."), FText::FromName(PropertyName)));
		}
	}
	else if (GlobalViewModelInstance.IsValid())
	{
		UMVVMViewModelCollectionObject* Collection = nullptr;
		UMVVMViewModelBase* FoundViewModelInstance = UE::MVVM::Private::GetGlobalCollectionViewModel(InUserWidget, GlobalViewModelInstance, Collection);

		if (FoundViewModelInstance != nullptr)
		{
			ensureMsgf(FoundViewModelInstance->IsA(GlobalViewModelInstance.ContextClass), TEXT("The Global View Model Instance is not of the expected type."));
			Result = FoundViewModelInstance;
		}
		else if (!bOptional)
		{
			UE::MVVM::FMessageLog Log(InUserWidget);
			if (Collection)
			{
				Log.Error(FText::Format(LOCTEXT("CreateInstanceFailedGlobal", "The source '{0}' was not found in the global view model collection."), FText::FromName(GlobalViewModelInstance.ContextName)));
			}
			else
			{
				Log.Error(FText::Format(LOCTEXT("CreateInstanceFailedGlobalInstance", "The source '{0}' will be invalid because the global view model collection could not be found."), FText::FromName(GlobalViewModelInstance.ContextName)));
			}
		}
	}
	else if (FieldPath.IsValid())
	{
		if (!GetInstanceObjectFromProperty())
		{
			UE::MVVM::FMessageLog Log(InUserWidget);
			Log.Error(FText::Format(LOCTEXT("CreateInstanceInvalidBinding", "The source '{0}' was evaluated to be invalid at initialization."), FText::FromName(PropertyName)));
		}
	}

	return Result;
}

void FMVVMViewClass_Source::ReleaseInstance(UObject* ViewModel, const UMVVMView* View) const
{
	if (Resolver)
	{
		Resolver->DestroyInstance(ViewModel, View);
	}
}

TScriptInterface<INotifyFieldValueChanged> FMVVMViewClass_Source::GetGlobalCollectionViewModel(UUserWidget* InUserWidget) const
{
	UMVVMViewModelCollectionObject* Collection = nullptr;
	UMVVMViewModelBase* FoundViewModelInstance = UE::MVVM::Private::GetGlobalCollectionViewModel(InUserWidget, GlobalViewModelInstance, Collection);
	return FoundViewModelInstance ? TScriptInterface<INotifyFieldValueChanged>(FoundViewModelInstance) : TScriptInterface<INotifyFieldValueChanged>();
}

FMVVMViewClass_Source::FParentSourceInfo FMVVMViewClass_Source::GetParentSupplyingSourceClass(UUserWidget* UserWidget) const
{
	// Find an existing instance from a parent source and/or bind to when that paren't source changes
	for (UUserWidget* ContextWidget = UserWidget; ContextWidget; ContextWidget = ContextWidget->GetTypedOuter<UUserWidget>())
	{
		if (const UContextDataWidgetExtension* ContextDataExtension = ContextWidget->GetExtension<UContextDataWidgetExtension>())
		{
			if (UObject* ContextData = ContextDataExtension->FindData(ExpectedSourceType))
			{
				FParentSourceInfo Result;
				Result.ParentUserWidget = ContextWidget;
				Result.Source = ContextData;
				return Result;
			}
		}

		if (ContextWidget != UserWidget)
		{
			if (const UMVVMView* ParentView = UMVVMSubsystem::GetViewFromUserWidget(ContextWidget); ParentView && ParentView->GetViewClass())
			{
				const int32 FoundIndex = ParentView->GetViewClass()->GetSources().IndexOfByPredicate([this](const FMVVMViewClass_Source& Other)
					{
						return Other.IsViewModel() && Other.GetSourceClass()->IsChildOf(ExpectedSourceType) &&
							(GetName().IsNone() ? Other.GetName().IsNone() : GetName() == Other.GetName());
					});

				if (ParentView->GetSources().IsValidIndex(FoundIndex))
				{
					FParentSourceInfo Result;
					Result.ParentView = ParentView;
					Result.ParentSourceKey = FoundIndex;
					Result.ParentUserWidget = ContextWidget;
					Result.Source = ParentView->GetSource(Result.ParentSourceKey).Source;
					return Result;
				}
			}
		}
	}

	return FParentSourceInfo();
}

#if UE_WITH_MVVM_DEBUGGING
FMVVMViewClass_Source::FToStringArgs FMVVMViewClass_Source::FToStringArgs::Short()
{
	FToStringArgs Result;
	Result.bUseDisplayName = false;
	Result.bAddCreationMode = false;
	Result.bAddFlags = false;
	return Result;
}

FMVVMViewClass_Source::FToStringArgs FMVVMViewClass_Source::FToStringArgs::All()
{
	return FToStringArgs();
}

FString FMVVMViewClass_Source::ToString(const UMVVMViewClass* ViewClass, FToStringArgs Args) const
{
	check(ViewClass);

	TStringBuilder<512> StringBuilder;
	StringBuilder << TEXT("Type: ");
#if WITH_EDITOR
	if (Args.bUseDisplayName)
	{
		StringBuilder << ExpectedSourceType->GetDisplayNameText().ToString();
	}
	else
#endif
	{
		StringBuilder << ExpectedSourceType->GetPathName();
	}
	
	StringBuilder << TEXT(", SourceName: ");
	StringBuilder << GetName();

	bool bHasFlag = false;
	auto AddPipe = [&StringBuilder, &bHasFlag]()
		{
			if (bHasFlag)
			{
				StringBuilder << TEXT('|');
			}
			bHasFlag = true;
		};

	if (Args.bAddFields)
	{
		StringBuilder << TEXT("\n    FieldIds: ");
		bHasFlag = false;
		for (const FMVVMViewClass_FieldId& FieldId : GetFieldIds())
		{
			AddPipe();
			StringBuilder << FieldId.GetName();
		}
	}

	if (Args.bAddBindings)
	{
		StringBuilder << TEXT("\n    Bindings: ");
		bHasFlag = false;
		for (const FMVVMViewClass_SourceBinding& SourceBinding : GetBindings())
		{
			StringBuilder << TEXT("\n        FieldId: ");
			StringBuilder << SourceBinding.GetFieldId().GetFieldName();
			StringBuilder << TEXT(" BindingIndex: ");
			StringBuilder << SourceBinding.GetBindingKey().GetIndex();
			StringBuilder << TEXT(" Flags: ");
			if (SourceBinding.ExecuteAtInitialization())
			{
				StringBuilder << TEXT("ExecuteAtInit ");
			}
		}
	}

	if (Args.bAddFlags)
	{
		StringBuilder << TEXT("\n    Flags: ");
		bHasFlag = false;
		if (IsUserWidget())
		{
			AddPipe();
			StringBuilder << TEXT("UserWidget");
		}
		if (IsUserWidgetProperty())
		{
			AddPipe();
			StringBuilder << TEXT("Property");
		}
		if (RequireSettingUserWidgetProperty())
		{
			AddPipe();
			StringBuilder << TEXT("SetProperty");
		}
		if (CanBeSet())
		{
			AddPipe();
			StringBuilder << TEXT("CanBeSet");
		}
		if (CanBeEvaluated())
		{
			AddPipe();
			StringBuilder << TEXT("CanBeEvaluated");
		}
		if (HasEvaluateBindings())
		{
			AddPipe();
			StringBuilder << TEXT("HasEvaluateBindings");
		}
		if (IsOptional())
		{
			AddPipe();
			StringBuilder << TEXT("Optional");
		}
		if (HasTickBindings())
		{
			AddPipe();
			StringBuilder << TEXT("HasTickBindings");
		}
		if (IsViewModel())
		{
			AddPipe();
			StringBuilder << TEXT("IsViewModel");
		}
		if ((Flags & (uint16)EFlags::IsViewModelInstanceExposed) != 0)
		{
			AddPipe();
			StringBuilder << TEXT("EditDefaultOnly");
		}
		if (RequireGlobalViewModelCollectionUpdate())
		{
			AddPipe();
			StringBuilder << TEXT("GlobalCollectionAutoUpdate");
		}
		if (AlwaysExecuteBindingsOnSetSource())
		{
			AddPipe();
			StringBuilder << TEXT("AlwaysExecuteBindingsOnSetSource");
		}
	}

	if (Args.bAddCreationMode)
	{
		StringBuilder << TEXT("\n    CreationType: ");
		if ((Flags & (uint16)EFlags::TypeCreateInstance) != 0)
		{
			StringBuilder << TEXT("CreateInstance");
			if ((Flags & (uint16)EFlags::IsViewModelInstanceExposed) != 0)
			{
				StringBuilder << TEXT(", InstanceExposed");
			}
		}
		else if (Resolver)
		{
			StringBuilder << Resolver->GetClass()->GetFullName();
		}
		else if (GlobalViewModelInstance.IsValid())
		{
			StringBuilder << TEXT("GlobalCollection={");
			if (GlobalViewModelInstance.ContextClass)
			{
				StringBuilder << GlobalViewModelInstance.ContextClass->GetPathName();
			}
			else
			{
				StringBuilder << TEXT("<Invalid>");
			}
			StringBuilder << TEXT(", ");
			StringBuilder << GlobalViewModelInstance.ContextName;
			StringBuilder << TEXT("}");
		}
		else if (FieldPath.IsValid())
		{
			StringBuilder << TEXT("Path=");
			TValueOrError<FString, FString> FieldPathString = ViewClass->GetBindingLibrary().FieldPathToString(FieldPath, Args.bUseDisplayName);
			if (FieldPathString.HasValue())
			{
				StringBuilder << FieldPathString.GetValue();
			}
			else
			{
				StringBuilder << TEXT("<Error>");
				StringBuilder << FieldPathString.GetError();
			}
		}
		else
		{
			StringBuilder << TEXT("None"); // The setting is Manual. At runtime, do nothing.
		}
	}

	return StringBuilder.ToString();
}
#endif

///////////////////////////////////////////////////////////////////////
// 
///////////////////////////////////////////////////////////////////////
namespace UE::MVVM::Private
{
	FAutoConsoleVariable CVarDefaultEvaluationMode(
		TEXT("MVVM.DefaultExecutionMode"),
		static_cast<uint8>(EMVVMExecutionMode::DelayedWhenSharedElseImmediate),
		TEXT("The default evaluation mode of a MVVM binding. Must restart if changed."),
		EConsoleVariableFlags::ECVF_ReadOnly
	);
}

EMVVMExecutionMode FMVVMViewClass_Binding::GetExecuteMode() const
{
	struct FLocal
	{
		FLocal()
		{
			IConsoleVariable* CVarDefaultExecutionMode = IConsoleManager::Get().FindConsoleVariable(TEXT("MVVM.DefaultExecutionMode"));
			if (ensure(CVarDefaultExecutionMode))
			{
				DefaultMode = (EMVVMExecutionMode)CVarDefaultExecutionMode->GetInt();
			}
		}
		EMVVMExecutionMode DefaultMode = EMVVMExecutionMode::DelayedWhenSharedElseImmediate;
	};
	static FLocal Local;

	EMVVMExecutionMode Result = (Flags & (uint8)EFlags::OverrideExecuteMode) == 0 ? Local.DefaultMode : ExecutionMode;
	return Result == EMVVMExecutionMode::DelayedWhenSharedElseImmediate ? (IsShared() ? EMVVMExecutionMode::Delayed : EMVVMExecutionMode::Immediate) : Result;
}

#if UE_WITH_MVVM_DEBUGGING
FMVVMViewClass_Binding::FToStringArgs FMVVMViewClass_Binding::FToStringArgs::Short()
{
	FToStringArgs Result;
	Result.bUseDisplayName = false;
	Result.bAddFlags = false;
	Result.bAddBindingId = false;
	Result.bAddBindingFields = true;
	Result.bAddSources = false;
	return Result;
}

FMVVMViewClass_Binding::FToStringArgs FMVVMViewClass_Binding::FToStringArgs::All()
{
	return FToStringArgs();
}

FString FMVVMViewClass_Binding::ToString(const UMVVMViewClass* ViewClass, FToStringArgs Args) const
{
	TStringBuilder<1024> StringBuilder;

	bool bAddNewLine = false;
	auto AddNewLine = [&StringBuilder, &bAddNewLine]()
		{
			if (bAddNewLine)
			{
				StringBuilder << TEXT("\n    ");
			}
			bAddNewLine = true;
		};

#if WITH_EDITOR
	if (Args.bAddBindingId)
	{
		AddNewLine();
		StringBuilder << TEXT("BindingId: ");
		EditorId.AppendString(StringBuilder);
	}
#endif

	if (Args.bAddBindingFields)
	{
		if (GetBinding().IsValid())
		{
			AddNewLine();
			StringBuilder << TEXT("Binding: ");

			FString SourceString;
			FString DestinationString;
			FString ConversionString;
			bool bErrorStrings = false;
			if (GetBinding().GetSourceFieldPath().IsValid())
			{
				TValueOrError<FString, FString> SourceFieldPathString = ViewClass->GetBindingLibrary().FieldPathToString(GetBinding().GetSourceFieldPath(), Args.bUseDisplayName);
				SourceString = SourceFieldPathString.HasValue() ? SourceFieldPathString.StealValue() : SourceFieldPathString.StealError();
				bErrorStrings = bErrorStrings || SourceFieldPathString.HasError();
			}
			{
				TValueOrError<FString, FString> DestinationFieldPathString = ViewClass->GetBindingLibrary().FieldPathToString(GetBinding().GetDestinationFieldPath(), Args.bUseDisplayName);
				DestinationString = DestinationFieldPathString.HasValue() ? DestinationFieldPathString.StealValue() : DestinationFieldPathString.StealError();
				bErrorStrings = bErrorStrings || DestinationFieldPathString.HasError();
			}
			if (GetBinding().GetConversionFunctionFieldPath().IsValid())
			{
				TValueOrError<FString, FString> ConversionFieldPathString = ViewClass->GetBindingLibrary().FieldPathToString(GetBinding().GetConversionFunctionFieldPath(), Args.bUseDisplayName);
				ConversionString = ConversionFieldPathString.HasValue() ? ConversionFieldPathString.StealValue() : ConversionFieldPathString.StealError();
				bErrorStrings = bErrorStrings || ConversionFieldPathString.HasError();
			}

			if (bErrorStrings)
			{
				StringBuilder << TEXT("Error: ");
			}

			if (Args.bUseDisplayName)
			{
				StringBuilder << TEXT('"');
			}
			StringBuilder << DestinationString;
			if (Args.bUseDisplayName)
			{
				StringBuilder << TEXT('"');
			}

			if (GetBinding().IsRuntimeBinding())
			{
				StringBuilder << TEXT(" = ");
				if (GetBinding().GetConversionFunctionFieldPath().IsValid())
				{
					if (Args.bUseDisplayName)
					{
						StringBuilder << TEXT('"');
					}
					StringBuilder << ConversionString;
					if (Args.bUseDisplayName)
					{
						StringBuilder << TEXT('"');
					}
					StringBuilder << TEXT(" ( ");
				}

				if (!GetBinding().HasComplexConversionFunction())
				{
					if (Args.bUseDisplayName)
					{
						StringBuilder << TEXT('"');
					}
					StringBuilder << SourceString;
					if (Args.bUseDisplayName)
					{
						StringBuilder << TEXT('"');
					}
				}

				if (GetBinding().GetConversionFunctionFieldPath().IsValid())
				{
					StringBuilder << TEXT(" )");
				}
			}
			else
			{
				StringBuilder << TEXT("()");
			}
		}
		else
		{
			StringBuilder << TEXT("!!Invalid Binding Error!!");
		}
	}

	if (Args.bAddSources)
	{
		AddNewLine();
		StringBuilder << TEXT("Sources: 0x");
		StringBuilder << FString::Printf(TEXT("%" UINT64_x_FMT), SourceBitField);
	}

	if (Args.bAddFlags)
	{
		AddNewLine();
		if ((Flags & (uint8)EFlags::OverrideExecuteMode) != 0)
		{
			StringBuilder << TEXT(", Mode: ");
			StringBuilder << StaticEnum<EMVVMExecutionMode>()->GetNameByValue((int64)ExecutionMode);
		}

		StringBuilder << TEXT(", Flags: ");

		bool bAddPipe = false;
		auto AddPipe = [&StringBuilder, &bAddPipe]()
			{
				if (bAddPipe)
				{
					StringBuilder << TEXT('|');
				}
				bAddPipe = true;
			};

		if ((Flags & (uint8)EFlags::OneWay) != 0)
		{
			AddPipe();
			StringBuilder << TEXT("OneWay");
		}
		if ((Flags & (uint8)EFlags::Shared) != 0)
		{
			AddPipe();
			StringBuilder << TEXT("Shared");
		}
		if ((Flags & (uint8)EFlags::OverrideExecuteMode) != 0)
		{
			AddPipe();
			StringBuilder << TEXT("OverrideExecuteMode");
		}
		if ((Flags & (uint8)EFlags::EnabledByDefault) != 0)
		{
			AddPipe();
			StringBuilder << TEXT("Enabled");
		}

		if (GetBinding().IsComplexBinding())
		{
			AddPipe();
			StringBuilder << TEXT("Complex");
		}
		else if (GetBinding().HasComplexConversionFunction())
		{
			AddPipe();
			StringBuilder << TEXT("ComplexConversion");
		}
		else if (GetBinding().HasSimpleConversionFunction())
		{
			AddPipe();
			StringBuilder << TEXT("SimpleConversion");
		}
	}

	return StringBuilder.ToString();
}
#endif

///////////////////////////////////////////////////////////////////////
// 
///////////////////////////////////////////////////////////////////////
#if UE_WITH_MVVM_DEBUGGING
FMVVMViewClass_EvaluateSource::FToStringArgs FMVVMViewClass_EvaluateSource::FToStringArgs::Short()
{
	FToStringArgs Result;
	Result.bUseDisplayName = false;
	return Result;
}

FMVVMViewClass_EvaluateSource::FToStringArgs FMVVMViewClass_EvaluateSource::FToStringArgs::All()
{
	return FToStringArgs();
}

FString FMVVMViewClass_EvaluateSource::ToString(const UMVVMViewClass* ViewClass, FToStringArgs Args) const
{
	TStringBuilder<1024> StringBuilder;

	if (ParentSource.IsValid())
	{
		StringBuilder << TEXT("Parent: ");
		StringBuilder << ViewClass->GetSource(ParentSource).GetName();
	}

	if (ParentFieldId.IsValid())
	{
		StringBuilder << TEXT("FieldId: ");
		StringBuilder << ParentFieldId.GetFieldName();
	}

	if (ToEvaluate.IsValid())
	{
		StringBuilder << TEXT("Source: ");
		StringBuilder << ViewClass->GetSource(ToEvaluate).GetName();
	}

	return StringBuilder.ToString();
}
#endif

///////////////////////////////////////////////////////////////////////
// 
///////////////////////////////////////////////////////////////////////
#if UE_WITH_MVVM_DEBUGGING
FMVVMViewClass_Event::FToStringArgs FMVVMViewClass_Event::FToStringArgs::Short()
{
	FToStringArgs Result;
	Result.bUseDisplayName = false;
	return Result;
}

FMVVMViewClass_Event::FToStringArgs FMVVMViewClass_Event::FToStringArgs::All()
{
	FToStringArgs Result;
	Result.bUseDisplayName = true;
	return Result;
}

FString FMVVMViewClass_Event::ToString(const UMVVMViewClass* ViewClass, FToStringArgs Args) const
{
	TStringBuilder<1024> StringBuilder;

	if (GetMulticastDelegatePath().IsValid())
	{
		StringBuilder << TEXT("Event: ");

		if (Args.bUseDisplayName)
		{
			StringBuilder << TEXT('"');
		}

		TValueOrError<FString, FString> SourceFieldPathString = ViewClass->GetBindingLibrary().FieldPathToString(GetMulticastDelegatePath(), Args.bUseDisplayName);
		StringBuilder << (SourceFieldPathString.HasValue() ? SourceFieldPathString.StealValue() : SourceFieldPathString.StealError());

		if (Args.bUseDisplayName)
		{
			StringBuilder << TEXT("\"\n");
		}
	}

	if (SourceToReevaluate.IsValid())
	{
		StringBuilder << TEXT("SourceName: ");
		StringBuilder << ViewClass->GetSource(SourceToReevaluate).GetName();
	}

	StringBuilder << TEXT(", Function: ");
	StringBuilder << GetUserWidgetFunctionName();

	return StringBuilder.ToString();
}
#endif

///////////////////////////////////////////////////////////////////////
// 
///////////////////////////////////////////////////////////////////////

void UMVVMViewClass::Initialize(UUserWidget* UserWidget)
{
	ensure(UserWidget->GetExtension<UMVVMView>() == nullptr);
	UMVVMView* View = UserWidget->AddExtension<UMVVMView>();
	if (ensure(View))
	{
		View->SetFlags(RF_Transient);
		View->ConstructView(this);
	}
}

void UMVVMViewClass::Construct(UUserWidget* UserWidget)
{
	++ViewCounter;
	ensure(ViewCounter <= std::numeric_limits<int32>::max());

	if (ViewCounter == 1)
	{
		BindingLibrary.Load();

		// Fixup the FieldIds
		for (const FMVVMViewClass_Source& Source : Sources)
		{
			const TArrayView<const FMVVMViewClass_FieldId> SourceFieldIds = Source.GetFieldIds();
			if (SourceFieldIds.Num() > 0)
			{
				const UClass* SourceClass = CastChecked<UClass>(Source.GetSourceClass());
				check(SourceClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()));
				TScriptInterface<INotifyFieldValueChanged> Interface = SourceClass->GetDefaultObject();

				for (const FMVVMViewClass_FieldId& SourceFieldId : SourceFieldIds)
				{
					UE::FieldNotification::FFieldId FieldId = Interface->GetFieldNotificationDescriptor().GetField(SourceClass, SourceFieldId.GetName());
					const_cast<FMVVMViewClass_FieldId&>(SourceFieldId).FieldIndex = FieldId.GetIndex();
				}
			}
		}
	}

#if WITH_EDITOR
	if (GEditor)
	{
		FModelViewViewModelModule& Module = FModuleManager::GetModuleChecked<FModelViewViewModelModule>("ModelViewViewModel");
		Module.OnBlueprintCompiled.AddUObject(this, &UMVVMViewClass::HandleBlueprintCompiled);
	}
#endif
}


void UMVVMViewClass::Destruct(UUserWidget* UserWidget)
{
#if WITH_EDITOR
	if (UserWidget)
	{
		bool bIsAnyViewFromThisClass = false;
		for (UUserWidgetExtension* Extension : UserWidget->GetExtensions(UMVVMView::StaticClass()))
		{
			if (UMVVMView* View = Cast<UMVVMView>(Extension))
			{
				if (View->GetViewClass() == this)
				{
					bIsAnyViewFromThisClass = true;
					break;
				}
			}
		}

		if (!bIsAnyViewFromThisClass)
		{
			return;
		}
	}
#endif

	--ViewCounter;
	ensure(ViewCounter >= 0);
	if (ViewCounter == 0)
	{
		BindingLibrary.Unload();
	}

#if WITH_EDITOR
	if (GEditor && BlueprintCompiledHandle.IsValid())
	{
		FModelViewViewModelModule& Module = FModuleManager::GetModuleChecked<FModelViewViewModelModule>("ModelViewViewModel");
		Module.OnBlueprintCompiled.Remove(BlueprintCompiledHandle);
	}
#endif
}

#if WITH_EDITOR
void UMVVMViewClass::BeginDestroy()
{
	if (GEditor && BlueprintCompiledHandle.IsValid())
	{
		GEditor->OnBlueprintCompiled().Remove(BlueprintCompiledHandle);
		BlueprintCompiledHandle.Reset();
	}
	Super::BeginDestroy();
}

void UMVVMViewClass::HandleBlueprintCompiled()
{
	// do not keep property alive of potential dead BP class.
	if (ViewCounter > 0)
	{
		BindingLibraryDirty.store(true, std::memory_order_relaxed);
	}
}

void UMVVMViewClass::ReloadIfDirty() const
{
	if (BindingLibraryDirty.load(std::memory_order_relaxed))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UMVVMViewClass::ReloadIfDirty);
		UMVVMViewClass* MutableThis = const_cast<UMVVMViewClass*>(this);
		MutableThis->BindingLibraryDirty.store(false, std::memory_order_relaxed);
		MutableThis->BindingLibrary.Unload();
		MutableThis->BindingLibrary.Load();
	}
}

#endif

#if UE_WITH_MVVM_DEBUGGING
UMVVMViewClass::FToStringArgs UMVVMViewClass::FToStringArgs::Short()
{
	FToStringArgs Result;
	Result.Source = FMVVMViewClass_Source::FToStringArgs::Short();
	Result.Binding = FMVVMViewClass_Binding::FToStringArgs::Short();
	Result.Event = FMVVMViewClass_Event::FToStringArgs::Short();
	return Result;
}

UMVVMViewClass::FToStringArgs UMVVMViewClass::FToStringArgs::All()
{
	FToStringArgs Result;
	Result.Source = FMVVMViewClass_Source::FToStringArgs::All();
	Result.Binding = FMVVMViewClass_Binding::FToStringArgs::All();
	Result.Event = FMVVMViewClass_Event::FToStringArgs::All();
	return Result;
}

FString UMVVMViewClass::ToString(FToStringArgs Args) const
{
	bool bPreviousLoad = ViewCounter > 0;
	if (!bPreviousLoad)
	{
		const_cast<UMVVMViewClass*>(this)->BindingLibrary.Load();
	}

	TStringBuilder<2048> Builder;
	Builder << TEXT("Sources: ");
	for (int32 Index = 0; Index < GetSources().Num(); ++Index)
	{
		const FMVVMViewClass_Source& Source = GetSources()[Index];
		Builder << TEXT("\n");
		Builder << TEXT('(') << Index << TEXT(')');
		Builder << Source.ToString(this, Args.Source);
	}

	Builder << TEXT("\nBindings: ");
	Builder << TEXT("Sources: ");
	for (int32 Index = 0; Index < GetBindings().Num(); ++Index)
	{
		const FMVVMViewClass_Binding& Binding = GetBindings()[Index];
		Builder << TEXT("\n");
		Builder << TEXT('(') << Index << TEXT(')');
		Builder << Binding.ToString(this, Args.Binding);
	}

	Builder << TEXT("\nEvaluates: ");
	for (int32 Index = 0; Index < GetEvaluateSources().Num(); ++Index)
	{
		const FMVVMViewClass_EvaluateSource& EvaluateSource = GetEvaluateSources()[Index];
		Builder << TEXT("\n");
		Builder << TEXT('(') << Index << TEXT(')');
		Builder << EvaluateSource.ToString(this, Args.Evaluate);
	}

	Builder << TEXT("\nEvents: ");
	for (int32 Index = 0; Index < GetEvents().Num(); ++Index)
	{
		const FMVVMViewClass_Event& Event = GetEvents()[Index];
		Builder << TEXT("\n");
		Builder << TEXT('(') << Index << TEXT(')');
		Builder << Event.ToString(this, Args.Event);
	}

	if (!bPreviousLoad)
	{
#if WITH_EDITOR
		const_cast<UMVVMViewClass*>(this)->BindingLibrary.Unload();
#endif
	}

	return Builder.ToString();
}
#endif

#undef LOCTEXT_NAMESPACE

