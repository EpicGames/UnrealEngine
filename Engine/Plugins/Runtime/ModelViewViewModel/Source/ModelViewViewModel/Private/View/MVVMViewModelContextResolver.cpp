// Copyright Epic Games, Inc. All Rights Reserved.

#include "View/MVVMViewModelContextResolver.h"

#if WITH_EDITOR
#include "ClassViewerFilter.h" //ClassViewer
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMViewModelContextResolver)

#if WITH_EDITOR
bool UMVVMViewModelContextResolver::DoesSupportViewModelClass(const UClass* Class) const
{
	if (Class == nullptr)
	{
		return false;
	}
	
	auto ContainsByPredicate = [Class](const TArray<FSoftClassPath>& List)
		{
			return List.ContainsByPredicate(
				[Class](const FSoftClassPath& Other)
				{
					const UClass* OtherClass = Other.ResolveClass();
					return OtherClass ? Class->IsChildOf(OtherClass) : false;
				});
		};
	
	if (AllowedViewModelClasses.Num() > 0)
	{
		bool bContains = ContainsByPredicate(AllowedViewModelClasses);		
		if (!bContains)
		{
			return false;
		}
	}
	
	if (DeniedViewModelClasses.Num() > 0)
	{
		bool bContains = ContainsByPredicate(DeniedViewModelClasses);		
		if (bContains)
		{
			return false;
		}
	}

	return true;
}
#endif
