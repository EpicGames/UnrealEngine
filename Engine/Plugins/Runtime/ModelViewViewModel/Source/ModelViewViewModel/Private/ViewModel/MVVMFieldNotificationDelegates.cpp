// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModel/MVVMFieldNotificationDelegates.h"


#define LOCTEXT_NAMESPACE "MVVMFieldNotificationDelegates"

namespace UE::MVVM {

FDelegateHandle FMVVMFieldNotificationDelegates::AddFieldValueChangedDelegate(UObject* InOwner, UE::FieldNotification::FFieldId InFieldId, INotifyFieldValueChanged::FFieldValueChangedDelegate InNewDelegate)
{
	FDelegateHandle Result;
	if (InFieldId.IsValid())
	{
		Result = Delegates.Add(InOwner, InFieldId, MoveTemp(InNewDelegate));
		if (Result.IsValid())
		{
			EnabledFieldNotifications.PadToNum(InFieldId.GetIndex() + 1, false);
			EnabledFieldNotifications[InFieldId.GetIndex()] = true;
		}
	}
	return Result;
}


void FMVVMFieldNotificationDelegates::AddFieldValueChangedDelegate(UObject* InOwner, const UE::FieldNotification::FFieldId InFieldId, const FFieldValueChangedDynamicDelegate& InDelegate)
{
	if (InFieldId.IsValid())
	{
		FDelegateHandle Result = Delegates.Add(InOwner, InFieldId, InDelegate);
		if (Result.IsValid())
		{
			EnabledFieldNotifications.PadToNum(InFieldId.GetIndex() + 1, false);
			EnabledFieldNotifications[InFieldId.GetIndex()] = true;
		}
	}
}


bool FMVVMFieldNotificationDelegates::RemoveFieldValueChangedDelegate(UObject* InOwner, UE::FieldNotification::FFieldId InFieldId, FDelegateHandle InHandle)
{
	bool bResult = false;
	if (InFieldId.IsValid() && InHandle.IsValid() && EnabledFieldNotifications.IsValidIndex(InFieldId.GetIndex()) && EnabledFieldNotifications[InFieldId.GetIndex()])
	{
		UE::FieldNotification::FFieldMulticastDelegate::FRemoveFromResult RemoveResult = Delegates.RemoveFrom(InOwner, InFieldId, InHandle);
		bResult = RemoveResult.bRemoved;
		EnabledFieldNotifications[InFieldId.GetIndex()] = RemoveResult.bHasOtherBoundDelegates;
	}
	return bResult;
}


bool FMVVMFieldNotificationDelegates::RemoveFieldValueChangedDelegate(UObject* InOwner, const UE::FieldNotification::FFieldId InFieldId, const FFieldValueChangedDynamicDelegate& InDelegate)
{
	bool bResult = false;
	if (InFieldId.IsValid() && EnabledFieldNotifications.IsValidIndex(InFieldId.GetIndex()) && EnabledFieldNotifications[InFieldId.GetIndex()])
	{
		UE::FieldNotification::FFieldMulticastDelegate::FRemoveFromResult RemoveResult = Delegates.RemoveFrom(InOwner, InFieldId, InDelegate);
		bResult = RemoveResult.bRemoved;
		EnabledFieldNotifications[InFieldId.GetIndex()] = RemoveResult.bHasOtherBoundDelegates;
	}
	return bResult;
}


int32 FMVVMFieldNotificationDelegates::RemoveAllFieldValueChangedDelegates(UObject* InOwner, FDelegateUserObjectConst InUserObject)
{
	int32 bResult = 0;
	if (InUserObject)
	{
		UE::FieldNotification::FFieldMulticastDelegate::FRemoveAllResult RemoveResult = Delegates.RemoveAll(InOwner, InUserObject);
		bResult = RemoveResult.RemoveCount;
		EnabledFieldNotifications = RemoveResult.HasFields;
	}
	return bResult;
}


int32 FMVVMFieldNotificationDelegates::RemoveAllFieldValueChangedDelegates(UObject* InOwner, UE::FieldNotification::FFieldId InFieldId, FDelegateUserObjectConst InUserObject)
{
	int32 bResult = 0;
	if (InFieldId.IsValid() && InUserObject)
	{
		UE::FieldNotification::FFieldMulticastDelegate::FRemoveAllResult RemoveResult = Delegates.RemoveAll(InOwner, InFieldId, InUserObject);
		bResult = RemoveResult.RemoveCount;
		EnabledFieldNotifications = RemoveResult.HasFields;
	}
	return bResult;
}


void FMVVMFieldNotificationDelegates::BroadcastFieldValueChanged(UObject* InOwner, UE::FieldNotification::FFieldId InFieldId)
{
	if (InFieldId.IsValid() && EnabledFieldNotifications.IsValidIndex(InFieldId.GetIndex()) && EnabledFieldNotifications[InFieldId.GetIndex()])
	{
		Delegates.Broadcast(InOwner, InFieldId);
	}
}

TArray<UE::FieldNotification::FFieldMulticastDelegate::FDelegateView> FMVVMFieldNotificationDelegates::GetView() const
{
	return Delegates.GetView();
}

} //namespace

#undef LOCTEXT_NAMESPACE

