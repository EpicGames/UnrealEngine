// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModel/MVVMInstancedViewModelGeneratedClass.h"

#include "INotifyFieldValueChanged.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMInstancedViewModelGeneratedClass)

#if WITH_EDITOR
UClass* UMVVMInstancedViewModelGeneratedClass::GetAuthoritativeClass()
{
	return this;
}

void UMVVMInstancedViewModelGeneratedClass::PurgeClass(bool bRecompilingOnLoad)
{
	Super::PurgeClass(bRecompilingOnLoad);
	PurgeNativeRepNotifyFunctions();
}

void UMVVMInstancedViewModelGeneratedClass::AddNativeRepNotifyFunction(UFunction* Function, const FProperty* Property)
{
	if (Property && Function && Property->RepNotifyFunc == Function->GetFName())
	{
		OnRepFunctionToLink.Add(Function);
		OnRepToPropertyMap.Add(Function, Property);
	}
}

void UMVVMInstancedViewModelGeneratedClass::PurgeNativeRepNotifyFunctions()
{
	OnRepFunctionToLink.Empty();
	OnRepToPropertyMap.Empty();
}
#endif

void UMVVMInstancedViewModelGeneratedClass::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
	Super::Link(Ar, bRelinkExistingProperties);

	OnRepFunctionToLink.RemoveAllSwap([](UFunction* Other){ return Other == nullptr; }, EAllowShrinking::Yes);

	for (UFunction* OnRep : OnRepFunctionToLink)
	{
		NativeFunctionLookupTable.Emplace(OnRep->GetFName(), &UMVVMInstancedViewModelGeneratedClass::K2_CallNativeOnRep);
	}

	OnRepToPropertyMap.Empty();
	for (TFieldIterator<FProperty> PropertyIter(this, EFieldIteratorFlags::ExcludeSuper); PropertyIter; ++PropertyIter)
	{
		if (!PropertyIter->RepNotifyFunc.IsNone())
		{
			TObjectPtr<UFunction>* FoundFunction = OnRepFunctionToLink.FindByPredicate([FunctionName = PropertyIter->RepNotifyFunc](const UFunction* Other){ return Other->GetFName() == FunctionName; });
			if (FoundFunction)
			{
				OnRepToPropertyMap.Add(FoundFunction->Get(), *PropertyIter);
			}
		}
	}
}

DEFINE_FUNCTION(UMVVMInstancedViewModelGeneratedClass::K2_CallNativeOnRep)
{
	UObject* CallingObject = P_THIS_OBJECT;

	P_NATIVE_BEGIN;

	if (CallingObject == nullptr || !CallingObject->GetClass()->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
	{
		return;
	}
	UMVVMInstancedViewModelGeneratedClass* GeneratedClass = Cast<UMVVMInstancedViewModelGeneratedClass>(CallingObject->GetClass());
	if (!ensure(GeneratedClass))
	{
		return;
	}

	FName PropertyName;
	{
		const UFunction* Function = Stack.CurrentNativeFunction;
		const FProperty* const* FoundProperty = GeneratedClass->OnRepToPropertyMap.Find(TObjectKey<UFunction>(Function));
		if (FoundProperty)
		{
			GeneratedClass->OnPropertyReplicated(CallingObject, *FoundProperty);
		}
	}

	P_NATIVE_END;
}

void UMVVMInstancedViewModelGeneratedClass::BroadcastFieldValueChanged(UObject* Object, const FProperty* Property)
{
	TScriptInterface<INotifyFieldValueChanged> CallingInterface = Object;
	UE::FieldNotification::FFieldId FieldId = CallingInterface->GetFieldNotificationDescriptor().GetField(Object->GetClass(), Property->GetFName());
	if (FieldId.IsValid())
	{
		CallingInterface->BroadcastFieldValueChanged(FieldId);
	}
}

void UMVVMInstancedViewModelGeneratedClass::OnPropertyReplicated(UObject* Object, const FProperty* Property)
{
	BroadcastFieldValueChanged(Object, Property);
}