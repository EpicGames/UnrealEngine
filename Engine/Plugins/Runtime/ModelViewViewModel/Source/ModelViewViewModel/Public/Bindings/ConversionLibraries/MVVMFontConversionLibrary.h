// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SlateWrapperTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Fonts/SlateFontInfo.h"

#include "MVVMFontConversionLibrary.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

class UMaterialInstanceDynamic;
class UMaterialInterface;
class UTexture;

/**
 * Conversion library that contains methods for FSlateFontInfo
 * 
 * Primarily consists of methods to set material params on an existing FSlateFontInfo
 */
UCLASS(MinimalAPI)
class UMVVMFontConversionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	
	/**
	 * Sets a scalar value on a font material assuming it exists, handles MID existance appropriately
	 * 
	 * @param Font			Font with a material we want to set a parameter on 
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Scalar Parameter (Font)", MVVMBindToDestination = "Font"))
	static UE_API FSlateFontInfo Conv_SetScalarParameter(FSlateFontInfo Font, FName ParameterName, float Value);
	
	/**
	 * Sets a vector value on a font material assuming it exists, handles MID existance appropriately
	 * 
	 * @param Font			Font with a material we want to set a parameter on 
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Vector Parameter (Font)", MVVMBindToDestination = "Font"))
	static UE_API FSlateFontInfo Conv_SetVectorParameter(FSlateFontInfo Font, FName ParameterName, FLinearColor Value);

	/**
	 * Sets a vector value on a font material assuming it exists, handles MID existance appropriately
	 * 
	 * @param Font			Font with a material we want to set a parameter on 
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Vector Parameter (Font) from Color", MVVMBindToDestination = "Font"))
	static UE_API FSlateFontInfo Conv_SetVectorParameter_FColor(FSlateFontInfo Font, FName ParameterName, FColor Value);
	
	/**
	 * Sets a texture value on a font material assuming it exists, handles MID existance appropriately
	 * 
	 * @param Font			Font with a material we want to set a parameter on 
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Texture Parameter (Font)", MVVMBindToDestination = "Font"))
	static UE_API FSlateFontInfo Conv_SetTextureParameter(FSlateFontInfo Font, FName ParameterName, UTexture* Value);

	/**
	 * Sets a scalar value on a font material assuming it exists, handles MID existance appropriately
	 *
	 * @param Font			Font with a material we want to set a parameter on
	 * @param Material		Material to set on the font
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Scalar Parameter MID (Font)", MVVMBindToDestination = "Font"))
	static UE_API FSlateFontInfo Conv_SetScalarParameterMID(FSlateFontInfo Font, UMaterialInterface* Material, FName ParameterName, float Value);

	/**
	 * Sets a vector value on a font material assuming it exists, handles MID existance appropriately
	 *
	 * @param Font			Font with a material we want to set a parameter on
	 * @param Material		Material to set on the font
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Vector Parameter MID (Font)", MVVMBindToDestination = "Font"))
	static UE_API FSlateFontInfo Conv_SetVectorParameterMID(FSlateFontInfo Font, UMaterialInterface* Material, FName ParameterName, FLinearColor Value);

	/**
	 * Sets a vector value on a font material assuming it exists, handles MID existance appropriately
	 *
	 * @param Font			Font with a material we want to set a parameter on
	 * @param Material		Material to set on the font
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Vector Parameter MID (Font) from Color", MVVMBindToDestination = "Font"))
	static UE_API FSlateFontInfo Conv_SetVectorParameterMID_FColor(FSlateFontInfo Font, UMaterialInterface* Material, FName ParameterName, FColor Value);

	/**
	 * Sets a texture value on a font material assuming it exists, handles MID existance appropriately
	 *
	 * @param Font			Font with a material we want to set a parameter on
	 * @param Material		Material to set on the font
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Texture Parameter MID (Font)", MVVMBindToDestination = "Font"))
	static UE_API FSlateFontInfo Conv_SetTextureParameterMID(FSlateFontInfo Font, UMaterialInterface* Material, FName ParameterName, UTexture* Value);

private:

	/**
	 * Tries to get a Material Instance Dynamic from the font, will create one if possible using the provided outer, else returns nullptr
	 */
	static UMaterialInstanceDynamic* TryGetDynamicMaterial(FSlateFontInfo& InFont, UObject* InOuter, UMaterialInterface* InMaterial = nullptr);
};

#undef UE_API
