// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SlateWrapperTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Styling/SlateBrush.h"

#include "MVVMSlateBrushConversionLibrary.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

class UMaterialInstanceDynamic;
class UMaterialInterface;
class UTexture;

/**
 * Conversion library that contains methods for FSlateBrush
 * 
 * Primarily consists of methods to set material params on an existing FSlateBrush
 */
UCLASS(MinimalAPI)
class UMVVMSlateBrushConversionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	
	/**
	 * Sets a scalar value on a brush material assuming it exists, handles MID existance appropriately
	 * 
	 * @param Brush			Brush with a material we want to set a parameter on 
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Scalar Parameter (Brush)", MVVMBindToDestination = "Brush"))
	static UE_API FSlateBrush Conv_SetScalarParameter(FSlateBrush Brush, FName ParameterName, float Value);
	
	/**
	 * Sets a vector value on a brush material assuming it exists, handles MID existance appropriately
	 * 
	 * @param Brush			Brush with a material we want to set a parameter on 
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Vector Parameter (Brush)", MVVMBindToDestination = "Brush"))
	static UE_API FSlateBrush Conv_SetVectorParameter(FSlateBrush Brush, FName ParameterName, FLinearColor Value);
	
	/**
     * Sets a vector value on a brush material assuming it exists, handles MID existance appropriately
     * 
     * @param Brush			Brush with a material we want to set a parameter on 
     * @param ParameterName	Name of material parameter to set
     * @param Value			Value to set material parameter to
     */
    UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Vector Parameter (Brush) from Color", MVVMBindToDestination = "Brush"))
	static UE_API FSlateBrush Conv_SetVectorParameter_FColor(FSlateBrush Brush, FName ParameterName, FColor Value);
    	
    	
	/**
	 * Sets a texture value on a brush material assuming it exists, handles MID existance appropriately
	 * 
	 * @param Brush			Brush with a material we want to set a parameter on 
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Texture Parameter (Brush)", MVVMBindToDestination = "Brush"))
	static UE_API FSlateBrush Conv_SetTextureParameter(FSlateBrush Brush, FName ParameterName, UTexture* Value);

	/**
	 * Sets a scalar value on a brush material assuming it exists, handles MID existance appropriately
	 *
	 * @param Brush			Brush with a material we want to set a parameter on
	 * @param Material		Material to set on the brush
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Scalar Parameter MID (Brush)", MVVMBindToDestination = "Brush"))
	static UE_API FSlateBrush Conv_SetScalarParameterMID(FSlateBrush Brush, UMaterialInterface* Material, FName ParameterName, float Value);

	/**
	 * Sets a vector value on a brush material assuming it exists, handles MID existance appropriately
	 *
	 * @param Brush			Brush with a material we want to set a parameter on
	 * @param Material		Material to set on the brush
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Vector Parameter MID (Brush)", MVVMBindToDestination = "Brush"))
	static UE_API FSlateBrush Conv_SetVectorParameterMID(FSlateBrush Brush, UMaterialInterface* Material, FName ParameterName, FLinearColor Value);
	
	/**
	* Sets a vector value on a brush material assuming it exists, handles MID existance appropriately
	*
	* @param Brush			Brush with a material we want to set a parameter on
	* @param Material		Material to set on the brush
	* @param ParameterName	Name of material parameter to set
	* @param Value			Value to set material parameter to
	*/
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Vector Parameter MID (Brush) from Color", MVVMBindToDestination = "Brush"))
	static UE_API FSlateBrush Conv_SetVectorParameterMID_FColor(FSlateBrush Brush, UMaterialInterface* Material, FName ParameterName, FColor Value);


	/**
	 * Sets a texture value on a brush material assuming it exists, handles MID existance appropriately
	 *
	 * @param Brush			Brush with a material we want to set a parameter on
	 * @param Material		Material to set on the brush
	 * @param ParameterName	Name of material parameter to set
	 * @param Value			Value to set material parameter to
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "Set Texture Parameter MID (Brush)", MVVMBindToDestination = "Brush"))
	static UE_API FSlateBrush Conv_SetTextureParameterMID(FSlateBrush Brush, UMaterialInterface* Material, FName ParameterName, UTexture* Value);

private:

	/**
	 * Tries to get a Material Instance Dynamic from the brush, will create one if possible using the provided outer, else returns nullptr
	 */
	static UMaterialInstanceDynamic* TryGetDynamicMaterial(FSlateBrush& InBrush, UObject* InOuter, UMaterialInterface* InMaterial = nullptr);
};

#undef UE_API
