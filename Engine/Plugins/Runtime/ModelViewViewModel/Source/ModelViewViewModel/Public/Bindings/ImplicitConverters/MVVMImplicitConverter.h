// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FProperty;

/**
 * Base class to support converting between simple types in MVVM bindings without the need for a conversion function
 */
class FMVVMImplicitConverter
{
public:
	FMVVMImplicitConverter() = default;
	virtual ~FMVVMImplicitConverter() = default;
	
	virtual bool CanConvertImplicitly(const FProperty* Source, const FProperty* Destination) = 0;
	virtual void ConvertImplicitly(const FProperty* Source, const FProperty* Destination, void* Data) = 0;
};
