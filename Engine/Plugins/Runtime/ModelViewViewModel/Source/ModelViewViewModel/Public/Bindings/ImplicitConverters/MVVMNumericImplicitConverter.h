// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/ImplicitConverters/MVVMImplicitConverter.h"

/**
 * For converting between numeric types
 */
class FMVVMNumericImplicitConverter : public FMVVMImplicitConverter
{
public:
	virtual bool CanConvertImplicitly(const FProperty* Source, const FProperty* Destination) override;
	virtual void ConvertImplicitly(const FProperty* Source, const FProperty* Destination, void* Data) override;
};
