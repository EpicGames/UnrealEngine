// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/Array.h"

class UStruct;
namespace UE::MVVM { struct FFieldContext; }
namespace UE::MVVM { struct FFunctionContext; }
namespace UE::MVVM { struct FMVVMConstFieldVariant; }
namespace UE::MVVM { struct FMVVMFieldVariant; }
struct FMVVMBindingName;
template <typename ValueType, typename ErrorType> class TValueOrError;

class FProperty;
class UFunction;

namespace UE::MVVM::BindingHelper
{

	/** Is the Property usable as a source (readable) by the binding system. It may required a Getter to read it. */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidForSourceBinding(const FProperty* InProperty);

	/**
	 * Is the Function usable as a source (readable) by the binding system.
	 * @note It may be a BlueprintGetter and binding to the Property would be better in the editor.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidForSourceBinding(const UFunction* InFunction);

	/**
	 * Is the Field usable as a source (readable) by the binding system.
	 * @note It may be a BlueprintGetter and binding to the Property would be better in the editor.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidForSourceBinding(const FMVVMConstFieldVariant InField);

	/** Is the Property usable as a destination (settable) by the binding system. It may required a Setter to write it. */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidForDestinationBinding(const FProperty* InProperty);

	/**
	 * Is the Function usable as a destination (settable) by the binding system.
	 * @note It may be a BlueprintSetter and binding to the Property would be better in the editor.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidForDestinationBinding(const UFunction* InFunction);

	/**
	 * Is the Field usable as a destination (settable) by the binding system.
	 * @note It may be a BlueprintSetter and binding to the Property would be better in the editor.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidForDestinationBinding(const FMVVMConstFieldVariant InVariant);

	/**
	 * Is the Function usable as a conversion function by the binding system without any wrapper.
	 * A simple conversion takes a single argument and returns a single value.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidForSimpleRuntimeConversion(const UFunction* InFunction);

	/**
	 * Is the Function usable as a conversion function by the binding system with a wrapper.
	 * A complex conversion takes a no argument and returns a single value.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidForComplexRuntimeConversion(const UFunction* InFunction);

	/**
	 * Is the Function usable as a delegate signature binding.
	 * Delegate signature functions are used for binding to generated events
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidForDelegateSignatureBinding(const UFunction* InFunction);

	/**
	 * Is the Function usable as an event binding.
	 * Events can trigger functions with any number of parameters and any return values.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidForEventBinding(const UFunction* InFunction);

	/**
	 * Is the Field usable as an event binding by the binding system.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidForEventBinding(const FMVVMConstFieldVariant InVariant);

#if WITH_EDITOR
	/** Is the Property usable as a source by the binding system and can it be read directly or it requires a Getter. */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsAccessibleDirectlyForSourceBinding(const FProperty* InProperty);

	/** Is the Property usable as a destination by the binding system and can it be write directly or it requires a Setter. */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsAccessibleDirectlyForDestinationBinding(const FProperty* InProperty);

	/** Is the Property usable as a source by the binding system and a Getter exists. */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsAccessibleWithGetterForSourceBinding(const FProperty* InProperty);

	/** Is the Property usable as a destination by the binding system and a Setter exists. */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsAccessibleWithSetterForDestinationBinding(const FProperty* InProperty);
#endif

	/**
	 * Converts a given graph / function name into it's delegate signature name (By appending the correct UE suffix).
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API FName GetDelegateSignatureName(FName InGraphName);

	/**
	 * Returns the Property or the Function that matches that BindingName.
	 * @note It doesn't check for BlueprintGetter or BlueprintSetter.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API FMVVMFieldVariant FindFieldByName(const UStruct* Container, FMVVMBindingName BindingName);

	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<const FProperty*, FText> TryGetPropertyTypeForSourceBinding(const FMVVMConstFieldVariant& InField);
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<const FProperty*, FText> TryGetPropertyTypeForDestinationBinding(const FMVVMConstFieldVariant& InField);
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<const FProperty*, FText> TryGetPropertyTypeForSourceBinding(const FProperty* InProperty);
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<const FProperty*, FText> TryGetPropertyTypeForDestinationBinding(const FProperty* InProperty);
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<const FProperty*, FText> TryGetPropertyTypeForSourceBinding(const UFunction* InFunction);
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<const FProperty*, FText> TryGetPropertyTypeForDestinationBinding(const UFunction* InFunction);

	/** Returns the return property for a conversion function, or an error if the function is not valid as a conversion function. */
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<const FProperty*, FText> TryGetReturnTypeForConversionFunction(const UFunction* InFunction);

	/** Returns the argument properties for a conversion function, or an error if the function is not valid as a conversion function. */
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<TArray<const FProperty*>, FText> TryGetArgumentsForConversionFunction(const UFunction* InFunction);

	/** Are type the same or could be converted at runtime. */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool ArePropertiesCompatible(const FProperty* Source, const FProperty* Destination);

	/**
	 * Returns the "return" property of the function. It can be a none-const ref/out argument.
	 * int Foo(double) -> returns int
	 * void Foo(int&, double) -> returns int
	 * void Foo(const int&, double) -> returns null
	 * int Foo(int&, double) -> returns null
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API const FProperty* GetReturnProperty(const UFunction* InFunction);

	/**
	 * Returns the first argument property. Ref/Out argument have to be const.
	 * int Foo(double) -> returns double
	 * void Foo(int& Out, double) -> returns double
	 * void Foo(const int& Out, double) -> returns int
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API const FProperty* GetFirstArgumentProperty(const UFunction* InFunction);


	/**
	 * Returns all argument properties of this function.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API TArray<const FProperty*> GetAllArgumentProperties(const UFunction* InFunction);

	/**
	 * Returns true if the given property is a valid function argument property.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsValidArgumentProperty(const FProperty* Property);

	/**
	 * Execute a binding that can be
	 *   LocalValue = Src.Property; Dst.Property = LocalValue;
	 *   LocalValue = Src.Property; Dst.SetProperty(LocalValue);
	 *   LocalValue = Src.GetProperty(); Dst.Property = LocalValue;
	 *   LocalValue = Src.GetProperty(); Dst.SetProperty(LocalValue);
	 * with conversion from float to double or integral to integral between the getter and the setter.
	 * 
	 * @note No test is performed to see if the Src Property can be safely assign to the Destination Property. Use with caution.
	 * @note No test is performed to see if the Src.GetProperty can be safely executed. Use with caution.
	 * @note No test is performed to see if the Dst.SetProperty can be safely executed. Use with caution.
	 */
	MODELVIEWVIEWMODEL_API void ExecuteBinding_NoCheck(const FFieldContext& Source, const FFieldContext& Destination);

	/**
	 * Execute a binding that can be
	 *   LocalValue = ConcFuncOwner.ConversionFunction(LocalValue); Dst.Property = LocalValue;
	 *   LocalValue = ConcFuncOwner.ConversionFunction(LocalValue); Dst.SetProperty(LocalValue);
	 * with conversion from float to double or integral to integral between the getter and the conversion function and/or the conversion function and the setter.
	 *
	 * @note No test is performed to see if the Dst.SetProperty can be safely executed. Use with caution.
	 * @note No test is performed to see if the ConversionFunction can be safely executed. Use with caution.
	 */
	MODELVIEWVIEWMODEL_API void ExecuteBinding_NoCheck(const FFieldContext& Destination, const FFunctionContext& ConversionFunction);

	/**
	 * Execute a binding that can be
	 *   LocalValue = Src.Property; LocalValue = ConcFuncOwner.ConversionFunction(LocalValue); Dst.Property = LocalValue;
	 *   LocalValue = Src.Property; LocalValue = ConcFuncOwner.ConversionFunction(LocalValue); Dst.SetProperty(LocalValue);
	 *   LocalValue = Src.GetProperty(); LocalValue = ConcFuncOwner.ConversionFunction(LocalValue); Dst.Property = LocalValue;
	 *   LocalValue = Src.GetProperty(); LocalValue = ConcFuncOwner.ConversionFunction(LocalValue); Dst.SetProperty(LocalValue);
	 * with conversion from float to double between the getter and the conversion function and/or the conversion function and the setter.
	 *
	 * @note No test is performed to see if the Src Property can be safely assign to the Destination Property. Use with caution.
	 * @note No test is performed to see if the Src.GetProperty can be safely executed. Use with caution.
	 * @note No test is performed to see if the Dst.SetProperty can be safely executed. Use with caution.
	 * @note No test is performed to see if the ConversionFunction can be safely executed. Use with caution.
	 */
	MODELVIEWVIEWMODEL_API void ExecuteBinding_NoCheck(const FFieldContext& Source, const FFieldContext& Destination, const FFunctionContext& ConversionFunction);

	/**
	 * Execute a function with no parameters and no return value
	 *
	 * @note No test is performed to verify the function doesn't have parameters and return value. Use with caution.
	 */
	MODELVIEWVIEWMODEL_API void ExecuteFunction_NoReturnValue(UFunction* InFunction, UObject* UserWidget);

} //namespace
