// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FieldNotificationId.h" // IWYU pragma: keep
#include "Types/MVVMObjectVariant.h"
#include "MVVMCompiledBindingLibrary.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

namespace UE::FieldNotification { struct FFieldId; }
namespace UE::MVVM { struct FFieldContext; }
namespace UE::MVVM { struct FFunctionContext; }
namespace UE::MVVM { struct FMVVMFieldVariant; }
template <typename ValueType, typename ErrorType> class TValueOrError;

struct FMVVMCompiledBindingLibrary;
namespace UE::MVVM
{
	class FCompiledBindingLibraryCompiler;
}


/** The info to fetch a list of FProperty or UFunction from a Class that will be needed by bindings. */
USTRUCT()
struct FMVVMVCompiledFields
{
	GENERATED_BODY()
	friend UE::MVVM::FCompiledBindingLibraryCompiler;

public:
	int32 GetPropertyNum() const
	{
		return ClassOrScriptStruct ? NumberOfProperties : 0;
	}

	int32 GetFunctionNum() const
	{
		return ClassOrScriptStruct ? NumberOfFunctions : 0;
	}

	FName GetPropertyName(const TArrayView<const FName> Names, int32 Index) const
	{
		check(Index < NumberOfProperties && Index >= 0);
		check(Names.IsValidIndex(LibraryStartIndex + Index));
		return Names[LibraryStartIndex + Index];
	}

	FName GetFunctionName(const TArrayView<const FName> Names, int32 Index) const
	{
		check(Index < NumberOfFunctions && Index >= 0);
		check(Names.IsValidIndex(LibraryStartIndex + NumberOfProperties + Index));
		return Names[LibraryStartIndex + NumberOfProperties + Index];
	}

	const UStruct* GetStruct() const
	{
		return ClassOrScriptStruct;
	}

	/** Find the FProperty from the  Class.*/
	UE_API FProperty* GetProperty(FName PropertyName) const;

	/** Find the UFunction from the Class.*/
	UE_API UFunction* GetFunction(FName FunctionName) const;

private:
	UPROPERTY()
	TObjectPtr<const UStruct> ClassOrScriptStruct = nullptr;

	UPROPERTY()
	int16 LibraryStartIndex = 0;

	UPROPERTY()
	int16 NumberOfProperties = 0;

	UPROPERTY()
	int16 NumberOfFunctions = 0;
};


/** Contains the info to evaluate a path for a specific library. */
USTRUCT()
struct FMVVMVCompiledFieldPath
{
	GENERATED_BODY()

	friend FMVVMCompiledBindingLibrary;
	friend UE::MVVM::FCompiledBindingLibraryCompiler;

	bool IsValid() const
	{
		return Num >= 0;
	}
	
	int16 GetStartIndex() const
	{
		return StartIndex;
	}

private:
	using IndexType = int16;

	UPROPERTY()
	int16 StartIndex = INDEX_NONE;

	UPROPERTY()
	int16 Num = INDEX_NONE;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FGuid CompiledBindingLibraryId;
#endif
};


/** */
USTRUCT()
struct FMVVMCompiledLoadedPropertyOrFunctionIndex
{
	GENERATED_BODY()

	friend FMVVMCompiledBindingLibrary;
	friend UE::MVVM::FCompiledBindingLibraryCompiler;

	FMVVMCompiledLoadedPropertyOrFunctionIndex()
	{
		bIsObjectProperty = false;
		bIsScriptStructProperty = false;
		bIsProperty = false;
	}

private:
	/** Is the index in LoadedProperties or LoadedFunctions.*/
	UPROPERTY()
	int16 Index = INDEX_NONE;

	/** Is the property or the return property of the UFunction is a FObjectPropertyBase */
	UPROPERTY()
	uint8 bIsObjectProperty : 1;

	/** Is the property or the return property of the UFunction is a FStructProperty */
	UPROPERTY()
	uint8 bIsScriptStructProperty : 1;

	/** Is the index in LoadedProperties or LoadedFunctions. */
	UPROPERTY()
	uint8 bIsProperty : 1;
};


/** Contains a single combination to execute a binding for a specific library. */
USTRUCT()
struct FMVVMVCompiledBinding
{
	GENERATED_BODY()

	friend FMVVMCompiledBindingLibrary;
	friend UE::MVVM::FCompiledBindingLibraryCompiler;

public:
	bool IsValid() const
	{
		const bool bSourceValid = SourceFieldPath.IsValid() || HasComplexConversionFunction() || IsComplexBinding();
		const bool bDestinationValid = DestinationFieldPath.IsValid();
		const bool bFunctionValid = ConversionFunctionFieldPath.IsValid() || (EType)Type == EType::None || IsComplexBinding();
		return bSourceValid && bDestinationValid && bFunctionValid;
	}

	FMVVMVCompiledFieldPath GetSourceFieldPath() const
	{
		return SourceFieldPath;
	}

	FMVVMVCompiledFieldPath GetDestinationFieldPath() const
	{
		return DestinationFieldPath;
	}

	FMVVMVCompiledFieldPath GetConversionFunctionFieldPath() const
	{
		return ConversionFunctionFieldPath;
	}

	/**
	 * A conversion function is needed to run the binding.
	 * The binding runs in native.
	 * It is on the form: Destination = ConversionFunction(Source)
	 */
	bool HasSimpleConversionFunction() const
	{
		return (EType)Type == EType::HasConversionFunction;
	}
	
	/**
	 * A conversion function is needed to run the binding.
	 * The conversion function takes more than one input.
	 * The binding doesn't requires the SourceFieldPath to execute.
	 * The binding runs in native.
	 * It is on the form Destination = ComplexConversionFunction()
	 */
	bool HasComplexConversionFunction() const
	{
		return (EType)Type == EType::HasComplexConversionFunction;
	}

	/**
	 * The binding runs in native.
	 * It is on the form: Destination = Source or Destination = ConversionFunction(Source) or Destination = ComplexConversionFunction()
	 */
	bool IsRuntimeBinding() const
	{
		return (EType)Type != EType::BindingComplex;
	}

	/**
	 * The binding runs with the BP virtual machine.
	 * It is on the form: Destination()
	 */
	bool IsComplexBinding() const
	{
		return (EType)Type == EType::BindingComplex;
	}

private:
	using IndexType = int16;

	UPROPERTY()
	FMVVMVCompiledFieldPath SourceFieldPath;

	UPROPERTY()
	FMVVMVCompiledFieldPath DestinationFieldPath;

	UPROPERTY()
	FMVVMVCompiledFieldPath ConversionFunctionFieldPath;

	enum class EType : uint8
	{
		None = 0,
		HasConversionFunction = 1,
		HasComplexConversionFunction = 2,
		BindingComplex = 3,
	};

	UPROPERTY()
	uint8 Type = (uint8)EType::None;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FGuid CompiledBindingLibraryId;
#endif
};


/** Library of all the compiled bindings. */
USTRUCT()
struct FMVVMCompiledBindingLibrary
{
	GENERATED_BODY()

	friend UE::MVVM::FCompiledBindingLibraryCompiler;

public:
	UE_API FMVVMCompiledBindingLibrary();
#if WITH_EDITOR
	UE_API FMVVMCompiledBindingLibrary(FGuid LibraryId);
#endif

public:
	/** */
	enum class EExecutionFailingReason
	{
		IncompatibleLibrary,
		InvalidSource,
		InvalidDestination,
		InvalidConversionFunction,
		InvalidCast,
	};

	static UE_API FText LexToText(EExecutionFailingReason Reason);

	/** */
	enum class EConversionFunctionType
	{
		// accept an single argument and returns a single property
		Simple,
		// returns a single property. The argument are fetch from inside the function 
		Complex,
	};

	/** Fetch the FProperty and UFunction. */
	UE_API void Load();
	/** Release the acquired FProperty and UFunction. */
	UE_API void Unload();
	/** FProperty and UFunction are fetched. */
	UE_API bool IsLoaded() const;

	/**
	 * Execute a binding, in one direction.
	 * The ExecutionSource is the View (UserWidget) instance.
	 * The Source of the binding (ViewModel, UserWidget, Widget, ...) can be provided if already known.
	 * The Destination or the binding (ViewModel, UserWidget, Widget, ...) can be provided if already known.
	 */
	UE_API TValueOrError<void, EExecutionFailingReason> Execute(UObject* ExecutionSource, const FMVVMVCompiledBinding& Binding, EConversionFunctionType ConversionType) const;
	UE_API TValueOrError<void, EExecutionFailingReason> ExecuteWithSource(UObject* ExecutionSource, const FMVVMVCompiledBinding& Binding, UObject* Source) const;

	/**
	 * Evaluate the path to find the container use by a binding.
	 * ExecutionSource is the View (UserWidget) instance.
	 * @returns the container and the last segment of the path. The last segment can be a property or a function.
	 */
	UE_API TValueOrError<UE::MVVM::FFieldContext, void> EvaluateFieldPath(UObject* ExecutionSource, const FMVVMVCompiledFieldPath& FieldPath) const;

	/** Return a readable version of the FieldPath. */
	UE_API TValueOrError<FString, FString> FieldPathToString(FMVVMVCompiledFieldPath FieldPath, bool bUseDisplayName) const;

private:
	UE_API TValueOrError<void, EExecutionFailingReason> ExecuteImpl(UObject* ExecutionSource, const FMVVMVCompiledBinding& Binding, UObject* Source, EConversionFunctionType ConversionType) const;
	UE_API TValueOrError<void, EExecutionFailingReason> ExecuteImpl(UE::MVVM::FFieldContext& Source, UE::MVVM::FFieldContext& Destination, UE::MVVM::FFunctionContext& ConversionFunction, EConversionFunctionType ConversionType) const;

	UE_API TValueOrError<UE::MVVM::FMVVMFieldVariant, void> GetFinalFieldFromPathImpl(UE::MVVM::FObjectVariant CurrentContainer, const FMVVMVCompiledFieldPath& FieldPath) const;

	UE_API bool CanAccessLoadedProperiesAtIndex(int16 index) const;

private:
	struct FLoadedFunction
	{
		FLoadedFunction() = default;
		MODELVIEWVIEWMODEL_API FLoadedFunction(const UFunction* Function);

		TWeakObjectPtr<const UClass> ClassOwner;
		FName FunctionName;
		bool bIsFunctionVirtual = false;

		UFunction* GetFunction() const;
		UFunction* GetFunction(const UObject* CallingContext) const;
	};

	TArray<FProperty*> LoadedProperties;
	TArray<FLoadedFunction> LoadedFunctions;

	UPROPERTY()
	TArray<FMVVMCompiledLoadedPropertyOrFunctionIndex> FieldPaths;

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FGuid CompiledBindingLibraryId;
#endif

	/** Only needed for loading the FProperty/UFunction. */
	UPROPERTY()
	TArray<FMVVMVCompiledFields> CompiledFields;

	/** Only needed for loading the FProperty/UFunction. */
	UPROPERTY()
	TArray<FName> CompiledFieldNames;
};

#undef UE_API
