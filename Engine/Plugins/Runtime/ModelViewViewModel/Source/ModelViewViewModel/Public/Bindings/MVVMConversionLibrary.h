// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SlateWrapperTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MVVMConversionLibrary.generated.h"

UCLASS(MinimalAPI)
class UMVVMConversionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
	/**
	 * Converts a bool to a slate visibility.
	 */
	UFUNCTION(BlueprintPure, Category = "Widget", meta = (DisplayName = "To Visibility (Boolean)"))
	static ESlateVisibility Conv_BoolToSlateVisibility(bool bIsVisible, ESlateVisibility TrueVisibility = ESlateVisibility::Visible, ESlateVisibility FalseVisibility = ESlateVisibility::Collapsed);
};
