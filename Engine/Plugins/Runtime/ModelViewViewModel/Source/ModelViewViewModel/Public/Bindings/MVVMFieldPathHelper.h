// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FieldNotificationId.h"
#include "Templates/SubclassOf.h"

namespace UE::MVVM { struct FFieldContext; }
namespace UE::MVVM { struct FMVVMConstFieldVariant; }
namespace UE::MVVM { struct FMVVMFieldVariant; }
template <typename ValueType, typename ErrorType> class TValueOrError;
class UWidgetBlueprintGeneratedClass;

namespace UE::MVVM::FieldPathHelper
{
	/**
	 * Info about the viewmodel and field notify.
	 * It can has invalid values and still be a valid OneTime binding.
	 */
	struct FParsedNotifyBindingInfo
	{
		/**
		 * The viewmodel that is used to create the binding.
		 * Value is the class of "ViewmodelB" when the path is "ViewmodelA.ViewmodelB.Vector.X"
		 */
		const UClass* NotifyFieldClass = nullptr;

		/**
		 * The property we will bind to
		 * Value is "Vector" when the path is "ViewmodelA.ViewmodelB.Vector.X"
		 */
		FFieldNotificationId NotifyFieldId;

		/**
		 * The viewmodel index in the original path.
		 * When it's INDEX_NONE but the ParsedBindingInfo is valid, the class accessor is the Accessor class.
		 * Value 1 when the path is "ViewmodelA.ViewmodelB.Vector.X"
		 */
		int32 ViewModelIndex = INDEX_NONE;
	};

	/** */
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<TArray<FMVVMConstFieldVariant>, FText> GenerateFieldPathList(const UClass* From, FStringView FieldPath, bool bForSourceBinding);

	/** */
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<TArray<FMVVMConstFieldVariant>, FText> GenerateFieldPathList(TArrayView<const FMVVMConstFieldVariant> Fields, bool bForSourceBinding);

	/**
	 * Generate the FParsedNotifyBindingInfo from the parsed field path.
	 * This gives us which FieldId and Viewmodel that will be used by the path.
	 * Note. The code may have changed since the binding was created in the editor. As an example, a sub-object can be a viewmodel now.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<FParsedNotifyBindingInfo, FText> GetNotifyBindingInfoFromFieldPath(const UClass* Accessor, TArrayView<const FMVVMConstFieldVariant> Fields);

	/** */
	[[nodiscard]] MODELVIEWVIEWMODEL_API FString ToString(TArrayView<const FMVVMFieldVariant> Fields);
	[[nodiscard]] MODELVIEWVIEWMODEL_API FString ToString(TArrayView<const FMVVMConstFieldVariant> Fields);
	[[nodiscard]] MODELVIEWVIEWMODEL_API FText ToText(TArrayView<const FMVVMFieldVariant> Fields);
	[[nodiscard]] MODELVIEWVIEWMODEL_API FText ToText(TArrayView<const FMVVMConstFieldVariant> Fields);

	/** Returns the object value from the FieldContext. */
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<UObject*, void> EvaluateObjectProperty(const FFieldContext& InSource);

	/**
	 * Returns the container that the field represents.
	 * If the field is a Object property, then returns the PropertyClass.
	 * If the field is a Struct property, then returns the Struct.
	 * If the field is a Function, then returns the return value property's container.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API TValueOrError<const UStruct*, void> GetFieldAsContainer(const UE::MVVM::FMVVMConstFieldVariant Field);

	/**
	 * Returns the generated widget blueprint class of the widget that owns this component.
	 * Accessor: the widget blueprint class that owns the binding.
	 * ComponentPaths: The path to a field on a component.
	 */
	[[nodiscard]] MODELVIEWVIEWMODEL_API const UWidgetBlueprintGeneratedClass* GetComponentPropertyPathSource(TArray<UE::MVVM::FMVVMConstFieldVariant> ComponentPaths, const UWidgetBlueprintGeneratedClass* Accessor);

} // namespace
