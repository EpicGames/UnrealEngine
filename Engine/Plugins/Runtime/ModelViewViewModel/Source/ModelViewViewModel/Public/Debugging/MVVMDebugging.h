// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h" // IWYU pragma: keep

#ifndef UE_WITH_MVVM_DEBUGGING
#define UE_WITH_MVVM_DEBUGGING (!(UE_BUILD_SHIPPING || UE_BUILD_TEST)) || WITH_EDITOR
#endif


#if UE_WITH_MVVM_DEBUGGING

#include "Bindings/MVVMCompiledBindingLibrary.h"
#include "View/MVVMViewTypes.h"

class UMVVMView;
struct FMVVMViewClass_CompiledBinding;
struct FMVVMViewSource;
class UUserWidget;

namespace UE::MVVM
{
/** */
class FDebugging
{
public:
	struct FView
	{
	public:
		FView() = default;
		MODELVIEWVIEWMODEL_API FView(const UMVVMView* View);
		MODELVIEWVIEWMODEL_API FView(const UUserWidget* UserWidget, const UMVVMView* View);

	public:
		const UUserWidget* GetUserWidget() const
		{
			return UserWidget;
		}

		const UMVVMView* GetView() const
		{
			return View;
		}
	private:
		const UUserWidget* UserWidget = nullptr;
		const UMVVMView* View = nullptr;
	};
public:
	struct FViewConstructedArgs
	{
	};

	DECLARE_EVENT_TwoParams(FDebugging, FViewConstructed, const FView&, const FViewConstructedArgs&);
	/** Broadcast when a view is created and the viewmodels are instantiated and the binding are not registered. */
	static MODELVIEWVIEWMODEL_API FViewConstructed OnViewConstructed;
	static MODELVIEWVIEWMODEL_API void BroadcastViewConstructed(const UMVVMView* View);


	struct FViewBeginDestructionArgs
	{
	};

	DECLARE_EVENT_TwoParams(FDebugging, FViewDestructing, const FView&, const FViewBeginDestructionArgs&);
	/** Broadcast before a view is destroyed. */
	static MODELVIEWVIEWMODEL_API FViewDestructing OnViewBeginDestruction;
	static MODELVIEWVIEWMODEL_API void BroadcastViewBeginDestruction(const UMVVMView* View);

public:
	struct FViewSourceValueArgs
	{
		FViewSourceValueArgs(const FMVVMViewClass_SourceKey& Class, const FMVVMView_SourceKey& View);
		FMVVMViewClass_SourceKey ClassSource;
		FMVVMView_SourceKey ViewSource;
	};

	DECLARE_EVENT_TwoParams(FDebugging, FViewSourceValueChanged, const FView&, const FViewSourceValueArgs&);
	/** Broadcast when a source changes. */
	static MODELVIEWVIEWMODEL_API FViewSourceValueChanged OnViewSourceValueChanged;
	static MODELVIEWVIEWMODEL_API void BroadcastViewSourceValueChanged(const UMVVMView* View, const FMVVMViewClass_SourceKey ClassSourceKey, const FMVVMView_SourceKey ViewSourceKey);

public:
	struct FLibraryBindingExecutedArgs
	{
		FLibraryBindingExecutedArgs(FMVVMViewClass_BindingKey Binding);
		FLibraryBindingExecutedArgs(FMVVMViewClass_BindingKey Binding, FMVVMCompiledBindingLibrary::EExecutionFailingReason Result);
		FMVVMViewClass_BindingKey Binding;
		TOptional<FMVVMCompiledBindingLibrary::EExecutionFailingReason> FailingReason;
	};

	/**  */
	DECLARE_EVENT_TwoParams(FDebugging, FLibraryBindingExecuted, const FView&, const FLibraryBindingExecutedArgs&);
	/** Broadcast when a registered field is modified and a binding need to execute. */
	static MODELVIEWVIEWMODEL_API FLibraryBindingExecuted OnLibraryBindingExecuted;
	static MODELVIEWVIEWMODEL_API void BroadcastLibraryBindingExecuted(const UMVVMView* View, const FLibraryBindingExecutedArgs& Args);
	static MODELVIEWVIEWMODEL_API void BroadcastLibraryBindingExecuted(const UMVVMView* View, FMVVMViewClass_BindingKey Binding);
	static MODELVIEWVIEWMODEL_API void BroadcastLibraryBindingExecuted(const UMVVMView* View, FMVVMViewClass_BindingKey Binding, FMVVMCompiledBindingLibrary::EExecutionFailingReason Result);
};
} // UE::MVVM

#else


namespace UE::MVVM
{
class FDebugging
{
};
} // UE::MVVM

#endif
