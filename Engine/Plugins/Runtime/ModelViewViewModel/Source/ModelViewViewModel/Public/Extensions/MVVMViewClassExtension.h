// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MVVMViewClassExtension.generated.h"

class UMVVMView;
class UUserWidget;

class UMVVMViewExtension;
class UMVVMViewClassExtension;

/**
 * A runtime extension class instance. This information is per instance of a UMVVMViewClassExtension
 */
UCLASS(MinimalAPI, Within=MVVMView)
class UMVVMViewExtension : public UObject
{
	GENERATED_BODY()

protected:
	UMVVMView* GetView() const
	{
		return GetOuterUMVVMView();
	}
};

/**
 * A runtime extension class to define MVVM-related properties and behavior. This information comes from the
 * corresponding UMVVMBlueprintViewExtension class. This class provides a hook into the MVVM runtime initializations.
 */
UCLASS(MinimalAPI, Within=MVVMViewClass)
class UMVVMViewClassExtension : public UObject
{
	GENERATED_BODY()

public:
	//~ Functions to be overriden in a user-defined UMVVMViewMyWidgetExtension class

	/** When the view is constructed. The class extension can create a view instance if needed or return nullptr. */
	virtual UMVVMViewExtension* ViewConstructed(UUserWidget* UserWidget, UMVVMView* View)
	{
		return nullptr;
	}
	virtual void OnSourcesInitialized(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension) {};
	virtual void OnBindingsInitialized(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension) {};
	virtual void OnEventsInitialized(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension) {};
	virtual void OnEventsUninitialized(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension) {};
	virtual void OnBindingsUninitialized(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension) {};
	virtual void OnSourcesUninitialized(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension) {};
	virtual void OnViewDestructed(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension) {};
};
