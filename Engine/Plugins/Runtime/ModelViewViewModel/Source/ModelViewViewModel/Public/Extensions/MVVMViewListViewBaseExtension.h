// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MVVMCompiledBindingLibrary.h"
#include "MVVMViewClassExtension.h"
#include "UObject/ObjectKey.h"
#include "MVVMViewListViewBaseExtension.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

class UListViewBase;
class UMVVMView;
class UUserWidget;

UCLASS(MinimalAPI)
class UMVVMViewListViewBaseClassExtension : public UMVVMViewClassExtension
{
	GENERATED_BODY()

public:
	//~ Begin UMVVMViewClassExtension overrides
	UE_API virtual void OnSourcesInitialized(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension) override;
	UE_API virtual void OnSourcesUninitialized(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension) override;
	//~ End UMVVMViewClassExtension overrides

#if WITH_EDITOR
	struct FInitListViewBaseExtensionArgs
	{
		FInitListViewBaseExtensionArgs(FName InWidgetName, FName InEntryViewModelName, const FMVVMVCompiledFieldPath& InWidgetPath)
			: WidgetName(InWidgetName),
			EntryViewModelName(InEntryViewModelName),
			WidgetPath(InWidgetPath)
		{}

		FName WidgetName;
		FName EntryViewModelName;
		FMVVMVCompiledFieldPath WidgetPath;
	};

	UE_API void Initialize(FInitListViewBaseExtensionArgs InArgs);
#endif

	FName GetEntryViewModelName() const
	{ 
		return EntryViewModelName; 
	}

private:
	UE_API void HandleSetViewModelOnEntryWidget(UUserWidget& EntryWidget, UUserWidget* OwningUserWidget, UListViewBase* ListWidget, FName EntryViewmodelName);

private:
	UPROPERTY(VisibleAnywhere, Category = "MVVM Extension")
	FName WidgetName;

	UPROPERTY(VisibleAnywhere, Category = "MVVM Extension")
	FName EntryViewModelName;

	UPROPERTY()
	FMVVMVCompiledFieldPath WidgetPath;

	TMap<FObjectKey, TWeakObjectPtr<UListViewBase>> CachedListViewWidgets;
};

#undef UE_API
