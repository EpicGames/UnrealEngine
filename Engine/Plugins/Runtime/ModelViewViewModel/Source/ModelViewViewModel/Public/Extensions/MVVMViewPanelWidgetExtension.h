// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MVVMCompiledBindingLibrary.h"
#include "Blueprint/UserWidget.h"
#include "MVVMViewClassExtension.h"
#include "MVVMViewPanelWidgetExtension.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

class UMVVMView;
class UPanelWidget;
class UUserWidget;

UCLASS(MinimalAPI)
class UMVVMPanelWidgetViewExtension : public UMVVMViewExtension
{
	GENERATED_BODY()

public:
	UE_API void Initialize(UMVVMViewPanelWidgetClassExtension* ClassExtension, UPanelWidget* PanelWidget);

	UFUNCTION(BlueprintCallable, Category = PanelWidget, meta = (AllowPrivateAccess = true, DisplayName = "Set Items", ViewmodelBlueprintWidgetExtension = "EntryViewModel"))
	UE_API void BP_SetItems(const TArray<UObject*>& InItems);

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnItemsChanged);

	UPROPERTY(BlueprintAssignable)
	FOnItemsChanged OnItemsChanged;

private:
	void SetViewModelOnEntryWidget(UUserWidget* EntryWidget, UObject* ViewModelObject, UUserWidget* OwningUserWidget);
	void ReplaceAllSlots(TArrayView<TTuple<UPanelSlot*, UWidget*>> NewSlots);
	UUserWidget* GetUserWidget() const;

private:
	UPROPERTY(Transient)
	TObjectPtr<UPanelWidget> PanelWidget;

	UPROPERTY(Transient)
	TObjectPtr<UMVVMViewPanelWidgetClassExtension> ClassExtension;
};

UCLASS(MinimalAPI)
class UMVVMViewPanelWidgetClassExtension : public UMVVMViewClassExtension
{
	GENERATED_BODY()

public:
	//~ Begin UMVVMViewClassExtension overrides
	UE_API virtual UMVVMViewExtension* ViewConstructed(UUserWidget* UserWidget, UMVVMView* View) override;
	UE_API virtual void OnViewDestructed(UUserWidget* UserWidget, UMVVMView* View, UMVVMViewExtension* Extension) override;
	//~ End UMVVMViewClassExtension overrides

#if WITH_EDITOR
	struct FInitPanelWidgetExtensionArgs
	{
		FInitPanelWidgetExtensionArgs(FName InWidgetName, FName InEntryViewModelName, const FMVVMVCompiledFieldPath& InWidgetPath, const TSubclassOf<UUserWidget>& InEntryWidgetClass, UPanelSlot* InSlotTemplate, FName InPanelPropertyName, UClass* InEntryViewModelClass)
			: WidgetName(InWidgetName),
			EntryViewModelName(InEntryViewModelName),
			WidgetPath(InWidgetPath),
			EntryWidgetClass(InEntryWidgetClass),
			SlotTemplate(InSlotTemplate),
			PanelPropertyName(InPanelPropertyName),
			EntryViewModelClass(InEntryViewModelClass)
		{}

		FName WidgetName;
		FName EntryViewModelName;
		FMVVMVCompiledFieldPath WidgetPath;
		TSubclassOf<UUserWidget> EntryWidgetClass;
		TObjectPtr<UPanelSlot> SlotTemplate;
		FName PanelPropertyName;
		TObjectPtr<UClass> EntryViewModelClass;
	};

	UE_API void Initialize(FInitPanelWidgetExtensionArgs InArgs);

#endif

public:
	FName GetWidgetName() const
	{ 
		return WidgetName;
	}
	
	FName GetEntryViewModelName() const
	{ 
		return EntryViewModelName; 
	}
	
	UPanelSlot* GetSlotTemplate() const
	{ 
		return SlotTemplate;
	}

	TSubclassOf<UUserWidget> GetEntryWidgetClass() const
	{
		return EntryWidgetClass;
	}

	UClass* GetEntryViewModelClass() const
	{
		return EntryViewModelClass;
	}

private:
	UPROPERTY(VisibleAnywhere, Category = "MVVM Extension")
	FName WidgetName;

	UPROPERTY(VisibleAnywhere, Category = "MVVM Extension")
	FName EntryViewModelName;

	UPROPERTY(VisibleAnywhere, Category = "MVVM Extension")
	TSubclassOf<UUserWidget> EntryWidgetClass;

	UPROPERTY(VisibleAnywhere, Category = "MVVM Extension")
	TObjectPtr<UPanelSlot> SlotTemplate;

	UPROPERTY(VisibleAnywhere, Category = "MVVM Extension")
	FName PanelPropertyName;

	UPROPERTY(VisibleAnywhere, Category = "MVVM Extension")
	TObjectPtr<UClass> EntryViewModelClass;

	UPROPERTY()
	FMVVMVCompiledFieldPath WidgetPath;
};

#undef UE_API
