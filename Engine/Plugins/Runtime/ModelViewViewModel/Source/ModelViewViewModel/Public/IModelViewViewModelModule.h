// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FMVVMImplicitConverter;

class IModelViewViewModelModule : public IModuleInterface
{
public:
	virtual void RegisterImplicitConverter(const TSharedRef<FMVVMImplicitConverter>& Converter) = 0;
	virtual void UnregisterImplicitConverter(const TSharedRef<FMVVMImplicitConverter>& Converter) = 0;
};
