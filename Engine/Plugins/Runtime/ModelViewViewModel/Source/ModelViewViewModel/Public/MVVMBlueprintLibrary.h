// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MVVMBlueprintLibrary.generated.h"

UCLASS()
class UMVVMBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category="MVVM | Utilities")
	static MODELVIEWVIEWMODEL_API void SetViewModelByClass(UUserWidget* Widget, TScriptInterface<INotifyFieldValueChanged> ViewModel);
};