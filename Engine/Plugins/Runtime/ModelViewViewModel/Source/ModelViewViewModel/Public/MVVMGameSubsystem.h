// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/GameInstanceSubsystem.h"
#include "Types/MVVMViewModelCollection.h"

#include "UObject/Package.h"
#include "MVVMGameSubsystem.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API


/** */
UCLASS(MinimalAPI, DisplayName="Viewmodel Game Subsytem")
class UMVVMGameSubsystem : public UGameInstanceSubsystem
{
	GENERATED_BODY()

public:
	//~ Begin UEngineSubsystem interface
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	//~ End UEngineSubsystem interface

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UMVVMViewModelCollectionObject* GetViewModelCollection() const
	{
		return ViewModelCollection;
	}
private:
	UPROPERTY(Transient)
	TObjectPtr<UMVVMViewModelCollectionObject> ViewModelCollection;
};

#undef UE_API
