// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/MessageLog.h"

#include "UObject/WeakObjectPtr.h"

#define UE_API MODELVIEWVIEWMODEL_API

struct FMVVMViewClass_Binding;
struct FMVVMViewClass_BindingKey;
class UMVVMViewClass;
class UUserWidget;

MODELVIEWVIEWMODEL_API DECLARE_LOG_CATEGORY_EXTERN(LogMVVM, Log, All);

namespace UE::MVVM
{
class FMessageLog : private ::FMessageLog
{
public:
	static UE_API const FName LogName;

	UE_API FMessageLog(const UUserWidget* InUserWidget);

	using ::FMessageLog::AddMessage;
	using ::FMessageLog::AddMessages;
	UE_API TSharedRef<FTokenizedMessage> Message(EMessageSeverity::Type InSeverity, const FText& InMessage);
	UE_API TSharedRef<FTokenizedMessage> Error(const FText& InMessage);
	UE_API TSharedRef<FTokenizedMessage> PerformanceWarning(const FText& InMessage);
	UE_API TSharedRef<FTokenizedMessage> Warning(const FText& InMessage);
	UE_API TSharedRef<FTokenizedMessage> Info(const FText& InMessage);

	UE_API void AddBindingToken(TSharedRef<FTokenizedMessage> NewMessage, const UMVVMViewClass* Class, const FMVVMViewClass_Binding& ClassBinding, FMVVMViewClass_BindingKey Key);

private:
	TWeakObjectPtr<const UUserWidget> UserWidget;
};
}

#undef UE_API
