// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "Types/MVVMFieldVariant.h"
#include "Types/MVVMBindingMode.h"
#include "Types/MVVMViewModelCollection.h"
#include "Types/MVVMConditionOperation.h"

#include "UObject/Package.h"
#include "MVVMSubsystem.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

struct FMVVMAvailableBinding;
struct FMVVMBindingName;
template <typename ValueType, typename ErrorType> class TValueOrError;

class UMVVMView;
class UMVVMViewModelBase;
class UUserWidget;
class UWidget;
class UWidgetTree;

/** */
UCLASS(MinimalAPI, DisplayName="Viewmodel Engine Subsytem")
class UMVVMSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	//~ Begin UEngineSubsystem interface
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	//~ End UEngineSubsystem interface

	UFUNCTION(BlueprintCallable, Category="Viewmodel", meta = (DisplayName = "Compare Float Values"))
	UE_API bool K2_CompareFloatValues(EMVVMConditionOperation Operation, float Value, float CompareValue, float CompareMaxValue = 0.0f);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel", meta = (DisplayName = "Get View From User Widget"))
	UE_API UMVVMView* K2_GetViewFromUserWidget(const UUserWidget* UserWidget) const;

	static UE_API UMVVMView* GetViewFromUserWidget(const UUserWidget* UserWidget);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API bool DoesWidgetTreeContainedWidget(const UWidgetTree* WidgetTree, const UWidget* ViewWidget) const;

	/** @return The list of all the AvailableBindings that are available for the Class. */
	UFUNCTION(BlueprintCallable, Category = "Viewmodel", meta = (DisplayName = "Get Available Bindings"))
	UE_API TArray<FMVVMAvailableBinding> K2_GetAvailableBindings(const UClass* Class, const UClass* Accessor) const;

	static UE_API TArray<FMVVMAvailableBinding> GetAvailableBindings(const UClass* Class, const UClass* Accessor);

	/**
	 * @return The list of all the AvailableBindings that are available from the SriptStuct.
	 * @note When FMVVMAvailableBinding::HasNotify is false, a notification can still be triggered by the owner of the struct. The struct changed but which property of the struct changed is unknown.
	 */
	static UE_API TArray<FMVVMAvailableBinding> GetAvailableBindingsForStruct(const UScriptStruct* Struct);

	static UE_API TArray<FMVVMAvailableBinding> GetAvailableBindingsForEvent(const UClass* Class, const UClass* Accessor);

	/** @return The AvailableBinding from a BindingName. */
	UFUNCTION(BlueprintCallable, Category = "Viewmodel", meta = (DisplayName = "Get Available Binding"))
	UE_API FMVVMAvailableBinding K2_GetAvailableBinding(const UClass* Class, FMVVMBindingName BindingName, const UClass* Accessor) const;

	static UE_API FMVVMAvailableBinding GetAvailableBinding(const UClass* Class, FMVVMBindingName BindingName, const UClass* Accessor);

	/** @return The AvailableBinding from a field. */
	static UE_API FMVVMAvailableBinding GetAvailableBindingForField(UE::MVVM::FMVVMConstFieldVariant Variant, const UClass* Accessor);

	static UE_API FMVVMAvailableBinding GetAvailableBindingForEvent(UE::MVVM::FMVVMConstFieldVariant FieldVariant, const UClass* Accessor);

	static UE_API FMVVMAvailableBinding GetAvailableBindingForEvent(const UClass* Class, FMVVMBindingName BindingName, const UClass* Accessor);

	UE_DEPRECATED(5.3, "GetGlobalViewModelCollection has been deprecated, please use the game instance subsystem.")
	UFUNCTION(BlueprintCallable, Category = "Viewmodel", meta=(DeprecatedFunction, DeprecatedMessage = "This version of GetGlobalViewModelCollection has been deprecated, please use GetViewModelCollection from the Viewmodel Game subsystem."))
	UE_API UMVVMViewModelCollectionObject* GetGlobalViewModelCollection() const;

	struct FConstDirectionalBindingArgs
	{
		UE::MVVM::FMVVMConstFieldVariant SourceBinding;
		UE::MVVM::FMVVMConstFieldVariant DestinationBinding;
		const UFunction* ConversionFunction = nullptr;
	};

	struct FDirectionalBindingArgs
	{
		UE::MVVM::FMVVMFieldVariant SourceBinding;
		UE::MVVM::FMVVMFieldVariant DestinationBinding;
		UFunction* ConversionFunction = nullptr;

		FConstDirectionalBindingArgs ToConst() const
		{
			FConstDirectionalBindingArgs ConstArgs;
			ConstArgs.SourceBinding = SourceBinding;
			ConstArgs.DestinationBinding = DestinationBinding;
			ConstArgs.ConversionFunction = ConversionFunction;
			return MoveTemp(ConstArgs);
		}
	};

	struct FBindingArgs
	{
		EMVVMBindingMode Mode = EMVVMBindingMode::OneWayToDestination;
		FDirectionalBindingArgs ForwardArgs;
		FDirectionalBindingArgs BackwardArgs;
	};

	[[nodiscard]] UE_API TValueOrError<bool, FText> IsBindingValid(FConstDirectionalBindingArgs Args) const;
	[[nodiscard]] UE_API TValueOrError<bool, FText> IsBindingValid(FDirectionalBindingArgs Args) const;
	[[nodiscard]] UE_API TValueOrError<bool, FText> IsBindingValid(FBindingArgs Args) const;
};

#undef UE_API
