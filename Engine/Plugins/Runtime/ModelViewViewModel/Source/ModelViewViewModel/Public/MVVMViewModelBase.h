// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Algo/Compare.h"
#include "FieldNotificationDeclaration.h" // IWYU pragma: keep
#include "IFieldNotificationClassDescriptor.h"
#include "ViewModel/MVVMFieldNotificationDelegates.h"

#include "MVVMViewModelBase.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

/** After a field value changed. Broadcast the event. */
#define UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(MemberName) \
	BroadcastFieldValueChanged(ThisClass::FFieldNotificationClassDescriptor::MemberName)

/** If the property value changed then set the new value and notify. */
#define UE_MVVM_SET_PROPERTY_VALUE(MemberName, NewValue) \
	SetPropertyValue(MemberName, NewValue, ThisClass::FFieldNotificationClassDescriptor::MemberName)

/** Use this version to set property values that can't be captured as a function arguments (i.e. bitfields). */
#define UE_MVVM_SET_PROPERTY_VALUE_INLINE(MemberName, NewValue) \
	[this, InNewValue = (NewValue)]() { if (MemberName == InNewValue) { return false; } MemberName = InNewValue; BroadcastFieldValueChanged(ThisClass::FFieldNotificationClassDescriptor::MemberName); return true; }()

namespace UMMVVMViewModelBase
{
	// valid keywords for the UCLASS macro
	enum
	{
		// [ClassMetadata] Specifies which ContextCreationType is allowed for that viewmodel. ex: UCLASS(meta=(MVVMAllowedContextCreationType="Manual|CreateInstance"))
		MVVMAllowedContextCreationType,
		// [ClassMetadata] Specifies which ContextCreationType is disallowed for that viewmodel, all other type are allowed. ex: UCLASS(meta=(MVVMDisallowedContextCreationType="Manual|CreateInstance"))
		MVVMDisallowedContextCreationType
	};
}


/** Base class for MVVM viewmodel. */
UCLASS(MinimalAPI, Blueprintable, Abstract, DisplayName="MVVM Base Viewmodel", meta=(ShowWorldContextPin="true"))
class UMVVMViewModelBase : public UObject, public INotifyFieldValueChanged
{
	GENERATED_BODY()

public:
	struct FFieldNotificationClassDescriptor : public ::UE::FieldNotification::IClassDescriptor
	{
		UE_API virtual void ForEachField(const UClass* Class, TFunctionRef<bool(UE::FieldNotification::FFieldId FielId)> Callback) const override;
	};

public:
	//~ Begin INotifyFieldValueChanged Interface
	UE_API virtual FDelegateHandle AddFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FFieldValueChangedDelegate InNewDelegate) override final;
	UE_API virtual bool RemoveFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FDelegateHandle InHandle) override final;
	UE_API virtual int32 RemoveAllFieldValueChangedDelegates(FDelegateUserObjectConst InUserObject) override final;
	UE_API virtual int32 RemoveAllFieldValueChangedDelegates(UE::FieldNotification::FFieldId InFieldId, FDelegateUserObjectConst InUserObject) override final;
	UE_API virtual const UE::FieldNotification::IClassDescriptor& GetFieldNotificationDescriptor() const override;
	UE_API virtual void BroadcastFieldValueChanged(UE::FieldNotification::FFieldId InFieldId) override;
	//~ End INotifyFieldValueChanged Interface

	UE_API TArray<UE::FieldNotification::FFieldMulticastDelegate::FDelegateView> GetNotificationDelegateView() const;

	UFUNCTION(BlueprintCallable, Category = "FieldNotify", meta = (DisplayName = "Add Field Value Changed Delegate", ScriptName = "AddFieldValueChangedDelegate"))
	UE_API void K2_AddFieldValueChangedDelegate(FFieldNotificationId FieldId, FFieldValueChangedDynamicDelegate Delegate);

	UFUNCTION(BlueprintCallable, Category = "FieldNotify", meta = (DisplayName = "Remove Field Value Changed Delegate", ScriptName = "RemoveFieldValueChangedDelegate"))
	UE_API void K2_RemoveFieldValueChangedDelegate(FFieldNotificationId FieldId, FFieldValueChangedDynamicDelegate Delegate);

protected:
	UE_DEPRECATED(5.3, "BroadcastFieldValueChanged has been deprecated, please use the regular blueprint setter.")
	UFUNCTION(BlueprintCallable, Category="FieldNotify", meta=(DisplayName="Broadcast Field Value Changed", ScriptName="BroadcastFieldValueChanged", DeprecatedFunction, DeprecatedMessage = "Use the regular setter node."))
	UE_API void K2_BroadcastFieldValueChanged(FFieldNotificationId FieldId);

	UE_DEPRECATED(5.3, "SetPropertyValue has been deprecated, please use the regular blueprint setter.")
	UFUNCTION(BlueprintCallable, CustomThunk, Category="Viewmodel", meta=(CustomStructureParam="OldValue,NewValue", ScriptName="SetPropertyValue", DeprecatedFunction, DeprecatedMessage = "Use the regular setter node.", BlueprintInternalUseOnly="true"))
	UE_API bool K2_SetPropertyValue(UPARAM(ref) const int32& OldValue, UPARAM(ref) const int32& NewValue);

	/** Set the new value and notify if the property value changed. */
	template<typename T, typename U = T>
	bool SetPropertyValue(T& Value, const U& NewValue, UE::FieldNotification::FFieldId FieldId)
	{
		if (Value == NewValue)
		{
			return false;
		}

		Value = NewValue;
		BroadcastFieldValueChanged(FieldId);
		return true;
	}

	bool SetPropertyValue(FText& Value, const FText& NewValue, UE::FieldNotification::FFieldId FieldId)
	{
		if (Value.IdenticalTo(NewValue))
		{
			return false;
		}

		Value = NewValue;
		BroadcastFieldValueChanged(FieldId);
		return true;
	}

	bool SetPropertyValue(TArray<FText>& Value, const TArray<FText>& NewValue, UE::FieldNotification::FFieldId FieldId)
	{
		if (Algo::Compare(Value, NewValue, [](const FText& LHS, const FText& RHS)
			{
				return LHS.IdenticalTo(RHS);
			}))
		{
			return false;
		}

		Value = NewValue;
		BroadcastFieldValueChanged(FieldId);
		return true;
	}

private:
	DECLARE_FUNCTION(execK2_SetPropertyValue);

private:
	UE::MVVM::FMVVMFieldNotificationDelegates NotificationDelegates;
};

#undef UE_API
