// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/MVVMBindingName.h"

#include "MVVMAvailableBinding.generated.h"


USTRUCT(BlueprintType)
struct FMVVMAvailableBinding
{
	GENERATED_BODY()

public:
	FMVVMAvailableBinding() = default;

	explicit FMVVMAvailableBinding(FMVVMBindingName InBindingName, bool bInReadable, bool bInWritable, bool bInHasNotify)
		: BindingName(InBindingName)
		, bIsReadable(bInReadable)
		, bIsWritable(bInWritable)
		, bHasNotify(bInHasNotify)
	{

	}

public:
	FMVVMBindingName GetBindingName() const
	{
		return BindingName;
	}

	bool HasNotify() const
	{
		return bHasNotify;
	}

	bool IsReadable() const
	{
		return bIsReadable;
	}

	bool IsWritable() const
	{
		return bIsWritable;
	}

	bool IsValid() const
	{
		return BindingName.IsValid();
	}

	bool operator==(const FMVVMAvailableBinding& Other) const 
	{
		return BindingName == Other.BindingName
			&& bIsReadable == Other.bIsReadable
			&& bIsWritable == Other.bIsWritable
			&& bHasNotify == Other.bHasNotify;
	}

	bool operator!=(const FMVVMAvailableBinding& Other) const
	{
		return !(this->operator==(Other));
	}

	friend int32 GetTypeHash(const FMVVMAvailableBinding& Value)
	{
		return GetTypeHash(Value.BindingName);
	}

private:
	UPROPERTY()
	FMVVMBindingName BindingName;

	UPROPERTY()
	bool bIsReadable = false;

	UPROPERTY()
	bool bIsWritable = false;

	UPROPERTY()
	bool bHasNotify = false;
};
