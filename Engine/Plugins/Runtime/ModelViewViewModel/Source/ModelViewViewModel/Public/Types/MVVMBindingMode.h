// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMBindingMode.generated.h"


/** */
UENUM()
enum class EMVVMBindingMode : uint8
{
	OneTimeToDestination = 0,
	OneWayToDestination,
	TwoWay,
	OneTimeToSource UMETA(Hidden),
	OneWayToSource,
};


namespace UE::MVVM
{
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsForwardBinding(EMVVMBindingMode Mode);
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsBackwardBinding(EMVVMBindingMode Mode);
	[[nodiscard]] MODELVIEWVIEWMODEL_API bool IsOneTimeBinding(EMVVMBindingMode Mode);
}
