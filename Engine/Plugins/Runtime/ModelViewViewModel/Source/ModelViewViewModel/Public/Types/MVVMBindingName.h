// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MVVMBindingName.generated.h"

USTRUCT(BlueprintType)
struct FMVVMBindingName
{
	GENERATED_BODY()

public:
	FMVVMBindingName() = default;

	explicit FMVVMBindingName(FName InBindingName)
		: BindingName(InBindingName)
	{

	}

public:
	bool IsValid() const
	{
		return !BindingName.IsNone();
	}

	FName ToName() const
	{
		return BindingName;
	}

	FString ToString() const
	{
		return BindingName.ToString();
	}

	bool operator== (const FMVVMBindingName& Other) const
	{
		return BindingName == Other.BindingName;
	}

	bool operator!= (const FMVVMBindingName& Other) const
	{
		return BindingName != Other.BindingName;
	}


	friend int32 GetTypeHash(const FMVVMBindingName& Value)
	{
		return GetTypeHash(Value.BindingName);
	}

private:
	//todo this should be and make helper function to build the name in BP with a picker
	UPROPERTY(EditAnywhere, Category = "Viewmodel")
	FName BindingName;
};
