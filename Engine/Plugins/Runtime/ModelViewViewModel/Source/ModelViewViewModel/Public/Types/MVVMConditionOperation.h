// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMConditionOperation.generated.h"


/** */
UENUM(BlueprintType)
enum class EMVVMConditionOperation : uint8
{
	Equal,
	NotEqual,
	MoreThan,
	MoreThanOrEqual,
	LessThan,
	LessThanOrEqual,
	BetweenInclusive,
	BetweenExclusive
};

MODELVIEWVIEWMODEL_API const TCHAR* LexToString(EMVVMConditionOperation Enum);
