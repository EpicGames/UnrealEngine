// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMEventField.generated.h"

/**
 * Generic structure to notify when an event occurs.
 * 
 * class UMyViewmodel : public UMVVMViewModelBase
 * {
 *   UPROPERTY(FieldNotify)
 *   FMVVMEventField SomeEvent;
 *
 *   void OnSomeEvent()
 *   {
 *     UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SomeEvent);
 *   }
 * };
};
 */
USTRUCT(BlueprintType)
struct FMVVMEventField
{
	GENERATED_BODY()
};
