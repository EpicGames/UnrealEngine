// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMExecutionMode.generated.h"


/** */
UENUM()
enum class EMVVMExecutionMode : uint8
{
	/** Execute the binding as soon as the source value changes. */
	Immediate = 0,
	/** Execute the binding at the end of the frame before drawing when the source value changes. */
	Delayed = 1,
	/** Always execute the binding at the end of the frame. */
	Tick = 2,
	/** When the binding can be triggered from multiple fields, use Delayed. Else, uses Immediate. */
	DelayedWhenSharedElseImmediate = 3 UMETA(DisplayName="Auto"),
};
