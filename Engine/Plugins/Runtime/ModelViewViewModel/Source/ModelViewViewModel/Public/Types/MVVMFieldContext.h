// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/MVVMFieldVariant.h"
#include "Types/MVVMObjectVariant.h"


namespace UE::MVVM
{

	/** */
	struct FFieldContext : private TPair<UE::MVVM::FObjectVariant, UE::MVVM::FMVVMFieldVariant>
	{
	public:
		using TPair<UE::MVVM::FObjectVariant, UE::MVVM::FMVVMFieldVariant>::TPair;

		const UE::MVVM::FObjectVariant& GetObjectVariant() const
		{
			return Get<0>();
		}
		const UE::MVVM::FMVVMFieldVariant& GetFieldVariant() const
		{
			return Get<1>();
		}
	};

	/** */
	struct FConstFieldContext : private TPair<UE::MVVM::FConstObjectVariant, UE::MVVM::FMVVMConstFieldVariant>
	{
	public:
		using TPair<UE::MVVM::FConstObjectVariant, UE::MVVM::FMVVMConstFieldVariant>::TPair;

		const UE::MVVM::FConstObjectVariant& GetObjectVariant() const
		{
			return Get<0>();
		}
		const UE::MVVM::FMVVMConstFieldVariant& GetFieldVariant() const
		{
			return Get<1>();
		}
	};

} //namespace
