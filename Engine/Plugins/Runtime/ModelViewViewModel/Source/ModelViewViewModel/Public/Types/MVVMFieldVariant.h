// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Field.h"

class FProperty;
class UFunction;

namespace UE::MVVM
{

	/**
	 * Represents a possibly-const binding to either a UFunction or FProperty
	 */
	template<bool bConst>
	struct TMVVMFieldVariant
	{
	private:
		friend TMVVMFieldVariant<!bConst>;
		using FunctionType = std::conditional_t<bConst, const UFunction, UFunction>;
		using PropertyType = std::conditional_t<bConst, const FProperty, FProperty>;
		union VariantType
		{
			FunctionType* Function = nullptr;
			PropertyType* Property;
		};
		enum class EVariantType : uint8
		{
			Empty,
			Function,
			Property,
		};

	public:
		TMVVMFieldVariant() = default;

		explicit TMVVMFieldVariant(FFieldVariant InVariant)
		{
			if (PropertyType* Property = InVariant.Get<PropertyType>())
			{
				Binding.Property = Property;
				Type = EVariantType::Property;
			}
			else if (FunctionType* Function = InVariant.Get<FunctionType>())
			{
				Binding.Function = Function;
				Type = EVariantType::Function;
			}
		}

		explicit TMVVMFieldVariant(PropertyType* InValue)
		{
			Binding.Property = InValue;
			Type = EVariantType::Property;
		}

		explicit TMVVMFieldVariant(FunctionType* InValue)
		{
			Binding.Function = InValue;
			Type = EVariantType::Function;
		}

		[[nodiscard]] bool IsProperty() const
		{
			return Type == EVariantType::Property;
		}

		[[nodiscard]] PropertyType* GetProperty() const
		{
			check(Type == EVariantType::Property);
			return Binding.Property;
		}

		void SetProperty(PropertyType* InValue)
		{
			Binding.Property = InValue;
			Type = EVariantType::Property;
		}

		[[nodiscard]] bool IsFunction() const
		{
			return Type == EVariantType::Function;
		}

		[[nodiscard]] FunctionType* GetFunction() const
		{
			check(Type == EVariantType::Function);
			return Binding.Function;
		}

		void SetFunction(FunctionType* InValue)
		{
			Binding.Function = InValue;
			Type = EVariantType::Function;
		}

		[[nodiscard]] FName GetName() const
		{
			if (Type == EVariantType::Property)
			{
				return Binding.Property ? Binding.Property->GetFName() : FName();
			}
			else if (Type == EVariantType::Function)
			{
				return Binding.Function ? Binding.Function->GetFName() : FName();
			}
			return FName();
		}

		[[nodiscard]] UStruct* GetOwner() const
		{
			if (Type == EVariantType::Property)
			{
				return Binding.Property ? Binding.Property->GetOwnerStruct() : nullptr;
			}
			else if (Type == EVariantType::Function)
			{
				return Binding.Function ? Binding.Function->GetOwnerClass() : nullptr;
			}
			return nullptr;
		}

		UE_DEPRECATED(5.8, "TMVVMFieldVariant::HasAnyFlags is deprecated, it is only relevant for functions. Consider using GetFunction()->HasAnyFlags() after checking IsFunction() instead.")
		[[nodiscard]] bool HasAnyFlags(EObjectFlags InFlags) const
		{
			if (Type == EVariantType::Function)
			{
				return Binding.Function ? Binding.Function->HasAnyFlags(InFlags) : false;
			}
			return false;
		}

		[[nodiscard]] bool IsValid() const
		{
			if (Type == EVariantType::Property)
			{
				return Binding.Property != nullptr;
			}
			else if (Type == EVariantType::Function)
			{
				return Binding.Function != nullptr;
			}
			return false;
		}

		[[nodiscard]] bool IsEmpty() const
		{
			return Type == EVariantType::Empty || Binding.Function == nullptr;
		}

		void Reset()
		{
			Binding.Function = nullptr;
			Type = EVariantType::Empty;
		}

		template<bool bOtherConst>
		bool operator==(const TMVVMFieldVariant<bOtherConst>& B) const
		{
			if (Type != B.Type)
			{
				return false;
			}
			if (IsEmpty())
			{
				return true;
			}
			return Binding.Function == B.Binding.Function;
		}

		template<bool bOtherConst>
		bool operator!=(const TMVVMFieldVariant<bOtherConst>& B) const
		{
			return !(*this == B);
		}

		friend int32 GetTypeHash(const TMVVMFieldVariant<bConst>& Variant)
		{
			switch (Variant.Type)
			{
				case EVariantType::Function:
					return GetTypeHash(Variant.Binding.Function);
				case EVariantType::Property:
					return GetTypeHash(Variant.Binding.Property);
			}
			return 0;
		}

	private:
		VariantType Binding;
		EVariantType Type = EVariantType::Empty;
	};

	/** */
	struct FMVVMFieldVariant : public TMVVMFieldVariant<false>
	{
	public:
		using TMVVMFieldVariant<false>::TMVVMFieldVariant;
		using TMVVMFieldVariant<false>::operator==;
	};

	/** */
	struct FMVVMConstFieldVariant : public TMVVMFieldVariant<true>
	{
	public:
		using TMVVMFieldVariant<true>::TMVVMFieldVariant;
		using TMVVMFieldVariant<true>::operator==;

		FMVVMConstFieldVariant(const FMVVMFieldVariant& OtherVariant)
		{
			if (OtherVariant.IsProperty())
			{
				SetProperty(OtherVariant.GetProperty());
			}
			else if (OtherVariant.IsFunction())
			{
				SetFunction(OtherVariant.GetFunction());
			}
		}

		FMVVMConstFieldVariant& operator=(const FMVVMFieldVariant& OtherVariant)
		{
			*this = FMVVMConstFieldVariant(OtherVariant);
			return *this;
		}
	};

} //namespace
