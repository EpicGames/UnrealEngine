// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"

namespace UE::MVVM { template <bool bConst> struct TObjectVariant; }

namespace UE::MVVM
{

	/** */
	template<bool bConst>
	struct TFunctionContext
	{
		using ObjectType = std::conditional_t<bConst, const UObject*, UObject*>;
		using FunctionType = std::conditional_t<bConst, const UFunction*, UFunction*>;

	public:
		TFunctionContext() = default;
		TFunctionContext(ObjectType InObject, FunctionType InFunction)
			: ObjectPtr(InObject)
			, FunctionPtr(InFunction)
		{
			check(ObjectPtr);
			check(InFunction);
			check(ObjectPtr->GetClass()->IsChildOf(FunctionPtr->GetOuterUClass()));
		}

	public:
		[[nodiscard]] ObjectType GetObject() const
		{
			return ObjectPtr;
		}

		[[nodiscard]] FunctionType GetFunction() const
		{
			return FunctionPtr;
		}

		void Reset()
		{
			*this = TFunctionContext();
		}

		template<bool bOtherConst>
		bool operator==(const TFunctionContext<bOtherConst>& Other) const
		{
			return ObjectPtr == Other.ObjectPtr && FunctionPtr == Other.FunctionPtr;
		}

		template<bool bOtherConst>
		bool operator!=(const TObjectVariant<bOtherConst>& Other) const
		{
			return !(*this == Other);
		}

	private:
		ObjectType ObjectPtr = nullptr;
		FunctionType FunctionPtr = nullptr;
	};

	/** */
	struct FFunctionContext : public TFunctionContext<false>
	{
	public:
		using TFunctionContext<false>::TFunctionContext;
		using TFunctionContext<false>::operator==;

		static FFunctionContext MakeStaticFunction(FunctionType InFunction)
		{
			check(InFunction && InFunction->HasAnyFunctionFlags(FUNC_Static));
			return FFunctionContext(InFunction->GetOuterUClass()->GetDefaultObject(), InFunction);
		}

	};

	/** */
	struct FConstFunctionContext : public TFunctionContext<true>
	{
	public:
		using TFunctionContext<true>::TFunctionContext;
		using TFunctionContext<true>::operator==;

		FConstFunctionContext(const FFunctionContext& Other)
			: TFunctionContext<true>(Other.GetObject(), Other.GetFunction())
		{			
		}

		static FConstFunctionContext MakeStaticFunction(FunctionType InFunction)
		{
			check(InFunction && InFunction->HasAnyFunctionFlags(FUNC_Static));
			return FConstFunctionContext(InFunction->GetOuterUClass()->GetDefaultObject(), InFunction);
		}

		FConstFunctionContext& operator=(const FFunctionContext& Other)
		{
			*this = FConstFunctionContext(Other);
			return *this;
		}
	};

} //namespace
