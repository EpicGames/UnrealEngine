// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TVariant.h"
#include "Misc/TVariantMeta.h"
#include "Templates/Casts.h"
#include "UObject/WeakObjectPtr.h"


namespace UE::MVVM
{

	/** */
	template<bool bConst>
	struct TObjectVariant
	{
	private:
		friend TObjectVariant<!bConst>;

		using ContainerType = std::conditional_t<bConst, const void*, void*>;

		struct FScriptStructStackedContainer
		{
			FScriptStructStackedContainer() = default;
			FScriptStructStackedContainer(const UScriptStruct* InScriptStruct, ContainerType InContainer)
				: ScriptStruct(InScriptStruct), Container(InContainer)
			{}
			const UScriptStruct* ScriptStruct = nullptr;
			ContainerType Container = nullptr;
		};

		struct FScriptStructAllocatedContainer
		{
			FScriptStructAllocatedContainer() = default;
			FScriptStructAllocatedContainer(const FScriptStructAllocatedContainer&) = delete;
			FScriptStructAllocatedContainer& operator=(const FScriptStructAllocatedContainer&) = delete;

			FScriptStructAllocatedContainer(const UScriptStruct* InScriptStruct)
				: ScriptStruct(InScriptStruct), Container(nullptr)
			{
				if (InScriptStruct)
				{
					Container = FMemory::Malloc(InScriptStruct->GetStructureSize() ? InScriptStruct->GetStructureSize() : 1);
					InScriptStruct->InitializeStruct(Container);
				}
			}
			~FScriptStructAllocatedContainer()
			{
				if (Container)
				{
					const UScriptStruct* ScriptStructPtr = ScriptStruct.Get();
					if (ensureAlwaysMsgf(ScriptStructPtr, TEXT("The script struct is null, that may cause resource leak.")))
					{
						ScriptStruct->DestroyStruct(Container);
					}
					FMemory::Free(Container);
				}
			}
			TWeakObjectPtr<const UScriptStruct> ScriptStruct = nullptr;
			ContainerType Container = nullptr;
		};

		using UScriptStructType = FScriptStructStackedContainer;
		using UObjectType = std::conditional_t<bConst, const UObject, UObject>;
		using VariantType = TVariant<FEmptyVariantState, UScriptStructType, UObjectType*>;

	public:
		TObjectVariant() = default;

		explicit TObjectVariant(UObjectType* InValue)
		{
			Binding = VariantType(TInPlaceType<UObjectType*>(), InValue);
		}

		explicit TObjectVariant(const UScriptStruct* InScriptStruct, ContainerType InValue)
		{
			Binding = VariantType(TInPlaceType<UScriptStructType>(), InScriptStruct, InValue);
		}

		[[nodiscard]] bool IsUObject() const
		{
			return Binding.template IsType<UObjectType*>();
		}

		[[nodiscard]] UObjectType* GetUObject() const
		{
			return Binding.template Get<UObjectType*>();
		}

		[[nodiscard]] const UClass* GetUClass() const
		{
			UObjectType* Instance = Binding.template Get<UObjectType*>();
			return Instance ? Instance->GetClass() : nullptr;
		}

		void SetUObject(UObjectType* InValue)
		{
			Binding.template Set<UObjectType*>(InValue);
		}

		[[nodiscard]] bool IsUScriptStruct() const
		{
			return Binding.template IsType<UScriptStructType>();
		}

		[[nodiscard]] ContainerType GetUScriptStructData() const
		{
			return Binding.template Get<UScriptStructType>().Container;
		}

		[[nodiscard]] const UScriptStruct* GetUScriptStruct() const
		{
			return Binding.template Get<UScriptStructType>().ScriptStruct;
		}

		void SetUScriptStruct(const UScriptStruct* InScriptStruct, ContainerType InValue)
		{
			Binding.template Set<UScriptStructType>(FScriptStructStackedContainer(InScriptStruct, InValue));
		}

		[[nodiscard]] ContainerType GetData() const
		{
			return IsUObject() ? reinterpret_cast<ContainerType>(GetUObject()) : IsUScriptStruct() ? GetUScriptStructData() : nullptr;
		}

		[[nodiscard]] const UStruct* GetOwner() const
		{
			if (IsUObject())
			{
				return GetUClass();
			}
			else if (IsUScriptStruct())
			{
				return Binding.template Get<UScriptStructType>().ScriptStruct;
			}
			return nullptr;
		}

		[[nodiscard]] bool IsEmpty() const
		{
			return Binding.template IsType<FEmptyVariantState>();
		}

		[[nodiscard]] bool IsNull() const
		{
			return IsUObject() ? GetUObject() == nullptr : IsUScriptStruct() ? GetUScriptStructData() == nullptr : true;
		}

		void Reset()
		{
			Binding = VariantType();
		}

		template<bool bOtherConst>
		bool operator==(const TObjectVariant<bOtherConst>& B) const
		{
			if (Binding.GetIndex() != B.Binding.GetIndex())
			{
				return false;
			}
			if (IsEmpty())
			{
				return true;
			}
			if (IsUObject())
			{
				return GetUObject() == B.GetUObject();
			}
			return GetUScriptStructData() == B.GetUScriptStructData();
		}

		template<bool bOtherConst>
		bool operator!=(const TObjectVariant<bOtherConst>& B) const
		{
			return !(*this == B);
		}

		friend int32 GetTypeHash(const TObjectVariant<bConst>& Variant)
		{
			if (Variant.IsUObject())
			{
				return GetTypeHash(Variant.GetUObject());
			}
			if (Variant.IsUScriptStruct())
			{
				return GetTypeHash(Variant.GetUScriptStructData());
			}
			return 0;
		}

	private:
		VariantType Binding;
	};

	/** */
	struct FObjectVariant : public TObjectVariant<false>
	{
	public:
		using TObjectVariant<false>::TObjectVariant;
		using TObjectVariant<false>::operator==;
	};

	/** */
	struct FConstObjectVariant : public TObjectVariant<true>
	{
	public:
		using TObjectVariant<true>::TObjectVariant;
		using TObjectVariant<true>::operator==;

		FConstObjectVariant(const FObjectVariant& OtherVariant)
		{
			if (OtherVariant.IsUObject())
			{
				SetUObject(OtherVariant.GetUObject());
			}
			else if (OtherVariant.IsUScriptStruct())
			{
				SetUScriptStruct(CastChecked<UScriptStruct>(OtherVariant.GetOwner()), OtherVariant.GetUScriptStructData());
			}
		}

		FConstObjectVariant& operator=(const FObjectVariant& OtherVariant)
		{
			*this = FConstObjectVariant(OtherVariant);
			return *this;
		}
	};

} //namespace
