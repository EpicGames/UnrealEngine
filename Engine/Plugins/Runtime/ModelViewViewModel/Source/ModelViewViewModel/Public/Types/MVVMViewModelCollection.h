// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/MVVMViewModelContext.h"
#include "Types/MVVMViewModelContextInstance.h" // IWYU pragma: keep
#include "MVVMViewModelCollection.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

class UMVVMViewModelBase;

/** */
USTRUCT()
struct FMVVMViewModelCollection
{
	GENERATED_BODY()

public:
	UE_API UMVVMViewModelBase* FindViewModelInstance(FMVVMViewModelContext Context) const;
	UE_API UMVVMViewModelBase* FindFirstViewModelInstanceOfType(const TSubclassOf<UMVVMViewModelBase>& ViewModelClass) const;

	UE_API bool AddInstance(FMVVMViewModelContext Context, UMVVMViewModelBase* ViewModel);
	UE_API bool RemoveInstance(FMVVMViewModelContext Context);
	UE_API int32 RemoveAllInstances(UMVVMViewModelBase* ViewModel);

	UE_API void Reset();

	FSimpleMulticastDelegate& OnCollectionChanged()
	{
		return OnCollectionChangedDelegate;
	}

private:
	UPROPERTY()
	mutable TArray<FMVVMViewModelContextInstance> ViewModelInstances;

	FSimpleMulticastDelegate OnCollectionChangedDelegate;
};


/** */
UCLASS(MinimalAPI, meta = (DisplayName = "MVVM View Model Collection Object"))
class UMVVMViewModelCollectionObject : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UMVVMViewModelBase* FindViewModelInstance(FMVVMViewModelContext Context) const
	{
		return ViewModelCollection.FindViewModelInstance(Context);
	}

	/**
	 * Finds a View Model of the given type.
	 * If the collection contains multiple instances of the same type then this will return the first one found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UMVVMViewModelBase* FindFirstViewModelInstanceOfType(const TSubclassOf<UMVVMViewModelBase>& ViewModelClass) const
	{
		return ViewModelCollection.FindFirstViewModelInstanceOfType(ViewModelClass);
	}
	
	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	bool AddViewModelInstance(FMVVMViewModelContext Context, UMVVMViewModelBase* ViewModel)
	{
		return ViewModelCollection.AddInstance(Context, ViewModel);
	}

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	bool RemoveViewModel(FMVVMViewModelContext Context)
	{
		return ViewModelCollection.RemoveInstance(Context);
	}

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	int32 RemoveAllViewModelInstance(UMVVMViewModelBase* ViewModel)
	{
		return ViewModelCollection.RemoveAllInstances(ViewModel);
	}

	FSimpleMulticastDelegate& OnCollectionChanged()
	{
		return ViewModelCollection.OnCollectionChanged();
	}

private:
	UPROPERTY(Transient)
	FMVVMViewModelCollection ViewModelCollection;
};

#undef UE_API
