// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"


#include "MVVMViewModelContext.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

class UMVVMViewModelBase;

/** */
USTRUCT(BlueprintType)
struct FMVVMViewModelContext
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category="Viewmodel")
	TSubclassOf<UMVVMViewModelBase> ContextClass;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category="Viewmodel")
	FName ContextName;

public:
	UE_API bool IsValid() const;
	UE_API bool operator== (const FMVVMViewModelContext& Other) const;
	UE_API bool IsCompatibleWith(const FMVVMViewModelContext& Other) const;
	UE_API bool IsCompatibleWith(const TSubclassOf<UMVVMViewModelBase>& OtherClass) const;
	UE_API bool IsCompatibleWith(const UMVVMViewModelBase* Other) const;
};

#undef UE_API
