// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMViewModelBase.h"
#include "MVVMViewModelContext.h"

#include "MVVMViewModelContextInstance.generated.h"

class UMVVMViewModelBase;

/** */
USTRUCT()
struct FMVVMViewModelContextInstance
{
	GENERATED_BODY()

public:
	FMVVMViewModelContextInstance() = default;
	FMVVMViewModelContextInstance(FMVVMViewModelContext InContext, UMVVMViewModelBase* InInstance)
		: Context(InContext)
		, Instance(InInstance)
	{
		if (!Context.IsCompatibleWith(Instance))
		{
			Instance = nullptr;
		}
	}

public:
	bool IsValid() const
	{
		return Context.IsValid() && Instance.Get();
	}

	FMVVMViewModelContext GetContext() const
	{
		return Context;
	}

	UMVVMViewModelBase* GetViewModel()
	{
		return Instance.Get();
	}

	const UMVVMViewModelBase* GetViewModel() const
	{
		return Instance.Get();
	}

private:
	UPROPERTY()
	FMVVMViewModelContext Context;

	UPROPERTY()
	TObjectPtr<UMVVMViewModelBase> Instance = nullptr;
};
