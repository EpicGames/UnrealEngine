// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Extensions/UserWidgetExtension.h"
#include "View/MVVMViewTypes.h"

#include "MVVMView.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

class INotifyFieldValueChanged;
namespace UE::FieldNotification { struct FFieldId; }
template <typename InterfaceType> class TScriptInterface;

class UMVVMViewClass;
class UMVVMViewClassExtension;
class UMVVMViewExtension;
struct FMVVMViewClass_Binding;
struct FMVVMViewClass_Event;
struct FMVVMViewClass_Source;
struct FMVVMViewClass_SourceBinding;
struct FMVVMViewClass_SourceKey;
struct FMVVMViewClass_SourceCondition;

/**
 * Instance FMVVMViewClass_Source for the UUserWdiget
 */
USTRUCT()
struct FMVVMView_Source
{
	GENERATED_BODY()

	/** The source object. The source implement the INotifyFieldValueChanged interface. */
	UPROPERTY(VisibleAnywhere, Category = "View")
	TObjectPtr<UObject> Source;

	/** The key of this source in the ViewClass. */
	UPROPERTY(VisibleAnywhere, Category = "View")
	FMVVMViewClass_SourceKey ClassKey;

	/** Number of bindings connected to the source. */
	UPROPERTY(VisibleAnywhere, Category = "View")
	int32 RegisteredCount = 0;

	/** The source is created. */
	UPROPERTY(VisibleAnywhere, Category = "View")
	bool bSourceInitialized = false;

	/** The source bindings are initialized. */
	UPROPERTY(VisibleAnywhere, Category = "View")
	bool bBindingsInitialized = false;

	/** The source was set manually via SetViewModel. */
	UPROPERTY(VisibleAnywhere, Category = "View")
	bool bSetManually = false;

	/** The source was set to a UserWidget property. */
	UPROPERTY(VisibleAnywhere, Category = "View")
	bool bAssignedToUserWidgetProperty = false;
};

/**
 * Instance UMVVMClassExtension_View for the UUserWidget
 */
UCLASS(MinimalAPI, Transient, DisplayName="MVVM View")
class UMVVMView : public UUserWidgetExtension
{
	GENERATED_BODY()

public:
	UE_API void ConstructView(const UMVVMViewClass* InGeneratedViewClass);

	//~ Begin UUserWidgetExtension implementation
	//virtual void Initialize() override;
	UE_API virtual void Construct() override;
	UE_API virtual void Destruct() override;
	//~ End UUserWidgetExtension implementation

	/** Is the view in a valid constructed state? */
	bool IsConstructed() const
	{
		return bConstructed;
	}

	/**
	 * Initialize the sources if they are not already initialized.
	 * Initializing the sources will also initialize the bindings if the option bInitializeBindingsOnConstruct is enabled.
	 * @note A sources can be a viewmodel or any other object used by a binding.
	 */
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API void InitializeSources();
	/**
	 * Uninitialize the sources if they are already initialized.
	 * It will uninitialized the bindings.
	 */
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API void UninitializeSources();
	/** The sources were initialized, manually or automatically. */
	UFUNCTION(BlueprintPure, Category = "View")
	bool AreSourcesInitialized() const
	{
		return bSourcesInitialized;
	}

	/**
	 * Initialize the bindings if they are not already initialized.
	 * Initializing the bindings will execute them.
	 */
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API void InitializeBindings();
	/** Uninitialize the bindings if they are already initialized. */
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API void UninitializeBindings();
	/** The bindings were initialized, manually or automatically. */
	UFUNCTION(BlueprintPure, Category = "View")
	bool AreBindingsInitialized() const
	{
		return bBindingsInitialized;
	}
	
	/** Initialize the events if they are not already initialized. */
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API void InitializeEvents();
	/** Uninitialize the events if they are already initialized. */
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API void UninitializeEvents();
	/** The events were initialized, manually or automatically. */
	UFUNCTION(BlueprintPure, Category = "View")
	bool AreEventsInitialized() const
	{
		return bEventsInitialized;
	}

	/** The shared information for each instance of the view. */
	const UMVVMViewClass* GetViewClass() const
	{
		return GeneratedViewClass;
	}

	/** The list of the sources needed by the view. */
	const TArrayView<const FMVVMView_Source> GetSources() const
	{
		return Sources;
	}

	/** The source used by the view. */
	const FMVVMView_Source& GetSource(FMVVMView_SourceKey Key) const
	{
		check(Sources.IsValidIndex(Key.GetIndex()));
		return Sources[Key.GetIndex()];
	}

	UE_API void ExecuteDelayedBinding(const FMVVMViewClass_BindingKey& DelayedBinding) const;
	UE_API void ExecuteTickBindings() const;

	/** Find and return the viewmodel with the specified name. */
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API TScriptInterface<INotifyFieldValueChanged> GetViewModel(FName ViewModelName) const;

	/**
	 * Set the viewmodel of the specified name.
	 * The viewmodel needs to be settable and the type should match (child of the defined viewmodel).
	 * If the view is initialized, all bindings that uses that viewmodel will be re-executed with the new viewmodel instance.
	 */
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API bool SetViewModel(FName ViewModelName, TScriptInterface<INotifyFieldValueChanged> ViewModel);

	/**
	 * Set the first viewmodel matching the exact specified type. If none is found, set the first viewmodel matching a child of the specified type
	 * The viewmodel needs to be settable and it should have a valid name.
	 * If the view is initialized, all bindings that uses that viewmodel will be re-executed with the new viewmodel instance.
	 */
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API bool SetViewModelByClass(TScriptInterface<INotifyFieldValueChanged> NewValue);

	/**
	 * Execute all the bindings that use the viewmodel.
	 */
	UFUNCTION(BlueprintCallable, Category = "View")
	UE_API bool ExecuteViewModelBindings(FName ViewModelName);

private:
	/**
	 * Set the viewmodel of the specified name.
	 * The viewmodel needs to be settable and the type should match (child of the defined viewmodel).
	 * If the view is initialized, all bindings that uses that viewmodel will be re-executed with the new viewmodel instance.
	 */
	UFUNCTION(BlueprintCallable, Category = "View", meta=(BlueprintInternalUseOnly="true"))
	UE_API bool AreSourcesValidForEvent(int32 EventKey) const;

	/**
	 * Checks if all source bindings (Src / Dst objects) are valid for a binding
	 * Currently used to verify if after effects of async conversion functions are safe to trigger.
	 */
	UFUNCTION(BlueprintCallable, Category = "View", meta = (BlueprintInternalUseOnly = "true"))
	UE_API bool AreSourcesValidForBinding(int32 BindingKey) const;

private:
	//~ Source
	UE_API void InitializeSource(FMVVMView_SourceKey SourceKey);
	UE_API void InitializeSourceInternal(UObject* NewSource, FMVVMViewClass_SourceKey SourceKey, const FMVVMViewClass_Source& ClassSource, FMVVMView_Source& ViewSource);
	UE_API void UninitializeSource(FMVVMView_SourceKey SourceKey);
	UE_API bool SetSourceInternal(FMVVMViewClass_SourceKey SourceKey, TScriptInterface<INotifyFieldValueChanged> ViewModel, bool bForDynamicSource);

	void InitializeSourceParentBinding(FMVVMView_SourceKey SourceKey);
	void UninitializeSourceParentBinding(FMVVMView_SourceKey SourceKey);

	//~ Bindings
	UE_API void InitializeSourceBindings(FMVVMView_SourceKey SourceKey, bool bRunAllBindings);
	UE_API void InitializeSourceBindingsCommon();
	UE_API void UninitializeSourceBindings(FMVVMViewClass_SourceKey SourceKey, const FMVVMViewClass_Source& ClassSource, FMVVMView_Source& ViewSource);
	UE_API void HandledLibraryBindingValueChanged(UObject* InSource, UE::FieldNotification::FFieldId InFieldId, FMVVMView_SourceKey SourceKey);
	UE_API void ExecuteBindingInternal(const FMVVMViewClass_SourceBinding& SourceBinding) const;
	UE_API void ExecuteBindingImmediately(const FMVVMViewClass_Binding& ClassBinding, FMVVMViewClass_BindingKey KeyForLog) const;
	UE_API void ExecuteViewModelBindingsInternal(FMVVMViewClass_SourceKey SourceKey);

	UE_API void ExecuteConditionInternal(const FMVVMViewClass_SourceCondition& SourceCondition) const;

	//~ evaluate source
	UE_API bool EvaluateSource(FMVVMViewClass_SourceKey SourceIndex);
	UE_API void HandleViewModelCollectionChanged();

	//~ events
	UE_API void BindEvent(const FMVVMViewClass_Event& ClassItem, FMVVMViewClass_EventKey KeyForLog);
	UE_API void UnbindEvent(int32 BoundEventIndex);
	UE_API void ReinitializeEvents(FMVVMViewClass_SourceKey SourceKey, UObject* PreviousValue, UObject* NewValue);
#if WITH_EDITOR
	void HandleBlueprintPreCompile(UBlueprint* Blueprint);
#endif

private:
	UPROPERTY(Transient)
	TObjectPtr<const UMVVMViewClass> GeneratedViewClass;

	UPROPERTY(VisibleAnywhere, Transient, Category = "View")
	TArray<FMVVMView_Source> Sources;

	struct FBoundEvent
	{
		FWeakObjectPtr Object;
		FName PropertyName;
		FMVVMViewClass_EventKey EventKey;
	};
	/** The event that are registered by the view to the sources. */
	TArray<FBoundEvent> BoundEvents;

	TMap<TPair<int32 /* Source Index */, int32 /* Field Index */>, FDelegateHandle> RegisteredBindings;

	UPROPERTY(VisibleAnywhere, Transient, Category = "View")
	TArray<TObjectPtr<UMVVMViewExtension>> Extensions;

	/** Bitfield that represents the valid sources. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "View")
	uint64 ValidSources = 0;

	/** The number of source has at least one binding that need to be ticked every frame. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "View")
	uint8 NumberOfSourceWithTickBinding = 0;

	/** Should log when a binding is executed. */
	UPROPERTY(EditAnywhere, Transient, Category = "View")
	bool bLogBinding = false;

	/** Is the Construct method called. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "View")
	bool bConstructed = false;
	
	/** Is the Initialize method called. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "View")
	bool bSourcesInitialized = false;

	/** Is the Initialize method called. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "View")
	bool bBindingsInitialized = false;
	
	/** Is the Initialize method called. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "View")
	bool bEventsInitialized = false;

	/** Is the view is registered to the binding subsystem for tick. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "View")
	bool bHasDefaultTickBinding = false;

#if WITH_EDITORONLY_DATA
	FDelegateHandle BlueprintPreCompileHandle;
#endif
};

#undef UE_API
