// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MVVMCompiledBindingLibrary.h"
#include "Extensions/WidgetBlueprintGeneratedClassExtension.h"
#include "Types/MVVMExecutionMode.h"
#include "Types/MVVMViewModelContext.h"
#include "View/MVVMViewTypes.h"

#include "UObject/Package.h"
#include "MVVMViewClass.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API


class UMVVMView;
class UMVVMViewClass;
class UMVVMViewClassExtension;
class UMVVMViewModelContextResolver;
class UUserWidget;

namespace UE::MVVM::Private { struct FMVVMViewBlueprintCompiler; }


/**
 * A structure to identify the Binding and the associated FieldId
 */
USTRUCT()
struct FMVVMViewClass_SourceBinding
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;

public:
	/**
	 * The id for the FieldId on the source.
	 * Valid when it is OneWay or when we need to register to the FieldNotify system.
	 */
	FFieldNotificationId GetFieldId() const
	{
		return FieldId;
	}

	/** The key to identify a binding in the view class. */
	FMVVMViewClass_BindingKey GetBindingKey() const
	{
		return BindingKey;
	}

	/** @return true if this Binding should be executed at initialization. */
	bool ExecuteAtInitialization() const
	{
		return (Flags & (uint8)EFlags::ExecuteAtInitialization) != 0;
	}

private:
	UPROPERTY(VisibleAnywhere, Category = "View")
	FFieldNotificationId FieldId;

	UPROPERTY(VisibleAnywhere, Category = "View")
	FMVVMViewClass_BindingKey BindingKey;

	enum class EFlags : uint8
	{
		None = 0,
		ExecuteAtInitialization = 1 << 0,
	};

	UPROPERTY(VisibleAnywhere, Category = "View")
	uint8 Flags = (uint8)EFlags::None;
};

/**
 * A structure to identify the Condition and the associated FieldId
 */
USTRUCT()
struct FMVVMViewClass_SourceCondition
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;

public:
	/**
	 * The id for the FieldId on the source.
	 * Valid when it is OneWay or when we need to register to the FieldNotify system.
	 */
	FFieldNotificationId GetFieldId() const
	{
		return FieldId;
	}

	/** The key to identify a binding in the view class. */
	FMVVMViewClass_ConditionKey GetConditionKey() const
	{
		return ConditionKey;
	}

private:
	UPROPERTY(VisibleAnywhere, Category = "View")
	FFieldNotificationId FieldId;

	UPROPERTY(VisibleAnywhere, Category = "View")
	FMVVMViewClass_ConditionKey ConditionKey;
};


/**
 * A compiled and shared binding for ViewModel<->View
 */
USTRUCT()
struct FMVVMViewClass_Binding
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;

public:
	/** @return the binding. From source to destination (if forward) or from destination to source (if backward). */
	const FMVVMVCompiledBinding& GetBinding() const
	{
		return Binding;
	}

	/** @return true if multiple field or source can trigger this binding. */
	bool IsShared() const
	{
		return (Flags & (uint8)EFlags::Shared) != 0;
	}

	/** @return true if this Binding should be executed once (at initialization) and when FieldId is broadcasted. */
	bool IsOneWay() const
	{
		return (Flags & (uint8)EFlags::OneWay) != 0;
	}

	/**
	 * A view binding may require more than one view sources to run the binding.
	 * A binding will not execute if any view source is invalid.
	 * It will not warn if the view source is make as optional.
	 */
	uint64 GetSources() const
	{
		return SourceBitField;
	}

	/** How/when the binding should be executed. */
	MODELVIEWVIEWMODEL_API EMVVMExecutionMode GetExecuteMode() const;

#if WITH_EDITORONLY_DATA
	/** Get the id of the BlueprintViewBinding. */
	FGuid GetEditorId() const
	{
		return EditorId;
	}
#endif

#if UE_WITH_MVVM_DEBUGGING
	struct FToStringArgs
	{
		bool bUseDisplayName = true;
		bool bAddFlags = true;
		bool bAddBindingId = true;
		bool bAddBindingFields = true;
		bool bAddSources = true;

		MODELVIEWVIEWMODEL_API static FToStringArgs Short();
		MODELVIEWVIEWMODEL_API static FToStringArgs All();
	};

	/** @return a human readable version of the source that can be use for debugging purposes. */
	MODELVIEWVIEWMODEL_API FString ToString(const UMVVMViewClass* ViewClass, FToStringArgs Args) const;
#endif

private:
	enum class EFlags : uint8
	{
		None = 0,
		OneWay = 1 << 0,
		Shared = 1 << 1,
		OverrideExecuteMode = 1 << 2,
		EnabledByDefault = 1 << 3,
	};

	UPROPERTY(VisibleAnywhere, Category = "View")
	FMVVMVCompiledBinding Binding;

	//~ FMVVMVCompiledBinding ends with a uint8, adding the byte variable here to help with packing.

	UPROPERTY(VisibleAnywhere, Category = "View")
	uint8 Flags = (uint8)EFlags::None;

	UPROPERTY(VisibleAnywhere, Category = "View")
	EMVVMExecutionMode ExecutionMode = EMVVMExecutionMode::Immediate;

	UPROPERTY(VisibleAnywhere, Category = "View")
	uint64 SourceBitField = 0;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FGuid EditorId;
#endif
};


/**
 * A binding to evaluate the source when something change in its path.
 */
USTRUCT()
struct FMVVMViewClass_EvaluateSource
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;

public:
	/** The id for the FieldId on the source on the parent that will trigger the evaluation. */
	FFieldNotificationId GetFieldId() const
	{
		return ParentFieldId;
	}
	
	/** The source that owns the source. */
	FMVVMViewClass_SourceKey GetParentSource() const
	{
		return ParentSource;
	}

	/** The source that needs to be evaluated. */
	FMVVMViewClass_SourceKey GetSource() const
	{
		return ToEvaluate;
	}

#if UE_WITH_MVVM_DEBUGGING
	struct FToStringArgs
	{
		bool bUseDisplayName = true;

		MODELVIEWVIEWMODEL_API static FToStringArgs Short();
		MODELVIEWVIEWMODEL_API static FToStringArgs All();
	};

	/** @return a human readable version of the source that can be use for debugging purposes. */
	MODELVIEWVIEWMODEL_API FString ToString(const UMVVMViewClass* ViewClass, FToStringArgs Args) const;
#endif

private:
	UPROPERTY(VisibleAnywhere, Category = "View")
	FFieldNotificationId ParentFieldId;

	UPROPERTY(VisibleAnywhere, Category = "View")
	FMVVMViewClass_SourceKey ParentSource;

	UPROPERTY(VisibleAnywhere, Category = "View")
	FMVVMViewClass_SourceKey ToEvaluate;
};


/**
 * Shared data to find or create a ViewModel at runtime.
 */
USTRUCT()
struct FMVVMViewClass_Source
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;

public:
	/** Get or create the instance used to register the bindings. */
	MODELVIEWVIEWMODEL_API UObject* GetOrCreateInstance(const UMVVMViewClass* ViewClass, UMVVMView* View, UUserWidget* UserWidget) const;

	/** The source is not needed anymore. */
	MODELVIEWVIEWMODEL_API void ReleaseInstance(UObject* ViewModel, const UMVVMView* View) const;

	/** Returns the viewmodel inside the global viewmodel collection. */
	MODELVIEWVIEWMODEL_API TScriptInterface<INotifyFieldValueChanged> GetGlobalCollectionViewModel(UUserWidget* UserWidget) const;

	// Helper struct to provide info about where context data is coming from
	struct FParentSourceInfo
	{
		// Parent view info will only be set if a parent widget was found that has the same class/name in it's view
		const UMVVMView* ParentView = nullptr;
		FMVVMView_SourceKey ParentSourceKey;

		// Parent User Widget will always be valid if Source is valid or if the ParentView is valid
		UUserWidget* ParentUserWidget = nullptr;

		// May be valid or not if the parent view is found, otherwise will only be valid if and external source was provided
		UObject* Source = nullptr;
	};

	/** Get the parent vm view that has a matching vm class source */
	FParentSourceInfo GetParentSupplyingSourceClass(UUserWidget* UserWidget) const;

	/** The expected class of the source. */
	UClass* GetSourceClass() const
	{
		return ExpectedSourceType.Get();
	}

	/** The source is the UserWidget. */
	bool IsUserWidget() const
	{
		return (Flags & (uint16)EFlags::SelfReference) != 0;
	}

	/** The source is a viewmodel added in the editor. */
	bool IsViewModel() const
	{
		return (Flags & (uint16)EFlags::IsViewModel) != 0;
	}

	/**
	 * The source is UserWidget's property.
	 * @return false if the source is from a long path or the source is the UserWidget.
	 */
	bool IsUserWidgetProperty() const
	{
		return (Flags & (uint16)EFlags::IsUserWidgetProperty) != 0;
	}
	
	/** The UserWidget's property need to be set/reset when the Source is created/release. */
	bool RequireSettingUserWidgetProperty() const
	{
		return (Flags & (uint16)EFlags::SetUserWidgetProperty) != 0;
	}

	/** Can be set at runtime from SetViewModel. */
	bool CanBeSet() const
	{
		return (Flags & (uint16)EFlags::CanBeSet) != 0;
	}

	/** The source "GetOrCreateInstance" can be reevaluated. */
	bool CanBeEvaluated() const
	{
		return (Flags & (uint16)EFlags::CanBeEvaluated) != 0;
	}
	
	/** The source has at least one evaluate binding. */
	bool HasEvaluateBindings() const
	{
		return (Flags & (uint16)EFlags::HasEvaluatedBindings) != 0;
	}
	
	/** The source has at least one evaluate binding. */
	bool RequireGlobalViewModelCollectionUpdate() const
	{
		return (Flags & (uint16)EFlags::GlobalViewModelCollectionUpdate) != 0;
	}

	/** When a source is manually set, then always execute that source the bindings. */
	bool AlwaysExecuteBindingsOnSetSource() const
	{
		return (Flags & (uint16)EFlags::AlwaysExecuteBindingsOnSetSource) != 0;
	}

	/** If we're going to lookup the vm from a parent view */
	bool IsParentContextReference() const
	{
		return (Flags & (uint16)EFlags::ParentContextReference) != 0;
	}
	
	/**
	 * The source GetOrCreateInstance can fail.
	 * The view will not warn if a binding can't execute because the source is invalid.
	 */
	bool IsOptional() const
	{
		return (Flags & (uint16)EFlags::IsOptional) != 0;
	}
	
	/** The source has at least one binding that need to be tick every frame. */
	bool HasTickBindings() const
	{
		return (Flags & (uint16)EFlags::HasTickBindings) != 0;
	}

	/** The name of the source. */
	FName GetName() const
	{
		return PropertyName;
	}

	/** The name of the UserWidget's property. */
	FName GetUserWidgetPropertyName() const
	{
		return IsUserWidgetProperty() ? PropertyName : FName();
	}

	/**
	 * FieldId owns by the source that we need to register to.
	 * Only contains id for OneWay bindings or Evaluate bindings.
	 */
	const TArrayView<const FMVVMViewClass_FieldId> GetFieldIds() const
	{
		return FieldToRegisterTo;
	}

	/** The list of bindings owns by the source. A binding can be owns by more than one source. */
	const TArrayView<const FMVVMViewClass_SourceBinding> GetBindings() const
	{
		return Bindings;
	}

	const TArrayView<const FMVVMViewClass_SourceCondition> GetConditions() const
	{
		return Conditions;
	}

#if UE_WITH_MVVM_DEBUGGING
	struct FToStringArgs
	{
		FMVVMViewClass_Binding::FToStringArgs Bindings;
		bool bUseDisplayName = true;
		bool bAddCreationMode = true;
		bool bAddFields = true;
		bool bAddBindings = true;
		bool bAddFlags = true;

		MODELVIEWVIEWMODEL_API static FToStringArgs Short();
		MODELVIEWVIEWMODEL_API static FToStringArgs All();
	};

	/** @return a human readable version of the source that can be use for debugging purposes. */
	MODELVIEWVIEWMODEL_API FString ToString(const UMVVMViewClass* ViewClass, FToStringArgs Args) const;
#endif

private:
	/** Class type to create a source at runtime. */
	UPROPERTY(VisibleAnywhere, Category = "View")
	TSubclassOf<UObject> ExpectedSourceType;

	/** The resolver to fetch the source at runtime. */
	UPROPERTY(VisibleAnywhere, Category = "View", Instanced)
	TObjectPtr<UMVVMViewModelContextResolver> Resolver = nullptr;

	/** Info to find the ViewModel instance at runtime. */
	UPROPERTY(VisibleAnywhere, Category = "View")
	FMVVMViewModelContext GlobalViewModelInstance;

	/**
	 * A resolvable path to retrieve the source instance at runtime.
	 * It can be a path "Property = Object.Function.Object".
	 * It can be a UFunction's name of a FProperty's name.
	 */
	UPROPERTY(VisibleAnywhere, Category = "View")
	FMVVMVCompiledFieldPath FieldPath;

	UPROPERTY(VisibleAnywhere, Category = "View")
	FName PropertyName;

	/** All the fields that the view need to register to for this source. */
	UPROPERTY(VisibleAnywhere, Category = "View")
	TArray<FMVVMViewClass_FieldId> FieldToRegisterTo;

	/**
	 * All the bindings this source need to execute at initialization (OneTime)
	 * And the bindings that needs to execute when the FieldId matches (OneWay).
	 */
	UPROPERTY(VisibleAnywhere, Category = "View")
	TArray<FMVVMViewClass_SourceBinding> Bindings;

	// All the conditions that need to execute when Field value changes
	UPROPERTY(VisibleAnywhere, Category = "View")
	TArray<FMVVMViewClass_SourceCondition> Conditions;

	enum class EFlags : uint16
	{
		None = 0,
		TypeCreateInstance = 1 << 0,
		IsUserWidgetProperty = 1 << 1,
		SetUserWidgetProperty = 1 << 2,
		IsOptional = 1 << 3,
		CanBeSet = 1 << 4,
		CanBeEvaluated = 1 << 5,
		HasEvaluatedBindings = 1 << 6,
		SelfReference = 1 << 7,
		HasTickBindings = 1 << 8,
		IsViewModel = 1 << 9,
		IsViewModelInstanceExposed = 1 << 10,
		GlobalViewModelCollectionUpdate = 1 << 11,
		AlwaysExecuteBindingsOnSetSource = 1 << 12,
		ParentContextReference = 1 << 13,
	};

	UPROPERTY(VisibleAnywhere, Category = "View")
	uint16 Flags = (uint16)EFlags::None;
};


/**
 * A compiled and shared delegate bindings
 */
USTRUCT()
struct FMVVMViewClass_Event
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;

public:
	/**
	 * The path to get access to the multicast from the UserWidget.
	 * Include the name of the Widget/ViewModel
	 */
	const FMVVMVCompiledFieldPath& GetMulticastDelegatePath() const
	{
		return FieldPath;
	}

	/** The name of the UFunction on the UserWidget. */
	const FName GetUserWidgetFunctionName() const
	{
		return UserWidgetFunctionName;
	}

	/**
	 * The source, if the multicast parent is a valid source.
	 * This is used when the source value changes at runtime and we want to bound the event again.
	 */
	FMVVMViewClass_SourceKey GetSourceKey() const
	{
		return SourceToReevaluate;
	}

	/**
	 * A view event may require more than one view sources to run the event.
	 * A event will not execute if any view source is invalid.
	 * It will not warn if the view source is make as optional.
	 */
	uint64 GetSources() const
	{
		return SourceBitField;
	}

#if UE_WITH_MVVM_DEBUGGING
	struct FToStringArgs
	{
		bool bUseDisplayName = true;
		
		MODELVIEWVIEWMODEL_API static FToStringArgs Short();
		MODELVIEWVIEWMODEL_API static FToStringArgs All();
	};
	/** @return a human readable version of the binding that can be use for debugging purposes. */
	UE_API FString ToString(const UMVVMViewClass* ViewClass, FToStringArgs Args) const;
#endif

private:
	UPROPERTY()
	FMVVMVCompiledFieldPath FieldPath;

	UPROPERTY()
	FName UserWidgetFunctionName;

	UPROPERTY()
	FMVVMViewClass_SourceKey SourceToReevaluate;
	
	UPROPERTY()
	uint64 SourceBitField = 0;
};

USTRUCT()
struct FMVVMViewClass_Condition
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;

public:

	/** The name of the UFunction on the UserWidget. */
	const FName GetUserWidgetFunctionName() const
	{
		return UserWidgetFunctionName;
	}

	/**
	 * The source, if the multicast parent is a valid source.
	 * This is used when the source value changes at runtime and we want to bound the event again.
	 */
	FMVVMViewClass_SourceKey GetSourceKey() const
	{
		return SourceToReevaluate;
	}

	/**
	 * A view event may require more than one view sources to run the event.
	 * A event will not execute if any view source is invalid.
	 * It will not warn if the view source is make as optional.
	 */
	uint64 GetSources() const
	{
		return SourceBitField;
	}

#if UE_WITH_MVVM_DEBUGGING
	struct FToStringArgs
	{
		bool bUseDisplayName = true;

		MODELVIEWVIEWMODEL_API static FToStringArgs Short();
		MODELVIEWVIEWMODEL_API static FToStringArgs All();
	};
	/** @return a human readable version of the binding that can be use for debugging purposes. */
	UE_API FString ToString(const UMVVMViewClass* ViewClass, FToStringArgs Args) const;
#endif

private:
	UPROPERTY()
	FName UserWidgetFunctionName;

	UPROPERTY()
	FMVVMViewClass_SourceKey SourceToReevaluate;

	UPROPERTY()
	uint64 SourceBitField = 0;
};

/**
 * Shared between every instances of the same View class.
 */
UCLASS(MinimalAPI)
class UMVVMViewClass : public UWidgetBlueprintGeneratedClassExtension
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;

public:
	//~ Begin UWidgetBlueprintGeneratedClassExtension
	UE_API virtual void Initialize(UUserWidget* UserWidget) override;
	UE_API virtual void Construct(UUserWidget* UserWidget) override;
	UE_API virtual void Destruct(UUserWidget* UserWidget) override;
#if WITH_EDITOR
	UE_API virtual void BeginDestroy() override;
#endif
	//~ End UWidgetBlueprintGeneratedClassExtension

public:
	/** Should it automatically create the binding sources when the view is constructed. */
	[[nodiscard]] bool DoesInitializeSourcesOnConstruct() const
	{
		return bInitializeSourcesOnConstruct;
	}

	/** Should it automatically register and execute the bindings when the view is constructed. */
	[[nodiscard]] bool DoesInitializeBindingsOnConstruct() const
	{
		return bInitializeBindingsOnConstruct;
	}

	/** Should it automatically register the events when the view is constructed. */
	[[nodiscard]] bool DoesInitializeEventsOnConstruct() const
	{
		return bInitializeEventsOnConstruct;
	}

	/** Should the view listen to modification made in the global viewmodel collection. */
	[[nodiscard]] bool DoesListenToViewModelCollectionChanged() const
	{
		return bListenToViewModelCollectionChanged;
	}
	
	/** Should the view be created even when there are no bindings or events. */
	[[nodiscard]] bool IsCreatedWithoutBindings() const
	{
		return bCreateViewWithoutBindings;
	}

	/** Get the container of all the bindings. */
	[[nodiscard]] const FMVVMCompiledBindingLibrary& GetBindingLibrary() const
	{
#if WITH_EDITOR
		ReloadIfDirty();
#endif
		return BindingLibrary;
	}

	/** The field of all the sources that are optional. */
	uint64 GetOptionalSources() const
	{
		return OptionalSources;
	}
	
	/** Get the list of all the needed viewmodel or widgets. */
	[[nodiscard]] const TArrayView<const FMVVMViewClass_Source> GetSources() const
	{
		return MakeArrayView(Sources);
	}
	
	/** The shared source used by the view. */
	[[nodiscard]] const FMVVMViewClass_Source& GetSource(FMVVMViewClass_SourceKey Key) const
	{
		check(Sources.IsValidIndex(Key.GetIndex()));
		return Sources[Key.GetIndex()];
	}

	/** The list of bindings. A binding can be used by more than one source. */
	[[nodiscard]] const TArrayView<const FMVVMViewClass_Binding> GetBindings() const
	{
		return Bindings;
	}

	/** The binding (can be shared by more than one source). */
	[[nodiscard]] const FMVVMViewClass_Binding& GetBinding(FMVVMViewClass_BindingKey Key) const
	{
		check(Bindings.IsValidIndex(Key.GetIndex()));
		return Bindings[Key.GetIndex()];
	}

	/** The list of evaluate bindings. */
	[[nodiscard]] const TArrayView<const FMVVMViewClass_EvaluateSource> GetEvaluateSources() const
	{
		return EvaluateSources;
	}

	/** The evaluate binding. */
	[[nodiscard]] const FMVVMViewClass_EvaluateSource& GetEvaluateSource(FMVVMViewClass_EvaluateBindingKey Key) const
	{
		check(EvaluateSources.IsValidIndex(Key.GetIndex()));
		return EvaluateSources[Key.GetIndex()];
	}
	
	/** The list of events. */
	[[nodiscard]] const TArrayView<const FMVVMViewClass_Event> GetEvents() const
	{
		return Events;
	}

	/** The event. */
	[[nodiscard]] const FMVVMViewClass_Event& GetEvent(FMVVMViewClass_EventKey Key) const
	{
		check(Events.IsValidIndex(Key.GetIndex()));
		return Events[Key.GetIndex()];
	}

	/** The list of conditions. */
	[[nodiscard]] const TArrayView<const FMVVMViewClass_Condition> GetConditions() const
	{
		return Conditions;
	}

	/** The condition. */
	[[nodiscard]] const FMVVMViewClass_Condition& GetCondition(FMVVMViewClass_ConditionKey Key) const
	{
		check(Conditions.IsValidIndex(Key.GetIndex()));
		return Conditions[Key.GetIndex()];
	}

	/** The list of extensions for widgets. */
	[[nodiscard]] const TArrayView<const TObjectPtr<UMVVMViewClassExtension>> GetViewClassExtensions() const
	{
		return ViewClassExtensions;
	}

#if UE_WITH_MVVM_DEBUGGING
	struct FToStringArgs
	{
		FMVVMViewClass_Source::FToStringArgs Source;
		FMVVMViewClass_Binding::FToStringArgs Binding;
		FMVVMViewClass_EvaluateSource::FToStringArgs Evaluate;
		FMVVMViewClass_Event::FToStringArgs Event;

		MODELVIEWVIEWMODEL_API static FToStringArgs Short();
		MODELVIEWVIEWMODEL_API static FToStringArgs All();
	};
	[[nodiscard]] UE_API FString ToString(FToStringArgs SourceArgs) const;
#endif

private:
#if WITH_EDITOR
	UE_API void HandleBlueprintCompiled();
	UE_API void ReloadIfDirty() const;
	std::atomic<bool> BindingLibraryDirty = false;
#endif

private:
	UPROPERTY(VisibleAnywhere, Category = "View")
	TArray<FMVVMViewClass_Source> Sources;
	UPROPERTY(VisibleAnywhere, Category = "View")
	TArray<FMVVMViewClass_Binding> Bindings;
	UPROPERTY(VisibleAnywhere, Category = "View")
	TArray<FMVVMViewClass_EvaluateSource> EvaluateSources;
	UPROPERTY(VisibleAnywhere, Category = "View")
	TArray<FMVVMViewClass_Event> Events;
	UPROPERTY(VisibleAnywhere, Category = "View")
	TArray<FMVVMViewClass_Condition> Conditions;

	/** All MVVM extensions on widgets of the owning userwidget. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UMVVMViewClassExtension>> ViewClassExtensions;

	/** All the bindings shared between all the View instance. */
	UPROPERTY()
	FMVVMCompiledBindingLibrary BindingLibrary;

	/** Which view source are optional. */
	UPROPERTY()
	uint64 OptionalSources;

	int32 ViewCounter = 0;

	UPROPERTY()
	bool bInitializeSourcesOnConstruct = true;

	UPROPERTY()
	bool bInitializeBindingsOnConstruct = true;

	UPROPERTY()
	bool bInitializeEventsOnConstruct = true;
	
	UPROPERTY()
	bool bListenToViewModelCollectionChanged = false;
	
	UPROPERTY()
	bool bCreateViewWithoutBindings = false;

#if WITH_EDITORONLY_DATA
	FDelegateHandle BlueprintCompiledHandle;
#endif
};

#undef UE_API
