// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Bindings/MVVMCompiledBindingLibrary.h"
#include "Types/MVVMExecutionMode.h"
#include "Types/MVVMViewModelContext.h"

#include "MVVMViewDeprecatedTypes.generated.h"

class UMVVMViewClass;
class UMVVMView;
class UMVVMViewModelContextResolver;
class UObject;
class UUserWidget;


USTRUCT()
struct
UE_DEPRECATED(5.4, "FMVVMVCompiledFieldId is no longer used.")
FMVVMVCompiledFieldId
{
	GENERATED_BODY()

	bool IsValid() const
	{
		return FieldIdIndex >= 0;
	}

private:
	using IndexType = int16;

	UPROPERTY()
	int16 FieldIdIndex = INDEX_NONE;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FGuid CompiledBindingLibraryId;
#endif
};

USTRUCT()
struct
UE_DEPRECATED(5.4, "FMVVMViewSource is no longer used, please use FMVVMView_Source instead.")
FMVVMViewSource
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	TObjectPtr<UObject> Source; // TScriptInterface<INotifyFieldValueChanged>

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FName SourceName;

	// Number of bindings connected to the source.
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	int32 RegisteredCount = 0;

	// The source is defined in the ClassExtension.
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	bool bCreatedSource = false;

	// The source bindings are initialized
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	bool bBindingsInitialized = false;

	// The source was set manually via SetViewModel.
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	bool bSetManually = false;

	// The source was set to a UserWidget property.
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	bool bAssignedToUserWidgetProperty = false;
};

USTRUCT()
struct
UE_DEPRECATED(5.4, "FMVVMViewClass_SourceCreator is no longer used, please use FMVVMViewClass_Source instead.")
FMVVMViewClass_SourceCreator
{
	GENERATED_BODY()

public:
	UObject* CreateInstance(const UMVVMViewClass* ViewClass, UMVVMView* View, UUserWidget* UserWidget) const
	{
		return nullptr;
	}

	void DestroyInstance(const UObject* ViewModel, const UMVVMView* View) const
	{
	}

	UClass* GetSourceClass() const
	{
		return ExpectedSourceType.Get();
	}

	bool IsSourceAUserWidgetProperty() const
	{
		return (Flags & (uint8)ESourceFlags::IsUserWidgetProperty) != 0;
	}
	
	bool CanBeSet() const
	{
		return (Flags & (uint8)ESourceFlags::CanBeSet) != 0;
	}

	bool CanBeEvaluated() const
	{
		return (Flags & (uint8)ESourceFlags::CanBeEvaluated) != 0;
	}
	
	bool IsOptional() const
	{
		return (Flags & (uint8)ESourceFlags::IsOptional) != 0;
	}

	FName GetSourceName() const
	{
		return PropertyName;
	}
	
	FName GetParentSourceName() const
	{
		return ParentSourceName;
	}

#if UE_WITH_MVVM_DEBUGGING
	struct FToStringArgs
	{
		bool bUseDisplayName = true;
		bool bAddCreationMode = true;
		bool bAddFlags = true;

		static FToStringArgs Short() { return FToStringArgs(); }
		static FToStringArgs All() { return FToStringArgs(); }
	};
	FString ToString(const FMVVMCompiledBindingLibrary& CompiledBindingLibrary, FToStringArgs Args) const{ return FString(); }
#endif

private:
	UPROPERTY()
	TSubclassOf<UObject> ExpectedSourceType;
	UPROPERTY(Instanced)
	TObjectPtr<UMVVMViewModelContextResolver> Resolver = nullptr;
	UPROPERTY()
	FMVVMViewModelContext GlobalViewModelInstance;
	UPROPERTY()
	FMVVMVCompiledFieldPath FieldPath;
	UPROPERTY()
	FName PropertyName;
	UPROPERTY()
	FName ParentSourceName;

	enum class ESourceFlags : uint8
	{
		None = 0,
		TypeCreateInstance = 1 << 0,
		IsUserWidgetProperty = 1 << 1,
		IsOptional = 1 << 2,
		CanBeSet = 1 << 3,
		CanBeEvaluated = 1 << 4,
		SelfReference = 1 << 5,
	};

	UPROPERTY()
	uint8 Flags = (uint8)ESourceFlags::None;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT()
struct
UE_DEPRECATED(5.4, "FMVVMViewClass_CompiledBinding is no longer used, please use FMVVMViewClass_Binding and FMVVMViewClass_SourceBinding instead.")
FMVVMViewClass_CompiledBinding
{
	GENERATED_BODY()

public:
	FMVVMVCompiledFieldId GetSourceFieldId() const
	{
		return FieldId;
	}

	FName GetSourceName() const
	{
		return SourcePropertyName;
	}

	int32 GetEvaluateSourceCreatorBindingIndex() const
	{
		return EvaluateSourceCreatorIndex;
	}

	bool IsSourceUserWidget() const
	{
		return (Flags & EBindingFlags::SourceObjectIsSelf) != 0;
	}

	const FMVVMVCompiledBinding& GetBinding() const
	{
		return Binding;
	}

	bool NeedsExecutionAtInitialization() const
	{
		return (Flags & EBindingFlags::ExecuteAtInitialization) != 0;
	}

	bool IsOneTime() const
	{
		return (Flags & EBindingFlags::OneTime) != 0;
	}

	bool IsEnabledByDefault() const
	{
		return (Flags & EBindingFlags::EnabledByDefault) != 0;
	}

	bool IsRegistrationOptional() const
	{
		return (Flags & EBindingFlags::ViewModelOptional) != 0;
	}
	
	bool IsEvaluateSourceCreatorBinding() const
	{
		return EvaluateSourceCreatorIndex != INDEX_NONE;
	}

	bool IsConversionFunctionComplex() const
	{
		return Binding.HasComplexConversionFunction();
	}
	
	EMVVMExecutionMode GetExecuteMode() const
	{
		return (Flags & EBindingFlags::OverrideExecuteMode) != 0 ? ExecutionMode : EMVVMExecutionMode::Immediate;
	}

#if UE_WITH_MVVM_DEBUGGING
	struct FToStringArgs
	{
		bool bUseDisplayName = true;
		bool bAddFieldPath = true;
		bool bAddFieldId = true;
		bool bAddFlags = true;
		bool bAddBindingId = true;

		static FToStringArgs Short()
		{
			return FToStringArgs();
		}
		static FToStringArgs All()
		{
			return FToStringArgs();
		}
	};


	FString ToString() const
	{
		return FString();
	}
	FString ToString(const FMVVMCompiledBindingLibrary& CompiledBindingLibrary, FToStringArgs Args) const
	{
		return FString();
	}
#endif

#if WITH_EDITOR
	FGuid GetEditorId() const
	{
		return EditorId;
	}
#endif

private:
	UPROPERTY()
	FMVVMVCompiledFieldId FieldId;
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FName SourcePropertyName;
	UPROPERTY()
	FMVVMVCompiledBinding Binding;
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	EMVVMExecutionMode ExecutionMode = EMVVMExecutionMode::Immediate;
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	int8 EvaluateSourceCreatorIndex = INDEX_NONE;

	enum EBindingFlags
	{
		None = 0,
		ExecuteAtInitialization = 1 << 0,
		Unused01 = 1 << 1,
		OneTime = 1 << 2,
		EnabledByDefault = 1 << 3,
		ViewModelOptional = 1 << 4,
		Unused02 = 1 << 5,
		OverrideExecuteMode = 1 << 6,
		SourceObjectIsSelf = 1 << 7,
	};

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	uint8 Flags = EBindingFlags::None;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FGuid EditorId;
#endif
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS


USTRUCT()
struct
UE_DEPRECATED(5.4, "MVVMViewClass_CompiledEvent is no longer used, please use MVVMViewClass_Event instead.")
FMVVMViewClass_CompiledEvent
{
	GENERATED_BODY()

public:
	FName GetSourceName() const
	{
		return SourceName;
	}

	const FMVVMVCompiledFieldPath& GetMulticastDelegatePath() const
	{
		return FieldPath;
	}

	const FName GetUserWidgetFunctionName() const
	{
		return FunctionName;
	}

#if UE_WITH_MVVM_DEBUGGING
	struct FToStringArgs
	{
		bool bUseDisplayName = true;
	};
	FString ToString(const FMVVMCompiledBindingLibrary& CompiledBindingLibrary, FToStringArgs Args) const
	{
		return FString();
	}
#endif

private:
	UPROPERTY()
	FMVVMVCompiledFieldPath FieldPath;
	UPROPERTY()
	FName FunctionName;
	UPROPERTY()
	FName SourceName;
};
