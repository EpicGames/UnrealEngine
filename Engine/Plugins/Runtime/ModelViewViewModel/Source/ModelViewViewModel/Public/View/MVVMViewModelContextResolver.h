// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "INotifyFieldValueChanged.h"

#include "MVVMViewModelContextResolver.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

class UMVVMView;
class UMVVMViewClass;
class UUserWidget;

/**
 * Shared data to find or create a ViewModel at runtime.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, EditInlineNew, DisplayName = "Viewmodel Resolver")
class UMVVMViewModelContextResolver : public UObject
{
	GENERATED_BODY()

public:
	virtual UObject* CreateInstance(const UClass* ExpectedType, const UUserWidget* UserWidget, const UMVVMView* View) const
	{
		return K2_CreateInstance(ExpectedType, UserWidget).GetObject();
	}

	UFUNCTION(BlueprintImplementableEvent, Category = "Viewmodel", meta = (DisplayName = "Create Instance"))
	UE_API TScriptInterface<INotifyFieldValueChanged> K2_CreateInstance(const UClass* ExpectedType, const UUserWidget* UserWidget) const;

	virtual void DestroyInstance(UObject* ViewModel, const UMVVMView* View) const
	{
		K2_DestroyInstance(ViewModel, View);
	}

	UFUNCTION(BlueprintImplementableEvent, Category = "Viewmodel", meta = (DisplayName = "Destroy Instance"))
	UE_API void K2_DestroyInstance(UObject* ViewModel, const UMVVMView* View) const;

public:
#if WITH_EDITOR
	UE_API virtual bool DoesSupportViewModelClass(const UClass* Class) const;
#endif

private:
#if WITH_EDITORONLY_DATA
	/** Viewmodel class that the resolver supports.*/
	UPROPERTY(EditDefaultsOnly, Category = "Viewmodel", meta=(MustImplement="/Script/FieldNotification.NotifyFieldValueChanged", DisallowedClasses="/Script/UMG.Widget"))
	TArray<FSoftClassPath> AllowedViewModelClasses;

	/** Viewmodel class that the resolver explicitly does not support. */
	UPROPERTY(EditDefaultsOnly, Category = "Viewmodel", meta=(MustImplement="/Script/FieldNotification.NotifyFieldValueChanged", DisallowedClasses="/Script/UMG.Widget"))
	TArray<FSoftClassPath> DeniedViewModelClasses;
#endif
};

#undef UE_API
