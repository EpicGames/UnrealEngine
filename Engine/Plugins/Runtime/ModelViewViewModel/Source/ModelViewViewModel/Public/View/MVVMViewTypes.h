// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FieldNotificationId.h"

#include "MVVMViewTypes.generated.h"

namespace UE::MVVM::Private { struct FMVVMViewBlueprintCompiler; }
class UMVVMViewClass;



/**
 * FieldName and Bit needed to create a UE::FieldNotification::FFieldId.
 */
USTRUCT()
struct FMVVMViewClass_FieldId
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;
	friend UMVVMViewClass;

public:
	FMVVMViewClass_FieldId() = default;
	explicit FMVVMViewClass_FieldId(UE::FieldNotification::FFieldId InFieldId)
	{
		FieldName = InFieldId.GetName();
		FieldIndex = InFieldId.GetIndex();
	}

	bool IsValid() const
	{
		return FieldIndex != INDEX_NONE && !FieldName.IsNone();
	}

	UE::FieldNotification::FFieldId GetFieldId() const
	{
		return UE::FieldNotification::FFieldId(FieldName, FieldIndex);
	}

	FName GetName() const
	{
		return FieldName;
	}

	bool operator== (const FMVVMViewClass_FieldId& Other) const
	{
		return FieldName == Other.FieldName && FieldIndex == Other.FieldIndex;
	}

private:
	UPROPERTY(VisibleAnywhere, Category = "View")
	FName FieldName;

	int32 FieldIndex = INDEX_NONE;
};

/**
 * Key that identify the FMVVMViewClass_Source.
 */
USTRUCT()
struct FMVVMViewClass_SourceKey
{
	GENERATED_BODY()

public:
	FMVVMViewClass_SourceKey() = default;
	FMVVMViewClass_SourceKey(int32 InIndex)
		: Index(InIndex)
	{}

	bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

	uint64 GetBit() const
	{
		// keep the first bit free for now as it may be useful in the future.
		return (uint64)1 << Index;
	}

	int32 GetIndex() const
	{
		return Index;
	}

	bool operator== (const FMVVMViewClass_SourceKey& Other) const
	{
		return Index == Other.Index;
	}

private:
	UPROPERTY(VisibleAnywhere, Category = "View")
	int32 Index = INDEX_NONE;
};

/**
 * Key that identify the FMVVMViewClass_Binding.
 */
USTRUCT()
struct FMVVMViewClass_BindingKey
{
	GENERATED_BODY()

public:
	FMVVMViewClass_BindingKey() = default;
	explicit FMVVMViewClass_BindingKey(int32 InIndex)
		: Index(InIndex)
	{}
	bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

	int32 GetIndex() const
	{
		return Index;
	}

	bool operator== (const FMVVMViewClass_BindingKey& Other) const
	{
		return Index == Other.Index;
	}

private:
	UPROPERTY(VisibleAnywhere, Category = "View")
	int32 Index = INDEX_NONE;
};

/**
 * Key that identify the FMVVMViewClass_EvaluateBinding.
 */
USTRUCT()
struct FMVVMViewClass_EvaluateBindingKey
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;

public:
	FMVVMViewClass_EvaluateBindingKey() = default;
	explicit FMVVMViewClass_EvaluateBindingKey(int32 InIndex)
		: Index(InIndex)
	{}
	bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

	int32 GetIndex() const
	{
		return Index;
	}

	bool operator== (const FMVVMViewClass_EvaluateBindingKey& Other) const
	{
		return Index == Other.Index;
	}

private:
	UPROPERTY(VisibleAnywhere, Category = "View")
	int32 Index = INDEX_NONE;
};

/**
 * Key that identify the FMVVMViewClass_Event.
 */
USTRUCT()
struct FMVVMViewClass_EventKey
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;

public:
	FMVVMViewClass_EventKey() = default;
	explicit FMVVMViewClass_EventKey(int32 InIndex)
		: Index(InIndex)
	{}
	bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

	int32 GetIndex() const
	{
		return Index;
	}

	bool operator== (const FMVVMViewClass_EventKey& Other) const
	{
		return Index == Other.Index;
	}

private:
	UPROPERTY(VisibleAnywhere, Category = "View")
	int32 Index = INDEX_NONE;
};

/**
 * Key that identify the FMVVMViewClass_Condition.
 */
USTRUCT()
struct FMVVMViewClass_ConditionKey
{
	GENERATED_BODY()

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;

public:
	FMVVMViewClass_ConditionKey() = default;
	explicit FMVVMViewClass_ConditionKey(int32 InIndex)
		: Index(InIndex)
	{}
	bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

	int32 GetIndex() const
	{
		return Index;
	}

	bool operator== (const FMVVMViewClass_ConditionKey& Other) const
	{
		return Index == Other.Index;
	}

private:
	UPROPERTY(VisibleAnywhere, Category = "View")
	int32 Index = INDEX_NONE;
};


/**
 * Key that identify the FMVVMView_Source.
 */
USTRUCT()
struct FMVVMView_SourceKey
{
	GENERATED_BODY()

public:
	FMVVMView_SourceKey() = default;
	FMVVMView_SourceKey(int32 InIndex)
		: Index(InIndex)
	{}

	bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

	int32 GetIndex() const
	{
		return Index;
	}

	bool operator== (const FMVVMView_SourceKey& Other) const
	{
		return Index == Other.Index;
	}

private:
	UPROPERTY(VisibleAnywhere, Category = "View")
	int32 Index = INDEX_NONE;
};
