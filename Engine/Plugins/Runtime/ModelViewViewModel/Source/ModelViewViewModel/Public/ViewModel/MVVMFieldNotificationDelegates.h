// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FieldNotificationDelegate.h"

#define UE_API MODELVIEWVIEWMODEL_API

class FDelegateHandle;
namespace UE::FieldNotification { struct FFieldId; }

namespace UE::MVVM
{

/** Basic implementation of the INotifyFieldValueChanged implementation. */
class FMVVMFieldNotificationDelegates
{
public:
	UE_API FDelegateHandle AddFieldValueChangedDelegate(UObject* Owner, UE::FieldNotification::FFieldId InFieldId, INotifyFieldValueChanged::FFieldValueChangedDelegate InNewDelegate);
	UE_API void AddFieldValueChangedDelegate(UObject* Owner, UE::FieldNotification::FFieldId FieldId, const FFieldValueChangedDynamicDelegate& Delegate);
	UE_API bool RemoveFieldValueChangedDelegate(UObject* Owner, UE::FieldNotification::FFieldId InFieldId, FDelegateHandle InHandle);
	UE_API bool RemoveFieldValueChangedDelegate(UObject* Owner, UE::FieldNotification::FFieldId FieldId, const FFieldValueChangedDynamicDelegate& Delegate);
	UE_API int32 RemoveAllFieldValueChangedDelegates(UObject* Owner, FDelegateUserObjectConst InUserObject);
	UE_API int32 RemoveAllFieldValueChangedDelegates(UObject* Owner, UE::FieldNotification::FFieldId InFieldId, FDelegateUserObjectConst InUserObject);
	UE_API void BroadcastFieldValueChanged(UObject* Owner, UE::FieldNotification::FFieldId InFieldId);
	UE_API TArray<UE::FieldNotification::FFieldMulticastDelegate::FDelegateView> GetView() const;

private:
	UE::FieldNotification::FFieldMulticastDelegate Delegates;
	TBitArray<> EnabledFieldNotifications;
};

} //namespace

#undef UE_API
