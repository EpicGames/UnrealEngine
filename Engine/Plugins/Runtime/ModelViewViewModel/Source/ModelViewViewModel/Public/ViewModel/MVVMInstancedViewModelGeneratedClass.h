// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/BlueprintGeneratedClass.h"
#include "UObject/ObjectKey.h"
#include "UObject/Package.h"
#include "MVVMInstancedViewModelGeneratedClass.generated.h"

#define UE_API MODELVIEWVIEWMODEL_API

/**
 *
 */
UCLASS(MinimalAPI)
class UMVVMInstancedViewModelGeneratedClass : public UBlueprintGeneratedClass
{
	GENERATED_BODY()

public:
	UE_API virtual void Link(FArchive& Ar, bool bRelinkExistingProperties) override;
	
#if WITH_EDITOR
	UE_API virtual UClass* GetAuthoritativeClass() override;
	UE_API virtual void PurgeClass(bool bRecompilingOnLoad) override;
	UE_API void AddNativeRepNotifyFunction(UFunction* Function, const FProperty* Property);
	UE_API void PurgeNativeRepNotifyFunctions();
#endif

public:
	UE_API DECLARE_FUNCTION(K2_CallNativeOnRep);

	UE_API void BroadcastFieldValueChanged(UObject* Object, const FProperty* Property);
	UE_API virtual void OnPropertyReplicated(UObject* Object, const FProperty* Property);

private:
	UPROPERTY()
	TArray<TObjectPtr<UFunction>> OnRepFunctionToLink;

	TMap<TObjectKey<UFunction>, const FProperty*> OnRepToPropertyMap;
};

#undef UE_API
