// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/BlueprintGeneratedClass.h"

#include "MVVMViewModelBlueprintGeneratedClass.generated.h"

/** */
UCLASS(Deprecated, MinimalAPI)
class UE_DEPRECATED(5.5, "Use the regular BlueprintGeneratedClass.") UDEPRECATED_MVVMViewModelBlueprintGeneratedClass : public UBlueprintGeneratedClass
{
	GENERATED_BODY()
};
