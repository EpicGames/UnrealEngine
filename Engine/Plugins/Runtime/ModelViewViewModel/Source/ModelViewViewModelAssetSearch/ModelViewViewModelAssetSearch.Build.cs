// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using UnrealBuildTool;

public class ModelViewViewModelAssetSearch : ModuleRules 
{
	public ModelViewViewModelAssetSearch(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetSearch",
				"Core",
				"CoreUObject",
				"ModelViewViewModel",
				"ModelViewViewModelBlueprint",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"InputCore",
				"UMGEditor",
			});
	}
}
