// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "IAssetIndexer.h"
#include "SearchSerializer.h"
#include "Utility/IndexerUtilities.h"

#include "MVVMBlueprintView.h"
#include "MVVMWidgetBlueprintExtension_View.h"


namespace UE::MVVM::Private
{
class FAssetSearchIndexer : public IAssetIndexer
{
private:
	enum class EIndexerVersion
	{
		Initial = 0,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

public:
	virtual FString GetName() const override
	{
		return TEXT("MVVM");
	}

	virtual int32 GetVersion() const override
	{
		return (int32)EIndexerVersion::LatestVersion;
	}

	virtual void IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const override
	{
		const UMVVMWidgetBlueprintExtension_View* AssetObject = CastChecked<UMVVMWidgetBlueprintExtension_View>(InAssetObject);

		if (const UMVVMBlueprintView* View = AssetObject->GetBlueprintView())
		{
			Serializer.BeginIndexingObject(View, TEXT("$self"));
			FIndexerUtilities::IterateIndexableProperties(View, [&Serializer](const FProperty* Property, const FString& Value) {
				Serializer.IndexProperty(Property, Value);
				UE_LOGF(LogTemp, Warning, "--%ls [Serialized]", *Property->GetName());
				});
			Serializer.EndIndexingObject();
		}
	}
};
} //namespace
