// Copyright Epic Games, Inc. All Rights Reserved.


#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "IAssetSearchModule.h"
#include "MVVMWidgetBlueprintExtension_View.h"

#include "MVVMAssetSearchIndexer.h"

#define LOCTEXT_NAMESPACE "ModelViewViewModelAssetSearch"

class FMVVMAssetSearchModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		if (IAssetSearchModule* Ptr = FModuleManager::GetModulePtr<IAssetSearchModule>(ModuleName))
		{
			RegisterAssetIndexer();
		}
		else
		{
			ModuleChangedHandle = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FMVVMAssetSearchModule::HandleModulesChanged);
		}
	}

	virtual void ShutdownModule() override
	{
		if (ModuleChangedHandle.IsValid())
		{
			FModuleManager::Get().OnModulesChanged().Remove(ModuleChangedHandle);
			ModuleChangedHandle.Reset();
		}
	}

private:
	void HandleModulesChanged(FName InModuleName, EModuleChangeReason InReason)
	{
		if (InModuleName == ModuleName && InReason == EModuleChangeReason::ModuleLoaded)
		{
			RegisterAssetIndexer();
			FModuleManager::Get().OnModulesChanged().Remove(ModuleChangedHandle);
		}
	}

	void RegisterAssetIndexer()
	{
		IAssetSearchModule::Get().RegisterAssetIndexer(UMVVMWidgetBlueprintExtension_View::StaticClass(), MakeUnique<UE::MVVM::Private::FAssetSearchIndexer>());
	}

private:
	const FName ModuleName = "AssetSearch";
	FDelegateHandle ModuleChangedHandle;
};


IMPLEMENT_MODULE(FMVVMAssetSearchModule, ModelViewViewModelAssetSearch);

#undef LOCTEXT_NAMESPACE
