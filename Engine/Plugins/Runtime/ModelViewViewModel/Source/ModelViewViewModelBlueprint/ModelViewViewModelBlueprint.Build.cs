// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using UnrealBuildTool;

public class ModelViewViewModelBlueprint : ModuleRules 
{
	public ModelViewViewModelBlueprint(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"FieldNotification",
				"ModelViewViewModel",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BlueprintGraph",
				"EnhancedInput",
				"Kismet",
				"KismetCompiler",
				"PropertyEditor",
				"PropertyPath",
				"SlateCore",
				"Slate",
				"UMG",
				"UMGEditor",
				"UnrealEd",
			});
	}
}
