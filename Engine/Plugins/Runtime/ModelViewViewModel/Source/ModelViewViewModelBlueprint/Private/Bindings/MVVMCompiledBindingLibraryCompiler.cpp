// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MVVMCompiledBindingLibraryCompiler.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Bindings/MVVMFieldPathHelper.h"
#include "Engine/Blueprint.h"
#include "Engine/Engine.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMSubsystem.h"
#include "Misc/TVariantMeta.h"
#include <limits> // IWYU pragma: keep

#define LOCTEXT_NAMESPACE "CompiledBindingLibraryCompiler"

namespace UE::MVVM::Private
{

static const FName NAME_BlueprintGetter = "BlueprintGetter";


/** */
struct FRawFieldId
{
	const UClass* NotifyFieldValueChangedClass;
	UE::FieldNotification::FFieldId FieldId;
	FCompiledBindingLibraryCompiler::FFieldIdHandle IdHandle;
};


/** */
struct FRawField
{
	FMVVMConstFieldVariant Field;
	bool bPropertyIsObjectProperty = false; // either the FProperty or the return value of the UFunction
	bool bPropertyIsStructProperty = false;
	int32 LoadedPropertyOrFunctionIndex = INDEX_NONE;

public:
	bool IsSameField(const FRawField& Other) const
	{
		return Field == Other.Field;
	}
};


/** */
struct FRawFieldPath
{
	TArray<int32> RawFieldIndexes;
	bool bIsReadable = false;
	bool bIsWritable = false;

	FCompiledBindingLibraryCompiler::FFieldPathHandle PathHandle;
	FMVVMVCompiledFieldPath CompiledFieldPath;

public:
	bool IsSameFieldPath(const FRawFieldPath& Other) const
	{
		return Other.RawFieldIndexes == RawFieldIndexes;
	}
};


/** */
struct FRawBinding
{
	FCompiledBindingLibraryCompiler::FFieldPathHandle SourcePathHandle;
	FCompiledBindingLibraryCompiler::FFieldPathHandle DestinationPathHandle;
	FCompiledBindingLibraryCompiler::FFieldPathHandle ConversionFunctionPathHandle;

	FCompiledBindingLibraryCompiler::FBindingHandle BindingHandle;
	FMVVMVCompiledBinding CompiledBinding;
	int32 BindingCount = 1;
	bool bIsConversionFunctionComplex = false;

public:
	bool IsSameBinding(const FRawBinding& Binding) const
	{
		return Binding.SourcePathHandle == SourcePathHandle
			&& Binding.DestinationPathHandle == DestinationPathHandle
			&& Binding.ConversionFunctionPathHandle == ConversionFunctionPathHandle;
	}
};

/** */
class FCompiledBindingLibraryCompilerImpl
{
public:
	FCompiledBindingLibraryCompilerImpl(const UBlueprint* Context)
		: BlueprintContext(Context)
	{}

public:
	TArray<FRawFieldId> FieldIds;
	TArray<FRawField> Fields;
	TArray<FRawFieldPath> FieldPaths;
	TArray<FRawBinding> Bindings;
	bool bCompiled = false;
	TWeakObjectPtr<const UBlueprint> BlueprintContext;

public:
	int32 AddUniqueField(FMVVMConstFieldVariant InFieldVariant)
	{
		int32 FoundFieldPath = Fields.IndexOfByPredicate([InFieldVariant](const Private::FRawField& Other)
			{
				return Other.Field == InFieldVariant;
			});
		if (FoundFieldPath == INDEX_NONE)
		{
			FRawField RawField;
			RawField.Field = InFieldVariant;
			check(!InFieldVariant.IsEmpty());
			const FProperty* FieldProperty = InFieldVariant.IsProperty() ? InFieldVariant.GetProperty() : BindingHelper::GetReturnProperty(InFieldVariant.GetFunction());
			// FieldProperty can be null if it's a setter function
			RawField.bPropertyIsObjectProperty = CastField<FObjectPropertyBase>(FieldProperty) != nullptr;
			RawField.bPropertyIsStructProperty = CastField<FStructProperty>(FieldProperty) != nullptr;

			FoundFieldPath = Fields.Add(RawField);
		}
		return FoundFieldPath;
	}
};

/** */
//FMVVMConstFieldVariant GetMostUpToDate(FMVVMConstFieldVariant Field)
//{
//	if (Field.IsValid())
//	{
//		UClass* Class = Cast<UClass>(Field.GetOwner());
//		if (!Class)
//		{
//			return Field;
//		}
//		if (Field.IsProperty())
//		{
//			return FMVVMConstFieldVariant(FBlueprintEditorUtils::GetMostUpToDateProperty(Field.GetProperty()));
//		}
//		else if (Field.IsFunction())
//		{
//			return FMVVMConstFieldVariant(FBlueprintEditorUtils::GetMostUpToDateFunction(Field.GetFunction()));
//		}
//	}
//	return Field;
//}


/** */
const UStruct* GetSavedGeneratedStruct(FMVVMConstFieldVariant Field)
{
	const UStruct* Result = Field.GetOwner();
	const UClass* Class = Cast<UClass>(Result);
	if (!Class || Class->HasAnyClassFlags(CLASS_Native) || Class->bCooked || !Field.IsValid())
	{
		return Result;
	}

	const UBlueprint* Blueprint = Cast<const UBlueprint>(Class->ClassGeneratedBy);
	if (!Blueprint)
	{
		return Result;
	}

	ensure(Blueprint->GeneratedClass);
	return Blueprint->GeneratedClass;
}

/** */
const UClass* GetSavedGeneratedStruct(const UClass* Class)
{
	if (!Class || Class->HasAnyClassFlags(CLASS_Native) || Class->bCooked)
	{
		return Class;
	}

	UBlueprint* Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy);
	if (!Blueprint)
	{
		return Class;
	}

	ensure(Blueprint->GeneratedClass);
	return Blueprint->GeneratedClass;
}

} //namespace UE::MVVM::Private



namespace UE::MVVM
{


int32 FCompiledBindingLibraryCompiler::FBindingHandle::IdGenerator = 0;
int32 FCompiledBindingLibraryCompiler::FFieldPathHandle::IdGenerator = 0;
int32 FCompiledBindingLibraryCompiler::FFieldIdHandle::IdGenerator = 0;

/**
 *
 */
FCompiledBindingLibraryCompiler::FCompileResult::FCompileResult(FGuid LibraryId)
	: Library(LibraryId)
{

}

/**
 *
 */
FCompiledBindingLibraryCompiler::FCompiledBindingLibraryCompiler(UBlueprint* GeneratingFor)
	: Impl(MakePimpl<Private::FCompiledBindingLibraryCompilerImpl>(GeneratingFor))
{

}

TValueOrError<FCompiledBindingLibraryCompiler::FFieldIdHandle, FText> FCompiledBindingLibraryCompiler::AddFieldId(const UClass* InSourceClass, FName FieldId)
{
	Impl->bCompiled = false;

	const UClass* SourceClass = FBlueprintEditorUtils::GetMostUpToDateClass(InSourceClass);

	if (FieldId.IsNone())
	{
		return MakeError(LOCTEXT("FieldNotDefined", "The Field does not have the specifier FieldNotify or cannot be used as a binding source. You may want to use the 'One Time' binding mode or make sure it is readable."));
	}

	if (!SourceClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
	{
		return MakeError(FText::Format(LOCTEXT("ClassDoesNotImplementInterface", "'{0}' doesn't implement the NotifyFieldValueChanged interface.")
			, SourceClass->GetDisplayNameText()));
	}

	const TScriptInterface<INotifyFieldValueChanged> ScriptObject = SourceClass->GetDefaultObject();
	if (ensure(ScriptObject.GetInterface()))
	{
		UE::FieldNotification::FFieldId FoundFieldId = ScriptObject->GetFieldNotificationDescriptor().GetField(SourceClass, FieldId);
		if (!FoundFieldId.IsValid())
		{
			return MakeError(FText::Format(LOCTEXT("FieldNotifyNotSupported", "The FieldNotify '{0}' is not supported by '{1}'.")
				, FText::FromName(FieldId)
				, SourceClass->GetDisplayNameText()));
		}

		int32 FoundFieldIdIndex = Impl->FieldIds.IndexOfByPredicate([FoundFieldId, SourceClass](const Private::FRawFieldId& Other)
			{
				return Other.FieldId == FoundFieldId && Other.NotifyFieldValueChangedClass == SourceClass;
			});
		if (FoundFieldIdIndex == INDEX_NONE)
		{
			Private::FRawFieldId RawFieldId;
			RawFieldId.NotifyFieldValueChangedClass = SourceClass;
			RawFieldId.FieldId = FoundFieldId;
			RawFieldId.IdHandle = FFieldIdHandle::MakeHandle();
			FoundFieldIdIndex = Impl->FieldIds.Add(MoveTemp(RawFieldId));
		}

		return MakeValue(Impl->FieldIds[FoundFieldIdIndex].IdHandle);
	}

	return MakeError(LOCTEXT("UnexpectedCaseInAddFieldId", "Unexpected case with AddFieldId."));
}


TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> FCompiledBindingLibraryCompiler::AddFieldPath(TArrayView<const FMVVMConstFieldVariant> InFieldPath, bool bInRead)
{
	Impl->bCompiled = false;

	return AddFieldPathImpl(InFieldPath, bInRead);
}


TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> FCompiledBindingLibraryCompiler::AddFieldPathImpl(TArrayView<const FMVVMConstFieldVariant> InFieldPath, bool bInRead)
{
	Impl->bCompiled = false;

	auto ValidateContainer = [](const FProperty* Property, bool bShouldBeInsideContainer, bool bIsObjectOrScriptStruct) -> FText
	{	
		const UStruct* OwnerStruct = Property->GetOwnerStruct();
		if (OwnerStruct == nullptr)
		{
			return FText::Format(LOCTEXT("FieldHasInvalidOwner", "The field {0} has an invalid owner struct."), Property->GetDisplayNameText());
		}

		if (bShouldBeInsideContainer)
		{
			if (!Cast<UScriptStruct>(OwnerStruct) && !Cast<UClass>(OwnerStruct))
			{
				return FText::Format(LOCTEXT("FieldDoesNotHaveValidOwnerForPath", "The field {0} doesn't have a valid owner for that path."), Property->GetDisplayNameText());
			}
		}

		if (bIsObjectOrScriptStruct)
		{
			if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property))
			{
				return FText::GetEmpty();
			}
			else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
			{
				if (StructProperty->HasGetter() || Property->HasMetaData(Private::NAME_BlueprintGetter))
				{
					return FText::Format(LOCTEXT("GetterNotSupported", "Property {0} has getter accessor. Accessor not supported on FStructProperty since it would create a temporary structure and we would not able to return a valid container from that structure."), StructProperty->GetDisplayNameText());
				}
				return FText::GetEmpty();
			}
			return FText::Format(LOCTEXT("FieldCanOnlyBeObjectOrStruct", "Field can only be object properties or struct properties. {0} is a {1}"), Property->GetDisplayNameText(), Property->GetClass()->GetDisplayNameText());
		}

		return FText::GetEmpty();
	};

	TArray<int32> RawFieldIndexes;
	RawFieldIndexes.Reserve(InFieldPath.Num());

	const UBlueprint* BlueprintContext = Impl->BlueprintContext.Get();
	if (BlueprintContext == nullptr)
	{
		return MakeError(LOCTEXT("InvalidBlueprint", "The Blueprint is invalid."));
	}

	const UStruct* CurrentContainer = BlueprintContext->GeneratedClass ? BlueprintContext->GeneratedClass : BlueprintContext->SkeletonGeneratedClass;

	for (int32 Index = 0; Index < InFieldPath.Num(); ++Index)
	{
		if (CurrentContainer == nullptr)
		{
			return MakeError(LOCTEXT("InvalidContainer", "The path has an invalid container."));
		}

		// Make sure the FieldVariant is not from a skeletalclass
		FMVVMConstFieldVariant FieldVariant = InFieldPath[Index];
		if (!FieldVariant.IsValid())
		{
			return MakeError(FText::Format(LOCTEXT("FieldDoesNotHaveValidOwner", "The field {0} doesn't have a valid owner."), FText::FromName(InFieldPath[Index].GetName())));
		}

		const bool bIsLast = Index == InFieldPath.Num() - 1;
		if (FieldVariant.IsProperty())
		{
			// They must all be readable except the last item if we are writing to the property.
			if (bIsLast && !bInRead)
			{
				if (!BindingHelper::IsValidForDestinationBinding(FieldVariant.GetProperty()))
				{
					return MakeError(FText::Format(LOCTEXT("PropertyNotWritableAtRuntime", "Property '{0}' is not writable at runtime."), FieldVariant.GetProperty()->GetDisplayNameText()));
				}
			}
			else if (!BindingHelper::IsValidForSourceBinding(FieldVariant.GetProperty()))
			{
				return MakeError(FText::Format(LOCTEXT("PropertyNotReadableAtRuntime", "Property '{0}' is not readable at runtime."), FieldVariant.GetProperty()->GetDisplayNameText()));
			}

			FText ValidatedStr = ValidateContainer(FieldVariant.GetProperty(), true, !bIsLast);
			if (!ValidatedStr.IsEmpty())
			{
				return MakeError(ValidatedStr);
			}

			if (!GetDefault<UMVVMDeveloperProjectSettings>()->IsPropertyAllowed(Impl->BlueprintContext.Get(), CurrentContainer, FieldVariant.GetProperty()))
			{
				return MakeError(LOCTEXT("PropertyNotAllow", "A property is not allowed."));
			}

			RawFieldIndexes.Add(Impl->AddUniqueField(FieldVariant));
		}
		else if (FieldVariant.IsFunction())
		{
			if (bIsLast && !bInRead)
			{
				if (!BindingHelper::IsValidForDestinationBinding(FieldVariant.GetFunction()))
				{
					return MakeError(FText::Format(LOCTEXT("FunctionNotWritableAtRuntime", "Function '{0}' is not writable at runtime."), FieldVariant.GetFunction()->GetDisplayNameText()));
				}
			}
			else if (!BindingHelper::IsValidForSourceBinding(FieldVariant.GetFunction()))
			{
				return MakeError(FText::Format(LOCTEXT("FunctionNotReadableAtRuntime", "Function '{0}' is not readable at runtime."), FieldVariant.GetFunction()->GetDisplayNameText()));
			}

			const UClass* CurrentContainerAsClass = Cast<const UClass>(CurrentContainer);
			if (CurrentContainerAsClass == nullptr)
			{
				return MakeError(LOCTEXT("InvalidContainer", "The path has an invalid container."));
			}

			if (!GetDefault<UMVVMDeveloperProjectSettings>()->IsFunctionAllowed(Impl->BlueprintContext.Get(), CurrentContainerAsClass, FieldVariant.GetFunction()))
			{
				return MakeError(LOCTEXT("FunctionNotAllow", "A function is not allowed."));
			}

			if (bIsLast && !bInRead)
			{
				const FProperty* FirstProperty = BindingHelper::GetFirstArgumentProperty(FieldVariant.GetFunction());
				ValidateContainer(FirstProperty, false, bIsLast);
				RawFieldIndexes.Add(Impl->AddUniqueField(FieldVariant));
			}
			else
			{
				const FProperty* ReturnProperty = BindingHelper::GetReturnProperty(FieldVariant.GetFunction());
				ValidateContainer(ReturnProperty, false, bIsLast);
				RawFieldIndexes.Add(Impl->AddUniqueField(FieldVariant));
			}
		}
		else
		{
			return MakeError(LOCTEXT("InvalidFieldInPath", "There is an invalid field in the field path."));
		}

		TValueOrError<const UStruct*, void> FieldAsContainerResult = UE::MVVM::FieldPathHelper::GetFieldAsContainer(FieldVariant);
		CurrentContainer = FieldAsContainerResult.HasValue() ? FieldAsContainerResult.GetValue() : nullptr;
	}

	int32 FoundFieldPath = Impl->FieldPaths.IndexOfByPredicate([&RawFieldIndexes](const Private::FRawFieldPath& Other)
		{
			return Other.RawFieldIndexes == RawFieldIndexes;
		});
	if (FoundFieldPath != INDEX_NONE)
	{
		Impl->FieldPaths[FoundFieldPath].bIsReadable = Impl->FieldPaths[FoundFieldPath].bIsReadable || bInRead;
		Impl->FieldPaths[FoundFieldPath].bIsWritable = Impl->FieldPaths[FoundFieldPath].bIsWritable || !bInRead;
		return MakeValue(Impl->FieldPaths[FoundFieldPath].PathHandle);
	}

	Private::FRawFieldPath RawFieldPath;
	RawFieldPath.RawFieldIndexes = RawFieldIndexes;
	RawFieldPath.PathHandle = FFieldPathHandle::MakeHandle();
	RawFieldPath.bIsReadable = bInRead;
	RawFieldPath.bIsWritable = !bInRead;
	FoundFieldPath = Impl->FieldPaths.Add(MoveTemp(RawFieldPath));

	return MakeValue(Impl->FieldPaths[FoundFieldPath].PathHandle);
}


TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> FCompiledBindingLibraryCompiler::AddObjectFieldPath(TArrayView<const UE::MVVM::FMVVMConstFieldVariant> FieldPath, const UClass* ExpectedType, bool bInRead)
{
	Impl->bCompiled = false;

	check(ExpectedType);

	if (FieldPath.Num() == 0)
	{
		return MakeError(FText::Format(LOCTEXT("FieldDoesNotReturnTypeEmptyPath", "The field does not return a '{0}', Field path was empty."), ExpectedType->GetDisplayNameText()));
	}

	UE::MVVM::FMVVMConstFieldVariant Last = FieldPath.Last();

	const FObjectPropertyBase* ObjectPropertyBase = nullptr;
	if (Last.IsProperty())
	{
		ObjectPropertyBase = CastField<const FObjectPropertyBase>(Last.GetProperty());
	}
	else if (Last.IsFunction())
	{
		ObjectPropertyBase = CastField<const FObjectPropertyBase>(BindingHelper::GetReturnProperty(Last.GetFunction()));
	}

	if (ObjectPropertyBase == nullptr)
	{
		return MakeError(FText::Format(LOCTEXT("FieldDoesNotReturnTypeNoProperty", "The field does not return a '{0}', Object Property not found."), ExpectedType->GetDisplayNameText()));
	}
	if (ObjectPropertyBase->PropertyClass == nullptr || !ObjectPropertyBase->PropertyClass->IsChildOf(ExpectedType))
	{
		return MakeError(FText::Format(LOCTEXT("FieldDoesNotReturnType", "The field does not return a '{0}'."), ExpectedType->GetDisplayNameText()));
	}

	return AddFieldPathImpl(FieldPath, bInRead);
}


TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> FCompiledBindingLibraryCompiler::AddConversionFunctionFieldPath(const UClass* InSourceClass, const UFunction* InFunction)
{
	auto ValidateConversionFunction = [](const UFunction* Function)
	{
		const bool bIsSimpleFunction = BindingHelper::IsValidForSimpleRuntimeConversion(Function);
		const bool bIsComplexFunction = BindingHelper::IsValidForComplexRuntimeConversion(Function);
		if (!bIsSimpleFunction && !bIsComplexFunction)
		{
			return FText::Format(LOCTEXT("FunctionCannotBeUsedAsConversionFunction", "Function {0} cannot be used as a runtime conversion function."), Function->GetDisplayNameText());
		}

		return FText::GetEmpty();
	};

	return AddFunctionFieldPathImpl(InSourceClass, InFunction, ValidateConversionFunction);
}

TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> FCompiledBindingLibraryCompiler::AddDelegateSignatureFieldPath(const UClass* InSourceClass, const UFunction* InFunction)
{
	auto ValidateDelegate = [](const UFunction* Function)
	{
		const bool bIsEvent = BindingHelper::IsValidForDelegateSignatureBinding(Function);
		if (!bIsEvent)
		{
			return FText::Format(LOCTEXT("EventCannotBeUsedAsDelegateSignature", "Event {0} cannot be used as a runtime delegate signature function."), Function->GetDisplayNameText());
		}

		return FText::GetEmpty();
	};

	return AddFunctionFieldPathImpl(InSourceClass, InFunction, ValidateDelegate);
}


TValueOrError<FCompiledBindingLibraryCompiler::FBindingHandle, FText> FCompiledBindingLibraryCompiler::AddBinding(FFieldPathHandle InSourceHandle, FFieldPathHandle InDestinationHandle)
{
	return AddBindingImpl(InSourceHandle, InDestinationHandle, FFieldPathHandle(), false);
}


TValueOrError<FCompiledBindingLibraryCompiler::FBindingHandle, FText> FCompiledBindingLibraryCompiler::AddBinding(FFieldPathHandle InSourceHandle, FFieldPathHandle InDestinationHandle, FFieldPathHandle InConversionFunctionHandle)
{
	return AddBindingImpl(InSourceHandle, InDestinationHandle, InConversionFunctionHandle, false);
}


TValueOrError<FCompiledBindingLibraryCompiler::FBindingHandle, FText> FCompiledBindingLibraryCompiler::AddComplexBinding(FFieldPathHandle InDestinationHandle, FFieldPathHandle InConversionFunctionHandle)
{
	return AddBindingImpl(FFieldPathHandle(), InDestinationHandle, InConversionFunctionHandle, true);
}


TValueOrError<FCompiledBindingLibraryCompiler::FBindingHandle, FText> FCompiledBindingLibraryCompiler::AddBindingImpl(FFieldPathHandle InSourceHandle, FFieldPathHandle InDestinationHandle, FFieldPathHandle InConversionFunctionHandle, bool bInIsComplexBinding)
{
	Impl->bCompiled = false;

	UMVVMSubsystem::FConstDirectionalBindingArgs DirectionBindingArgs;

	// Complex Conversion function do not have input arguments.
	if (!bInIsComplexBinding)
	{
		const int32 FoundSourceFieldPath = Impl->FieldPaths.IndexOfByPredicate([InSourceHandle](const Private::FRawFieldPath& Other)
			{
				return Other.PathHandle == InSourceHandle;
			});
		if (FoundSourceFieldPath == INDEX_NONE)
		{
			return MakeError(LOCTEXT("SourceHandleInvalid", "The source handle is invalid."));
		}

		Private::FRawFieldPath& SourceRawFieldPath = Impl->FieldPaths[FoundSourceFieldPath];
		if (!SourceRawFieldPath.bIsReadable)
		{
			return MakeError(LOCTEXT("SourceHandleNotReadable", "The source handle was not constructed as a readable path."));
		}
		if (SourceRawFieldPath.RawFieldIndexes.Num() == 0)
		{
			return MakeError(LOCTEXT("SourceHandleNotRegistered", "The source handle was not registered correctly."));
		}

		Private::FRawField& RawField = Impl->Fields[SourceRawFieldPath.RawFieldIndexes.Last()];
		if (RawField.Field.IsEmpty())
		{
			return MakeError(LOCTEXT("SourceHandleNotRegistered", "The source handle was not registered correctly."));
		}

		DirectionBindingArgs.SourceBinding = RawField.Field;
	}

	{
		const int32 FoundDestinationFieldPath = Impl->FieldPaths.IndexOfByPredicate([InDestinationHandle](const Private::FRawFieldPath& Other)
			{
				return Other.PathHandle == InDestinationHandle;
			});
		if (FoundDestinationFieldPath == INDEX_NONE)
		{
			return MakeError(LOCTEXT("DestinationHandleInvalid", "The destination handle is invalid."));
		}

		Private::FRawFieldPath& DestinationRawFieldPath = Impl->FieldPaths[FoundDestinationFieldPath];
		if (!DestinationRawFieldPath.bIsWritable)
		{
			return MakeError(LOCTEXT("DestinationHandleNotWritable", "The destination handle was not constructed as a writable path."));
		}
		if (DestinationRawFieldPath.RawFieldIndexes.Num() == 0)
		{
			return MakeError(LOCTEXT("DestinationHandleNotRegistered", "The destination handle was not registered correctly."));
		}

		Private::FRawField& RawField = Impl->Fields[DestinationRawFieldPath.RawFieldIndexes.Last()];
		if (RawField.Field.IsEmpty())
		{
			return MakeError(LOCTEXT("DestinationHandleNotRegistered", "The destination handle was not registered correctly."));
		}

		DirectionBindingArgs.DestinationBinding = RawField.Field;
	}

	if (InConversionFunctionHandle.IsValid())
	{
		const int32 FoundFunctionFieldPath = Impl->FieldPaths.IndexOfByPredicate([InConversionFunctionHandle](const Private::FRawFieldPath& Other)
			{
				return Other.PathHandle == InConversionFunctionHandle;
			});
		if (FoundFunctionFieldPath == INDEX_NONE)
		{
			return MakeError(LOCTEXT("FunctionHandleInvalid", "The function handle is invalid."));
		}

		Private::FRawFieldPath& ConversionFunctinRawFieldPath = Impl->FieldPaths[FoundFunctionFieldPath];
		if (ConversionFunctinRawFieldPath.RawFieldIndexes.Num() == 0)
		{
			return MakeError(LOCTEXT("FunctionHandleNotRegistered", "The function handle was not registered as a function."));
		}

		Private::FRawField& RawField = Impl->Fields[ConversionFunctinRawFieldPath.RawFieldIndexes.Last()];
		if (!RawField.Field.IsFunction())
		{
			return MakeError(LOCTEXT("FunctionHandleNotRegistered", "The function handle was not registered as a function."));
		}

		DirectionBindingArgs.ConversionFunction = RawField.Field.GetFunction();
	}

	TValueOrError<bool, FText> IsValidBinding = GEngine->GetEngineSubsystem<UMVVMSubsystem>()->IsBindingValid(DirectionBindingArgs);
	if (IsValidBinding.HasError())
	{
		return MakeError(IsValidBinding.StealError());
	}

	Private::FRawBinding NewBinding;
	NewBinding.SourcePathHandle = InSourceHandle;
	NewBinding.DestinationPathHandle = InDestinationHandle;
	NewBinding.ConversionFunctionPathHandle = InConversionFunctionHandle;
	NewBinding.bIsConversionFunctionComplex = bInIsComplexBinding;
	const int32 FoundSameBindingIndex = Impl->Bindings.IndexOfByPredicate([&NewBinding](const Private::FRawBinding& Binding)
		{
			return NewBinding.IsSameBinding(Binding);
		});

	if (FoundSameBindingIndex != INDEX_NONE)
	{
		if (!bInIsComplexBinding)
		{
			return MakeError(LOCTEXT("BindingAlreadyAdded", "The binding already exist."));
		}
		else
		{
			++Impl->Bindings[FoundSameBindingIndex].BindingCount;
			return MakeValue(Impl->Bindings[FoundSameBindingIndex].BindingHandle);
		}
	}
	else
	{
		FCompiledBindingLibraryCompiler::FBindingHandle ResultBindingHandle = FBindingHandle::MakeHandle();
		NewBinding.BindingHandle = ResultBindingHandle;
		Impl->Bindings.Add(MoveTemp(NewBinding));
		return MakeValue(ResultBindingHandle);
	}
}

TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> FCompiledBindingLibraryCompiler::AddFunctionFieldPathImpl(const UClass* InSourceClass, const UFunction* InFunction, TFunctionRef<FText(const UFunction*)> ValidateFunctionCallback)
{
	Impl->bCompiled = false;

	const UClass* SourceClass = FBlueprintEditorUtils::GetMostUpToDateClass(InSourceClass);
	const UFunction* Function = FBlueprintEditorUtils::GetMostUpToDateFunction(InFunction);

	// Transient Conversion function are only added to generated class and not to the skeletal class.
	bool bTransientConversionFunction = false;
	if (Function == nullptr && InFunction && InFunction->GetTypedOuter<UClass>() == InSourceClass)
	{
		Function = InFunction;
		bTransientConversionFunction = true;
	}

	if (SourceClass == nullptr)
	{
		return MakeError(LOCTEXT("SourceClassInvalid", "The source class is invalid."));
	}
	if (Function == nullptr)
	{
		return MakeError(LOCTEXT("FunctionPathEmpty", "The function path is empty."));
	}

	FText FunctionValidationError = ValidateFunctionCallback(Function);
	if (!FunctionValidationError.IsEmpty())
	{
		return MakeError(FunctionValidationError);
	}

	if (!Function->HasAllFunctionFlags(FUNC_Static))
	{
		if (!SourceClass->IsChildOf(Function->GetOuterUClass()) && !bTransientConversionFunction)
		{
			return MakeError(FText::Format(LOCTEXT("FunctionHasInvalidSelf", "Function {0} is going to be executed with an invalid self."), Function->GetDisplayNameText()));
		}
	}

	TArray<int32> RawFieldIndexes;
	RawFieldIndexes.Add(Impl->AddUniqueField(FMVVMConstFieldVariant(Function)));
	const int32 FoundFieldPath = Impl->FieldPaths.IndexOfByPredicate([&RawFieldIndexes](const Private::FRawFieldPath& Other)
		{
			return Other.RawFieldIndexes == RawFieldIndexes;
		});
	if (FoundFieldPath != INDEX_NONE)
	{
		return MakeValue(Impl->FieldPaths[FoundFieldPath].PathHandle);
	}

	Private::FRawFieldPath RawFieldPath;
	RawFieldPath.RawFieldIndexes = MoveTemp(RawFieldIndexes);
	RawFieldPath.PathHandle = FFieldPathHandle::MakeHandle();
	RawFieldPath.bIsReadable = false;
	RawFieldPath.bIsWritable = false;

	const int32 NewFieldPathIndex = Impl->FieldPaths.Add(MoveTemp(RawFieldPath));
	return MakeValue(Impl->FieldPaths[NewFieldPathIndex].PathHandle);
}


TValueOrError<FCompiledBindingLibraryCompiler::FCompileResult, FText> FCompiledBindingLibraryCompiler::Compile(FGuid LibraryId)
{
	Impl->bCompiled = false;

	struct FCompiledClassInfo
	{
		TArray<int32> RawFieldIndex;
		TArray<int32> RawFieldIdIndex;
	};

	// create the list of UClass. Uses the Generated class, not the SkeletalClass.
	TMap<const UStruct*, FCompiledClassInfo> MapOfFieldInClass;
	{
		for (int32 Index = 0; Index < Impl->Fields.Num(); ++Index)
		{
			const Private::FRawField& RawField = Impl->Fields[Index];
			check(!RawField.Field.IsEmpty());

			const UStruct* FinalOwner = Private::GetSavedGeneratedStruct(RawField.Field);
			check(FinalOwner);
			FCompiledClassInfo& ClassInfo = MapOfFieldInClass.FindOrAdd(FinalOwner);

			// Test if the Field is there more than one
			{
				FMVVMConstFieldVariant FieldToTest = RawField.Field;
				const TArray<Private::FRawField>& ListOfFields = Impl->Fields;
				const bool bContains = ClassInfo.RawFieldIndex.ContainsByPredicate([FieldToTest, &ListOfFields](int32 OtherIndex)
					{
						return ListOfFields[OtherIndex].Field == FieldToTest;
					});
				check(!bContains);
			}

			ClassInfo.RawFieldIndex.Add(Index);
		}
	}

	// Todo optimize that list to group common type. ie UWidget::ToolTip == UProgressBar::ToolTip. We can merge UWidget in UProgressBar.
	//Algo: for each class entry
		// if: a class is child of another class in the list (ProgressBar is child of Widget all property inside Widget are also in ProgressBar)
		// then: merge the 2 and restart the algo


	FCompileResult Result = FCompileResult(LibraryId);

	// Create FMVVMCompiledBindingLibrary::CompiledFields and FMVVMCompiledBindingLibrary::CompiledFieldNames
	int32 TotalNumberOfProperties = 0;
	int32 TotalNumberOfFunctions = 0;
	for (TPair<const UStruct*, FCompiledClassInfo>& StructCompiledFields : MapOfFieldInClass)
	{
		FMVVMVCompiledFields CompiledFields;
		CompiledFields.ClassOrScriptStruct = StructCompiledFields.Key; // The generated class not the skeletal class
		check(StructCompiledFields.Key);

		TArray<FName> PropertyNames;
		TArray<FName> FunctionNames;
		TArray<FName> FieldIdNames;
		for (const int32 FieldIndex : StructCompiledFields.Value.RawFieldIndex)
		{
			check(Impl->Fields.IsValidIndex(FieldIndex));
			Private::FRawField& RawField = Impl->Fields[FieldIndex];
			const FMVVMConstFieldVariant& Field = RawField.Field;

			if (Field.IsProperty())
			{
				// N.B. no need to translate from skeletal to generated, we only use the the name or the index.
				Result.Library.LoadedProperties.Add(const_cast<FProperty*>(Field.GetProperty()));
				PropertyNames.Add(Field.GetName());
				RawField.LoadedPropertyOrFunctionIndex = TotalNumberOfProperties;
				++TotalNumberOfProperties;
			}
			else
			{
				check(Field.IsFunction());
				// N.B. no need to translate from skeletal to generated, we only use the the name or the index.
				Result.Library.LoadedFunctions.Emplace(Field.GetFunction());
				FunctionNames.Add(Field.GetName());
				RawField.LoadedPropertyOrFunctionIndex = TotalNumberOfFunctions;
				++TotalNumberOfFunctions;
			}
		}

		if (PropertyNames.Num() > std::numeric_limits<FMVVMVCompiledBinding::IndexType>::max())
		{
			return MakeError(FText::Format(LOCTEXT("TooManyPropertiesBound", "There are too many properties bound to struct '{0}'"), StructCompiledFields.Key->GetDisplayNameText()));
		}
		CompiledFields.NumberOfProperties = static_cast<int16>(PropertyNames.Num());

		if (FunctionNames.Num() > std::numeric_limits<FMVVMVCompiledBinding::IndexType>::max())
		{
			return MakeError(FText::Format(LOCTEXT("TooManyFunctionsBound", "There are too many functions bound to struct '{0}'"), StructCompiledFields.Key->GetDisplayNameText()));
		}
		CompiledFields.NumberOfFunctions = static_cast<int16>(FunctionNames.Num());

		int32 LibraryStartIndex = Result.Library.CompiledFieldNames.Num();
		if (LibraryStartIndex > std::numeric_limits<FMVVMVCompiledBinding::IndexType>::max())
		{
			return MakeError(LOCTEXT("TooManyPropertiesAndFunctionsBound", "There are too many properties and functions bound in the library."));
		}
		CompiledFields.LibraryStartIndex = static_cast<int16>(LibraryStartIndex);
			
		Result.Library.CompiledFieldNames.Append(PropertyNames);
		PropertyNames.Reset();
		Result.Library.CompiledFieldNames.Append(FunctionNames);
		FunctionNames.Reset();
		if (Result.Library.CompiledFieldNames.Num() > std::numeric_limits<FMVVMVCompiledBinding::IndexType>::max())
		{
			return MakeError(LOCTEXT("TooManyPropertiesBoundInLibrary", "There are too many properties bound in the library."));
		}

		Result.Library.CompiledFields.Add(CompiledFields);

		check(Result.Library.LoadedProperties.Num() + Result.Library.LoadedFunctions.Num() == Result.Library.CompiledFieldNames.Num());
		check(Result.Library.LoadedProperties.Num() == TotalNumberOfProperties);
		check(Result.Library.LoadedFunctions.Num() == TotalNumberOfFunctions);
	}

	// Create FMVVMCompiledBindingLibrary::FieldPaths
	for (Private::FRawFieldPath& FieldPath : Impl->FieldPaths)
	{
		FieldPath.CompiledFieldPath.CompiledBindingLibraryId = Result.Library.CompiledBindingLibraryId;
		FieldPath.CompiledFieldPath.StartIndex = INDEX_NONE;
		FieldPath.CompiledFieldPath.Num = FieldPath.RawFieldIndexes.Num();
		if (FieldPath.RawFieldIndexes.Num())
		{
			FieldPath.CompiledFieldPath.StartIndex = Result.Library.FieldPaths.Num();
			for (const int32 RawFieldIndex : FieldPath.RawFieldIndexes)
			{
				const Private::FRawField& RawField = Impl->Fields[RawFieldIndex];
				check(!RawField.Field.IsEmpty());

				FMVVMCompiledLoadedPropertyOrFunctionIndex FieldIndex;
				FieldIndex.Index = RawField.LoadedPropertyOrFunctionIndex;
				FieldIndex.bIsObjectProperty = RawField.bPropertyIsObjectProperty;
				FieldIndex.bIsScriptStructProperty = RawField.bPropertyIsStructProperty;
				FieldIndex.bIsProperty = RawField.Field.IsProperty();
				Result.Library.FieldPaths.Add(FieldIndex);

				if (FieldIndex.bIsProperty)
				{
					check(Result.Library.LoadedProperties.IsValidIndex(FieldIndex.Index));
				}
				else
				{
					check(Result.Library.LoadedFunctions.IsValidIndex(FieldIndex.Index));
				}
			}
		}

		Result.FieldPaths.Add(FieldPath.PathHandle, FieldPath.CompiledFieldPath);
	}

	// Create FieldId
	for (Private::FRawFieldId& FieldId: Impl->FieldIds)
	{
		Result.FieldIds.Add(FieldId.IdHandle, FieldId.FieldId);
	}

	auto GetCompiledFieldPath = [this](const FFieldPathHandle Handle)
	{
		const Private::FRawFieldPath* FoundBinding = Impl->FieldPaths.FindByPredicate([Handle](const Private::FRawFieldPath& Other)
			{
				return Other.PathHandle == Handle;
			});
		if (FoundBinding)
		{
			return FoundBinding->CompiledFieldPath;
		}
		return FMVVMVCompiledFieldPath();
	};

	// Create the requested FMVVMVCompiledBinding
	for (Private::FRawBinding& Binding : Impl->Bindings)
	{
		Binding.CompiledBinding.CompiledBindingLibraryId = Result.Library.CompiledBindingLibraryId;
		check(Binding.CompiledBinding.CompiledBindingLibraryId.IsValid());

		Binding.CompiledBinding.SourceFieldPath = GetCompiledFieldPath(Binding.SourcePathHandle);
		check(Binding.bIsConversionFunctionComplex || Binding.CompiledBinding.SourceFieldPath.IsValid());

		Binding.CompiledBinding.DestinationFieldPath = GetCompiledFieldPath(Binding.DestinationPathHandle);
		check(Binding.CompiledBinding.DestinationFieldPath.IsValid());

		Binding.CompiledBinding.ConversionFunctionFieldPath = GetCompiledFieldPath(Binding.ConversionFunctionPathHandle);

		if (Binding.bIsConversionFunctionComplex)
		{
			Binding.CompiledBinding.Type = (uint8)FMVVMVCompiledBinding::EType::HasComplexConversionFunction;
		}
		else if (Binding.ConversionFunctionPathHandle.IsValid())
		{
			Binding.CompiledBinding.Type = (uint8)FMVVMVCompiledBinding::EType::HasConversionFunction;
		}
		else
		{
			Binding.CompiledBinding.Type = (uint8)FMVVMVCompiledBinding::EType::None;
		}

		Result.Bindings.Add(Binding.BindingHandle, Binding.CompiledBinding);
	}

	Result.Library.LoadedProperties.Reset();
	Result.Library.LoadedFunctions.Reset();

	Impl->bCompiled = true;
	return MakeValue(MoveTemp(Result));
}

} //namespace

#undef LOCTEXT_NAMESPACE
