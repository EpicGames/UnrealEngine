// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MVVMConversionFunctionHelper.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Bindings/MVVMFieldPathHelper.h"
#include "Logging/StructuredLog.h"
#include "MVVMMessageLog.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Components/Widget.h"
#include "Containers/Deque.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMFunctionGraphHelper.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "UObject/MetaData.h"

#include "K2Node_BaseAsyncTask.h"
#include "K2Node_BreakStruct.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Event.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_GeneratedBoundEvent.h"
#include "K2Node_Self.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MVVMConversionFunctionGraphSchema.h"
#include "Node/MVVMK2Node_AreSourcesValidForBinding.h"

#define LOCTEXT_NAMESPACE "MVVMConversionFunctionHelper"

namespace UE::MVVM::ConversionFunctionHelper
{

namespace Private
{
	static const FLazyName ConversionFunctionMetadataKey = "ConversionFunction";
	static const FStringView AutoPromoteFunctionMetadataKey = TEXT("AutoPromoteFunction");
	static const FLazyName ConversionFunctionCategory = "Conversion Functions";

namespace NamedNodes
{
	static const FLazyName GeneratedSetter = "MVVM_Named_Node_Generated_Setter";
	static const FLazyName GeneratedCallFunction = "MVVM_Named_Node_Generated_CallFunction";
}

	UMVVMBlueprintView* GetView(const UBlueprint* Blueprint)
	{
		const TObjectPtr<UBlueprintExtension>* ExtensionViewPtr = Blueprint->GetExtensions().FindByPredicate([](const UBlueprintExtension* Other) { return Other && Other->GetClass() == UMVVMWidgetBlueprintExtension_View::StaticClass(); });
		if (ExtensionViewPtr)
		{
			return CastChecked<UMVVMWidgetBlueprintExtension_View>(*ExtensionViewPtr)->GetBlueprintView();
		}
		return nullptr;
	}

	bool IsSystemInputPin(const UEdGraphPin* Pin)
	{
		return Pin && Pin->PinName != UEdGraphSchema_K2::PN_Execute && Pin->Direction == EGPD_Input && (!Pin->bOrphanedPin || Pin->ShouldSavePinIfOrphaned()) && !Pin->bHidden;
	}

	void MarkAsConversionFunction(const UK2Node* FunctionNode, const UEdGraph* Graph)
	{
		check(FunctionNode != nullptr);
		FunctionNode->GetPackage()->GetMetaData().SetValue(FunctionNode, ConversionFunctionMetadataKey.Resolve(), TEXT(""));
	}

	UK2Node_FunctionEntry* FindFunctionEntry(const UEdGraph* Graph)
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (UK2Node_FunctionEntry* FunctionEntry = Cast<UK2Node_FunctionEntry>(Node))
			{
				return FunctionEntry;
			}
		}
		return nullptr;
	}

	UK2Node_Event* FindEventEntry(const UEdGraph* Graph)
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (UK2Node_Event* EventEntry = Cast<UK2Node_Event>(Node))
			{
				return EventEntry;
			}
		}
		return nullptr;
	}

	UK2Node_FunctionResult* FindFunctionResult(const UEdGraph* Graph)
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (UK2Node_FunctionResult* FunctionResult = Cast<UK2Node_FunctionResult>(Node))
			{
				return FunctionResult;
			}
		}
		return nullptr;
	}
	
	struct FCreateGraphResult
	{
		UEdGraph* FunctionGraph = nullptr;
		UK2Node_Event* EventEntry = nullptr;
		UK2Node_FunctionEntry* FunctionEntry = nullptr;
		UK2Node_FunctionResult* FunctionResult = nullptr;
	};

	struct FCreateGraphParams
	{
		bool bIsConst = false;
		bool bIsEditable = false;
		bool bAddToBlueprint = false;

		/** If true implies this graph will create events*/
		bool bCreateUbergraphPage = false;
	};

	FCreateGraphResult CreateGraph(UBlueprint* Blueprint, FName GraphName, const UFunction* FunctionEntryDefinition, FCreateGraphParams InParams)
	{
		if (UObject* ExistingObject = StaticFindObject(nullptr, Blueprint, *GraphName.ToString(), EFindObjectFlags::ExactClass))
		{
			FunctionGraphHelper::RenameObjectToTransientPackage(ExistingObject);
		}

		FName UniqueFunctionName = FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, GraphName.ToString());

		// Ubergraph pages create multicast delegate variables of the same name during BP Skeleton generation prior to graph generation
		// These will cause name conflicts, so don't require unique names for ubergraphs
		if (InParams.bCreateUbergraphPage)
		{
			UniqueFunctionName = GraphName;
		}

		ensure(GraphName == UniqueFunctionName);

		// Create function graph
		UEdGraph* FunctionGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, UniqueFunctionName, UEdGraph::StaticClass(), UMVVMConversionFunctionGraphSchema::StaticClass());
		ensure(FunctionGraph->GetFName() == GraphName);

		FunctionGraph->bEditable = InParams.bIsEditable;
		if (InParams.bAddToBlueprint)
		{
			Blueprint->FunctionGraphs.Add(FunctionGraph);
		}
		else
		{
			FunctionGraph->SetFlags(RF_Transient);
		}

		const UEdGraphSchema_K2* Schema = GetDefault<UMVVMConversionFunctionGraphSchema>();
		Schema->MarkFunctionEntryAsEditable(FunctionGraph, InParams.bIsEditable);
		Schema->CreateDefaultNodesForGraph(*FunctionGraph);

		FCreateGraphResult Result;

		// Function entry node
		if (!InParams.bCreateUbergraphPage)
		{
			FGraphNodeCreator<UK2Node_FunctionEntry> FunctionEntryCreator(*FunctionGraph);
			UK2Node_FunctionEntry* FunctionEntry = FunctionEntryCreator.CreateNode();
			if (FunctionEntryDefinition)
			{
				UClass* OwnerClass = FunctionEntryDefinition->GetOwnerClass();
				FunctionEntry->FunctionReference.SetExternalMember(FunctionEntryDefinition->GetFName(), OwnerClass);
				FunctionEntry->CustomGeneratedFunctionName = GraphName;
			}
			else
			{
				FunctionEntry->FunctionReference.SetSelfMember(FunctionGraph->GetFName());
			}
			if (InParams.bIsConst)
			{
				FunctionEntry->AddExtraFlags(FUNC_BlueprintCallable | FUNC_BlueprintPure | FUNC_Const | FUNC_Protected | FUNC_Final);
			}
			else
			{
				FunctionEntry->AddExtraFlags(FUNC_BlueprintCallable | FUNC_Protected | FUNC_Final);
			}
			FunctionEntry->bIsEditable = InParams.bIsEditable;
			FunctionEntry->MetaData.Category = FText::FromName(ConversionFunctionCategory.Resolve());
			FunctionEntry->NodePosX = -500;
			FunctionEntry->NodePosY = 0;
			FunctionEntryCreator.Finalize();

			FGraphNodeCreator<UK2Node_FunctionResult> FunctionResultCreator(*FunctionGraph);
			UK2Node_FunctionResult* FunctionResult = FunctionResultCreator.CreateNode();
			FunctionResult->FunctionReference.SetSelfMember(FunctionGraph->GetFName());
			FunctionResult->bIsEditable = InParams.bIsEditable;
			FunctionResult->NodePosX = 500;
			FunctionResult->NodePosY = 0;
			FunctionResultCreator.Finalize();

			Result.FunctionEntry = FunctionEntry;
			Result.FunctionResult = FunctionResult;
		}
		else
		{
			FGraphNodeCreator<UK2Node_GeneratedBoundEvent> EventEntryCreator(*FunctionGraph);
			UK2Node_GeneratedBoundEvent* EventEntry = EventEntryCreator.CreateNode();

			EventEntry->EventReference.SetSelfMember(FunctionGraph->GetFName());
			EventEntry->bIsEditable = InParams.bIsEditable;
			EventEntry->CustomFunctionName = UE::MVVM::BindingHelper::GetDelegateSignatureName(FunctionGraph->GetFName());
			EventEntry->NodePosX = -750;
			EventEntry->NodePosY = 0;
			EventEntryCreator.Finalize();

			Result.EventEntry = EventEntry;
		}

		Result.FunctionGraph = FunctionGraph;

		return Result;
	}

	TArray<TTuple<UEdGraphNode*, UEdGraphPin*>> GetPropertyPathGraphNode(const UEdGraphPin* StartPin)
	{
		TArray<TTuple<UEdGraphNode*, UEdGraphPin*>> NodesInPath;

		auto AddNode = [&NodesInPath](const UEdGraphPin* Pin)
			{
				UEdGraphNode* Result = nullptr;
				if (Private::IsSystemInputPin(Pin) && Pin->LinkedTo.Num() == 1)
				{
					Result = Pin->LinkedTo[0]->GetOwningNode();
					if (Result && !IsAutoPromoteNode(Result))
					{
						NodesInPath.Emplace(Result, Pin->LinkedTo[0]);
					}
				}
				return Result;
			};

		UEdGraphNode* CurrentNode = AddNode(StartPin);
		while (CurrentNode)
		{
			TArray<UEdGraphPin*>& Pins = CurrentNode->Pins;
			CurrentNode = nullptr;
			for (UEdGraphPin* Pin : Pins)
			{
				if (UEdGraphNode* NewNode = AddNode(Pin))
				{
					CurrentNode = NewNode;
					break;
				}
			}
		}

		Algo::Reverse(NodesInPath);
		return NodesInPath;
	}

	FMVVMBlueprintPropertyPath GetPropertyPathForPin(const UBlueprint* Blueprint, const UEdGraphPin* StartPin, bool bSkipResolve)
	{
		if (!IsInputPin(StartPin))
		{
			return FMVVMBlueprintPropertyPath();
		}

		UMVVMBlueprintView* BlueprintView = GetView(Blueprint);
		if (BlueprintView == nullptr)
		{
			return FMVVMBlueprintPropertyPath();
		}

		FMVVMBlueprintPropertyPath ResultPath;

		auto AddRoot = [&ResultPath, BlueprintView, Blueprint, bSkipResolve](FMemberReference& Member)
			{
				if (bSkipResolve)
				{
					// if the generated class hasn't yet been generated we can blindly forge ahead and try to figure out if it's a widget or a viewmodel
					if (const FMVVMBlueprintViewModelContext* ViewModel = BlueprintView->FindViewModel(Member.GetMemberName()))
					{
						ResultPath.SetViewModelId(ViewModel->GetViewModelId());
					}
					else
					{
						ResultPath.SetWidgetName(Member.GetMemberName());
						ensure(Member.GetMemberName() != Blueprint->GetFName());
					}
				}
				else
				{
					if (const FObjectProperty* Property = CastField<FObjectProperty>(Member.ResolveMember<FProperty>(Blueprint->SkeletonGeneratedClass)))
					{
						if (Property->PropertyClass->IsChildOf<UWidget>() || Property->PropertyClass->IsChildOf<UBlueprint>())
						{
							ResultPath.SetWidgetName(Property->GetFName());
							ensure(Property->GetFName() != Blueprint->GetFName());
						}
						else if (Property->PropertyClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
						{
							if (const FMVVMBlueprintViewModelContext* ViewModel = BlueprintView->FindViewModel(Property->GetFName()))
							{
								ResultPath.SetViewModelId(ViewModel->GetViewModelId());
							}
						}
					}
				}
			};

		auto AddPropertyPath = [&ResultPath, Blueprint](FMemberReference& MemberReference)
			{
				if (UFunction* Function = MemberReference.ResolveMember<UFunction>(Blueprint->SkeletonGeneratedClass))
				{
					ResultPath.AppendPropertyPath(Blueprint, UE::MVVM::FMVVMConstFieldVariant(Function));
				}
				else if (const FProperty* Property = MemberReference.ResolveMember<FProperty>(Blueprint->SkeletonGeneratedClass))
				{
					ResultPath.AppendPropertyPath(Blueprint, UE::MVVM::FMVVMConstFieldVariant(Property));
				}
			};

		auto AddBreakNode = [&ResultPath, Blueprint](UScriptStruct* Struct, FName PropertyName)
			{
				FProperty* FoundProperty = Struct->FindPropertyByName(PropertyName);
				if (ensure(FoundProperty))
				{
					ResultPath.AppendPropertyPath(Blueprint, UE::MVVM::FMVVMConstFieldVariant(FoundProperty));
				}
			};

		bool bFirst = true;
		TArray<TTuple<UEdGraphNode*, UEdGraphPin*>> NodesToSearch = GetPropertyPathGraphNode(StartPin);
		for (const TTuple<UEdGraphNode*, UEdGraphPin*>& NodePair : NodesToSearch)
		{
			UEdGraphNode* Node = NodePair.Get<UEdGraphNode*>();
			if (UK2Node_VariableGet* GetNode = Cast<UK2Node_VariableGet>(Node))
			{
				if (bFirst)
				{
					AddRoot(GetNode->VariableReference);
				}
				else
				{
					AddPropertyPath(GetNode->VariableReference);
				}
			}
			else if (UK2Node_CallFunction* FunctionNode = Cast<UK2Node_CallFunction>(Node))
			{
				// UK2Node_CallFunction can be native break function
				if (UFunction* Function = FunctionNode->FunctionReference.ResolveMember<UFunction>(Blueprint->SkeletonGeneratedClass))
				{
					bool bAddPropertyPath = true;
					const FProperty* ArgumentProperty = UE::MVVM::BindingHelper::GetFirstArgumentProperty(Function);
					const FStructProperty* ArgumentStructProperty = CastField<FStructProperty>(ArgumentProperty);
					if (ArgumentStructProperty && ArgumentStructProperty->Struct)
					{
						const FString& MetaData = ArgumentStructProperty->Struct->GetMetaData(FBlueprintMetadata::MD_NativeBreakFunction);
						if (MetaData.Len() > 0)
						{
							if (const UFunction* NativeBreakFunction = FindObject<UFunction>(nullptr, *MetaData, EFindObjectFlags::ExactClass))
							{
								AddBreakNode(ArgumentStructProperty->Struct, NodePair.Get<UEdGraphPin*>()->GetFName());
								bAddPropertyPath = false;
							}
						}
					}

					if (bAddPropertyPath)
					{
						ResultPath.AppendPropertyPath(Blueprint, UE::MVVM::FMVVMConstFieldVariant(Function));
					}
				}
			}
			else if (UK2Node_BreakStruct* StructNode = Cast<UK2Node_BreakStruct>(Node))
			{
				if (ensure(StructNode->StructType))
				{
					AddBreakNode(StructNode->StructType, NodePair.Get<UEdGraphPin*>()->GetFName());
				}
			}
			else if (UK2Node_Self* Self = Cast<UK2Node_Self>(Node))
			{
				if (bFirst)
				{
					ResultPath.SetSelfContext();
				}
				else
				{
					ensure(false);
				}
			}
			bFirst = false;
		}

		return ResultPath;
	}

	UEdGraphPin* FindNewOutputPin(const UEdGraphNode* NewNode)
	{
		if (!NewNode)
		{
			return nullptr;
		}

		// then update our previous pin pointers
		for (UEdGraphPin* Pin : NewNode->Pins)
		{
			if (Pin->Direction == EGPD_Output)
			{
				if (Pin->PinName != UEdGraphSchema_K2::PN_Then && Pin->PinName != UEdGraphSchema_K2::PN_Completed)
				{
					return Pin;
				}
			}
		}
		return nullptr;
	};

	TValueOrError<TArray<UEdGraphPin*>, void> BuildPropertyPath(const UBlueprint* Blueprint, UEdGraph* FunctionGraph, const FMVVMBlueprintPropertyPath& PropertyPath, int32 NumberOfFieldExcludingThePropertyPathSource, FVector2f EndLocation)
	{
		// Add new nodes
		if (!PropertyPath.IsValid())
		{
			return MakeError();
		}

		auto CanNewConnections = [](UEdGraphPin* Pin, UEdGraphPin* PreviousDataPin, const UClass* Context)
			{
				return Pin->Direction == EGPD_Input
					&& Pin->PinName != UEdGraphSchema_K2::PN_Execute
					&& GetDefault<UMVVMConversionFunctionGraphSchema>()->ArePinsCompatible(PreviousDataPin, Pin, Context);
			};

		NumberOfFieldExcludingThePropertyPathSource = FMath::Clamp(NumberOfFieldExcludingThePropertyPathSource, 0, PropertyPath.GetFieldPaths().Num());

		const FVector2f LocationDelta = FVector2f(300.0f, 0.f);
		FVector2f Location = EndLocation;
		Location.X -= LocationDelta.X * (NumberOfFieldExcludingThePropertyPathSource + 1);
		UClass* BlueprintClass = Blueprint->SkeletonGeneratedClass ? Blueprint->SkeletonGeneratedClass : Blueprint->GeneratedClass;

		UEdGraphPin* PreviousDataPin = nullptr;
		UClass* PreviousClass = nullptr;
		const FProperty* PreviousProperty = nullptr;
		// create the root property getter node, ie. the Widget/ViewModel
		{
			const FProperty* RootProperty = nullptr;
			bool bCreateSelfNodeForRootProperty = false;
			{
				switch (PropertyPath.GetSource(Blueprint))
				{
				case EMVVMBlueprintFieldPathSource::SelfContext:
					bCreateSelfNodeForRootProperty = true;
					break;
				case EMVVMBlueprintFieldPathSource::ViewModel:
				{
					UMVVMBlueprintView* View = Private::GetView(Blueprint);
					const FMVVMBlueprintViewModelContext* Context = View ? View->FindViewModel(PropertyPath.GetViewModelId()) : nullptr;
					RootProperty = Context ? Blueprint->SkeletonGeneratedClass->FindPropertyByName(Context->GetViewModelName()) : nullptr;
					ensureAlwaysMsgf(RootProperty, TEXT("Could not find root property %s in blueprint %s"), *Context->GetViewModelName().ToString(), *GetNameSafe(Blueprint));
					break;
				}
				case EMVVMBlueprintFieldPathSource::Widget:
					if (PropertyPath.IsComponent())
					{
						const UWidgetBlueprintGeneratedClass* ComponentSource = FieldPathHelper::GetComponentPropertyPathSource(PropertyPath.GetFields(BlueprintClass), Cast<UWidgetBlueprintGeneratedClass>(BlueprintClass));
						if (ComponentSource == BlueprintClass)
						{
							bCreateSelfNodeForRootProperty = true;
							break;
						}
					}

					RootProperty = Blueprint->SkeletonGeneratedClass->FindPropertyByName(PropertyPath.GetWidgetName());
					ensureAlwaysMsgf(RootProperty, TEXT("Could not find root property %s in blueprint %s"), *PropertyPath.GetWidgetName().ToString(), *GetNameSafe(Blueprint));
					break;
				default:
					check(false);
					return MakeError();
				}
			}

			if (bCreateSelfNodeForRootProperty)
			{
				FGraphNodeCreator<UK2Node_Self> RootGetterCreator(*FunctionGraph);
				UK2Node_Self* RootSelfNode = RootGetterCreator.CreateNode();
				RootSelfNode->NodePosX = Location.X;
				RootSelfNode->NodePosY = Location.Y;
				RootGetterCreator.Finalize();

				PreviousDataPin = RootSelfNode->FindPinChecked(UEdGraphSchema_K2::PSC_Self);
				PreviousDataPin->PinType.PinSubCategoryObject = Blueprint->SkeletonGeneratedClass;
				PreviousClass = Blueprint->SkeletonGeneratedClass;
				PreviousProperty = RootProperty;
			}
			else
			{
				if (RootProperty == nullptr)
				{
					ensureMsgf(false, TEXT("Could not resolve root property!"));
					return MakeError();
				}

				FGraphNodeCreator<UK2Node_VariableGet> RootGetterCreator(*FunctionGraph);
				UK2Node_VariableGet* RootGetterNode = RootGetterCreator.CreateNode();
				RootGetterNode->NodePosX = Location.X;
				RootGetterNode->NodePosY = Location.Y;
				RootGetterNode->VariableReference.SetFromField<FProperty>(RootProperty, true, BlueprintClass);
				RootGetterCreator.Finalize();

				PreviousDataPin = RootGetterNode->Pins[0];
				PreviousClass = CastField<FObjectProperty>(RootProperty)->PropertyClass;
				PreviousProperty = RootProperty;
			}

			Location += LocationDelta;
		}

		TArray<UEdGraphPin*> Result;
		Result.Reserve(NumberOfFieldExcludingThePropertyPathSource);
		Result.Add(PreviousDataPin);

		TArray<UE::MVVM::FMVVMConstFieldVariant> Fields = PropertyPath.GetFields(BlueprintClass);

		// create all the subsequent nodes in the path
		for (int32 Index = 0; Index < NumberOfFieldExcludingThePropertyPathSource; ++Index)
		{
			const UE::MVVM::FMVVMConstFieldVariant& Field = Fields[Index];
			UEdGraphNode* NewNode = nullptr;
			UEdGraphPin* NewPreviousDataPin = nullptr;
			UClass* NewPreviousClass = nullptr;
			const FProperty* NewPreviousProperty = nullptr;
			if (Field.IsProperty())
			{
				const FProperty* Property = Field.GetProperty();
				// for struct in the middle of a path, we need to use a break node
				if (const FStructProperty* PreviousStructProperty = CastField<FStructProperty>(PreviousProperty))
				{
					const FString& MetaData = PreviousStructProperty->Struct->GetMetaData(FBlueprintMetadata::MD_NativeBreakFunction);
					if (MetaData.Len() > 0)
					{
						if (const UFunction* Function = FindObject<UFunction>(nullptr, *MetaData, EFindObjectFlags::ExactClass))
						{
							FGraphNodeCreator<UK2Node_CallFunction> MakeStructCreator(*FunctionGraph);
							UK2Node_CallFunction* FunctionNode = MakeStructCreator.CreateNode(false);
							FunctionNode->SetFromFunction(Function);
							MakeStructCreator.Finalize();

							NewNode = FunctionNode;
							NewPreviousProperty = UE::MVVM::BindingHelper::GetReturnProperty(Function);
						}
						else
						{
							UE_LOGFMT(LogMVVM, Error, "BuildPropertyPath: a function in path couldn't be resolved. Blueprint '{Blueprint}' Function '{Function}'.",
								("Blueprint", Blueprint->GetName()),
								("Function", MetaData));
							return MakeError();
						}
					}
					else
					{
						FGraphNodeCreator<UK2Node_BreakStruct> BreakCreator(*FunctionGraph);
						UK2Node_BreakStruct* BreakNode = BreakCreator.CreateNode();
						BreakNode->StructType = PreviousStructProperty->Struct;
						BreakNode->AllocateDefaultPins();
						BreakCreator.Finalize();

						NewNode = BreakNode;
						NewPreviousProperty = Property;
					}

					NewPreviousDataPin = nullptr;
					for (UEdGraphPin* Pin : NewNode->Pins)
					{
						if (Pin->Direction == EGPD_Output)
						{
							if (Pin->PinName == Property->GetFName())
							{
								NewPreviousDataPin = Pin;
								break;
							}
						}
					}
				}
				else if (PreviousClass != nullptr)
				{
					FGraphNodeCreator<UK2Node_VariableGet> GetterCreator(*FunctionGraph);
					UK2Node_VariableGet* GetterNode = GetterCreator.CreateNode();
					GetterNode->SetFromProperty(Property, false, PreviousClass);
					GetterNode->AllocateDefaultPins();
					GetterCreator.Finalize();

					NewNode = GetterNode;
					NewPreviousProperty = Property;
					NewPreviousDataPin = FindNewOutputPin(NewNode);
				}
				else
				{
					UE_LOGFMT(LogMVVM, Error, "BuildPropertyPath: cannot create getter node — no previous class context for property '{Property}' in blueprint '{Blueprint}'.",
						("Property", Property ? Property->GetFName() : NAME_None),
						("Blueprint", GetNameSafe(Blueprint)));
					return MakeError();
				}
			}
			else if (Field.IsFunction())
			{
				if (CastField<FStructProperty>(PreviousProperty))
				{
					UE_LOGFMT(LogMVVM, Error, "BuildPropertyPath: cannot call a function on struct property '{Property}' in blueprint '{Blueprint}'.",
						("Property", PreviousProperty ? PreviousProperty->GetFName() : NAME_None),
						("Blueprint", GetNameSafe(Blueprint)));
					return MakeError();
				}

				if (const UFunction* Function = Field.GetFunction())
				{
					FGraphNodeCreator<UK2Node_CallFunction> CallFunctionCreator(*FunctionGraph);
					UK2Node_CallFunction* FunctionNode = CallFunctionCreator.CreateNode();
					FunctionNode->SetFromFunction(Function);
					FunctionNode->AllocateDefaultPins();
					CallFunctionCreator.Finalize();

					NewNode = FunctionNode;
					NewPreviousProperty = UE::MVVM::BindingHelper::GetReturnProperty(Function);
					NewPreviousDataPin = FindNewOutputPin(NewNode);
				}
				else
				{
					const TArrayView<const FMVVMBlueprintFieldPath> FieldPaths = PropertyPath.GetFieldPaths();
					FName FunctionName;
					if (FieldPaths.IsValidIndex(Index))
					{
						FunctionName = FieldPaths[Index].GetRawFieldName();
					}
					UE_LOGFMT(LogMVVM, Error, "BuildPropertyPath: a function in path couldn't be resolved. Blueprint '{Blueprint}' Function '{Function}'.",
						("Blueprint", Blueprint->GetName()),
						("Function", FunctionName));
					return MakeError();
				}
			}
			else
			{
				UE_LOGFMT(LogMVVM, Error, "BuildPropertyPath: invalid path, empty field in path for blueprint '{Blueprint}'.",
					("Blueprint", GetNameSafe(Blueprint)));
				return MakeError();
			}

			check(NewNode);
			NewNode->NodePosX = Location.X;
			NewNode->NodePosY = Location.Y;
			Location += LocationDelta;

			if (!NewPreviousDataPin)
			{
				UE_LOGFMT(LogMVVM, Error, "BuildPropertyPath: a node in path doesn't have a return value. Node '{Node}'.",
					("Node", GetPathNameSafe(NewNode)));
				return MakeError();
			}

			// create new data connections
			for (UEdGraphPin* Pin : NewNode->Pins)
			{
				if (CanNewConnections(Pin, PreviousDataPin, PreviousClass))
				{
					GetDefault<UMVVMConversionFunctionGraphSchema>()->TryCreateConnection(Pin, PreviousDataPin);
				}
			}

			if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(NewPreviousProperty))
			{
				NewPreviousClass = ObjectProperty->PropertyClass;
			}
			else
			{
				NewPreviousClass = nullptr;
			}

			// then update our previous pin pointers
			PreviousDataPin = NewPreviousDataPin;
			PreviousClass = NewPreviousClass;
			PreviousProperty = NewPreviousProperty;

			Result.Add(PreviousDataPin);
		}
		return MakeValue(MoveTemp(Result));
	}

	struct FCanSetterGraphResult
	{
		/**
		 * true when the path is ObjectA.StructB.StructC.PropertyD
		 * false when the path is ObjectA.PropertyD
		 */
		bool bSplitPin = false;
		/**
		 * 1 when the path is ObjectA.StructB.StructC.PropertyD
		 * 2 when the path is ObjectA.StructB.ObjectC.PropertyD
		 * Can be INDEX_NONE if self is the container.
		 */
		int32 LocalContainerPathIndex = INDEX_NONE;
	};
	TValueOrError<FCanSetterGraphResult, FText> CanCreateSetterGraph(UBlueprint* WidgetBlueprint, const TArrayView<FMVVMConstFieldVariant> Path, const bool bIsForEvent)
	{
		/**
		 * Different types:
		 *	(1). The container is a object and it set a property or it call a setter function.
		 *		propertyA/functionA
		 *		objectA.objectB.propertyC/functionC
		 *		objectA.structB.objectC.propertyD/functionD (here we will need to split the pin to get objectC)
		 *	(2). The container is a struct and we need to create a local variable. We can't have setter function here.
		 *		objectA.structB.propertyC
		 *		objectA.structB.structC.propertyD
		 * We cannot create graph for
		 *	ObjectA.GetterB, because GetterB is not a valid destination
		 *	ObjectA.GetterB.PropertyC, if GetterB returns a structure, because we cannot set the local container.
		 */

		if (Path.Num() == 0)
		{
			return MakeError(LOCTEXT("SetterGraph_NoPath", "The path is empty."));
		}

		// Decide the type.
		FCanSetterGraphResult Result;
		for (int32 Index = 0; Index < Path.Num() - 1; ++Index)
		{
			const FMVVMConstFieldVariant& Field = Path[Index];
			TValueOrError<const UStruct*, void> ContainerAsResult = FieldPathHelper::GetFieldAsContainer(Field);
			if (ContainerAsResult.HasError())
			{
				return MakeError(FText::Format(LOCTEXT("SetterGraph_NoContainer", "Can't find the container for field {0}."), FText::FromName(Field.GetName())));
			}

			// Getter needs to be BlueprintPure. No param.
			if (!BindingHelper::IsValidForSourceBinding(Field))
			{
				return MakeError(FText::Format(LOCTEXT("SetterGraph_InvalidGetter", "The getter for field {0} can't be used."), FText::FromName(Field.GetName())));
			}

			// Find the container
			if (const UClass* Class = Cast<const UClass>(ContainerAsResult.GetValue()))
			{
				// was a struct, now a class
				Result = FCanSetterGraphResult();
				Result.LocalContainerPathIndex = Index;
			}
			else if (const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(ContainerAsResult.GetValue()))
			{
				// was a class, now a struct
				if (!Result.bSplitPin)
				{
					Result.LocalContainerPathIndex = Index;
				}
				Result.bSplitPin = true;
			}
		}

		const FMVVMConstFieldVariant& LastField = Path.Last();
		if (bIsForEvent)
		{
			if (!BindingHelper::IsValidForEventBinding(LastField))
			{
				return MakeError(FText::Format(LOCTEXT("SetterGraph_InvalidSetter", "The setter for field {0} can't be used."), FText::FromName(LastField.GetName())));
			}
		}
		else if (!BindingHelper::IsValidForDestinationBinding(LastField))
		{
			return MakeError(FText::Format(LOCTEXT("SetterGraph_InvalidSetter", "The setter for field {0} can't be used."), FText::FromName(LastField.GetName())));
		}

		if (Result.bSplitPin)
		{
			// Can the local container be set.
			check(Path.IsValidIndex(Result.LocalContainerPathIndex));
			const FMVVMConstFieldVariant& Field = Path[Result.LocalContainerPathIndex];
			if (bIsForEvent)
			{
				if (!BindingHelper::IsValidForEventBinding(LastField))
				{
					return MakeError(FText::Format(LOCTEXT("SetterGraph_ContainerSetter", "The path contains a getter of a struct that can't be set. See field {0}."), FText::FromName(Field.GetName())));
				}
			}
			else if (!BindingHelper::IsValidForDestinationBinding(Field))
			{
				return MakeError(FText::Format(LOCTEXT("SetterGraph_ContainerSetter", "The path contains a getter of a struct that can't be set. See field {0}."), FText::FromName(Field.GetName())));
			}
		}

		return MakeValue(Result);
	}
	
	void RemoveGraph(UBlueprint* Blueprint, bool bAddToBlueprint, Private::FCreateGraphResult& CreateGraphInternalResult)
	{
		if (CreateGraphInternalResult.FunctionGraph && bAddToBlueprint)
		{
			Blueprint->FunctionGraphs.RemoveSingleSwap(CreateGraphInternalResult.FunctionGraph);
		}
	};

	UE::MVVM::ConversionFunctionHelper::FCreateGraphResult CreateCallFunctionAndLinkNodes(const TSubclassOf<UK2Node>& Node
		, UE::MVVM::ConversionFunctionHelper::FCreateGraphParams& InParams
		, UE::MVVM::ConversionFunctionHelper::Private::FCreateGraphResult& CreateGraphInternalResult
		, TFunctionRef<void(UK2Node*)> InitNodeCallback)
	{
		UE::MVVM::ConversionFunctionHelper::FCreateGraphResult Result;
		Result.NewGraph = CreateGraphInternalResult.FunctionGraph;
		Result.WrappedNode = nullptr;
		Result.bIsUbergraphPage = InParams.bCreateUbergraphPage;

		const UMVVMConversionFunctionGraphSchema* GraphSchema = GetDefault<UMVVMConversionFunctionGraphSchema>();

		UK2Node* CallFunctionNode = nullptr;
		{
			FGraphNodeCreator<UK2Node> CallFunctionCreator(*CreateGraphInternalResult.FunctionGraph);
			CallFunctionNode = CallFunctionCreator.CreateNode(true, Node);
			InitNodeCallback(CallFunctionNode);
			CallFunctionNode->NodePosX = 0;
			CallFunctionCreator.Finalize();
			Private::MarkAsConversionFunction(CallFunctionNode, CreateGraphInternalResult.FunctionGraph);
		}

		Result.WrappedNode = CallFunctionNode;
		Result.NamedNodes.Add(Private::NamedNodes::GeneratedCallFunction, CallFunctionNode);

		// Create return value pin
		UEdGraphPin* CallFunctionOutputPin = Private::FindNewOutputPin(CallFunctionNode);
		if (!Result.bIsUbergraphPage && CallFunctionOutputPin)
		{
			TSharedPtr<FUserPinInfo> PinInfo = MakeShared<FUserPinInfo>();
			PinInfo->PinType = CallFunctionOutputPin->PinType;
			PinInfo->PinName = CallFunctionOutputPin->GetFName();
			PinInfo->DesiredPinDirection = EGPD_Input;
			CreateGraphInternalResult.FunctionResult->UserDefinedPins.Add(PinInfo);
			CreateGraphInternalResult.FunctionResult->ReconstructNode();
		}

		// Make link Entry -> CallFunction || Entry -> Return
		if (!Result.bIsUbergraphPage)
		{
			UEdGraphPin* FunctionEntryThenPin = CreateGraphInternalResult.FunctionEntry->GetThenPin();
			UEdGraphPin* FunctionResultExecPin = CreateGraphInternalResult.FunctionResult->GetExecPin();

			if (!CallFunctionNode->IsNodePure())
			{
				UEdGraphPin* CallFunctionExecPin = CallFunctionNode->GetExecPin();
				UEdGraphPin* CallFunctionThenPin = CallFunctionNode->GetThenPin();

				GraphSchema->TryCreateConnection(FunctionEntryThenPin, CallFunctionExecPin); 
				GraphSchema->TryCreateConnection(CallFunctionThenPin, FunctionResultExecPin);

				CallFunctionNode->NodePosY = 0;
			}
			else
			{
				GraphSchema->TryCreateConnection(FunctionEntryThenPin, FunctionResultExecPin);
				CallFunctionNode->NodePosY = 100;
			}
		}
		else
		{
			UEdGraphPin* EventEntryThenPin = CreateGraphInternalResult.EventEntry->GetThenPin();

			if (!CallFunctionNode->IsNodePure())
			{
				UEdGraphPin* CallFunctionExecPin = CallFunctionNode->GetExecPin();
				UEdGraphPin* CallFunctionThenPin = CallFunctionNode->GetThenPin();

				GraphSchema->TryCreateConnection(EventEntryThenPin, CallFunctionExecPin);

				CallFunctionNode->NodePosY = 0;
			}
		}

		if (!Result.bIsUbergraphPage && CallFunctionOutputPin)
		{
			UEdGraphPin* FunctionResultPin = CreateGraphInternalResult.FunctionResult->FindPin(CallFunctionOutputPin->GetFName(), EGPD_Input);
			check(FunctionResultPin);

			GraphSchema->TryCreateConnection(CallFunctionOutputPin, FunctionResultPin);
		}

		return Result;
	}

	const TValueOrError<UE::MVVM::ConversionFunctionHelper::FCreateGraphResult, FText> CreateSetterAndLinkNodes(const FMVVMBlueprintPropertyPath& PropertyPath
		, UBlueprint* Blueprint
		, UE::MVVM::ConversionFunctionHelper::FCreateGraphParams& InParams
		, UE::MVVM::ConversionFunctionHelper::FCreateGraphResult& Result
		, UE::MVVM::ConversionFunctionHelper::Private::FCanSetterGraphResult& CanCreateSetterGraphResult
		, UE::MVVM::ConversionFunctionHelper::Private::FCreateGraphResult& CreateGraphInternalResult)
	{
		TArray<UE::MVVM::FMVVMConstFieldVariant> Fields = PropertyPath.GetCompleteFields(Blueprint);

		const bool bIsEditable = false;
		const bool bAddToBlueprint = !InParams.bTransient;

		Result.bIsUbergraphPage = InParams.bCreateUbergraphPage;

		const UMVVMConversionFunctionGraphSchema* GraphSchema = GetDefault<UMVVMConversionFunctionGraphSchema>();

		// Create the wrapper setter node
		UK2Node* SetterNode = nullptr;
		{
			int32 WrapperFieldIndex = CanCreateSetterGraphResult.bSplitPin ? CanCreateSetterGraphResult.LocalContainerPathIndex : (CanCreateSetterGraphResult.LocalContainerPathIndex + 1);
			check(Fields.IsValidIndex(WrapperFieldIndex));
			const FMVVMConstFieldVariant& WrapperField = Fields[WrapperFieldIndex];

			if (WrapperField.IsProperty())
			{
				FGraphNodeCreator<UK2Node_VariableSet> CallFunctionCreator(*CreateGraphInternalResult.FunctionGraph);
				UK2Node_VariableSet* VariableNode = CallFunctionCreator.CreateNode(false);
				UEdGraphSchema_K2::ConfigureVarNode(VariableNode, WrapperField.GetName(), WrapperField.GetOwner(), Blueprint);
				VariableNode->NodePosX = Result.bIsUbergraphPage ? 500 : 0;
				CallFunctionCreator.Finalize();

				SetterNode = VariableNode;
			}
			else
			{
				check(!CanCreateSetterGraphResult.bSplitPin);
				check(WrapperField.IsFunction());

				FGraphNodeCreator<UK2Node_CallFunction> CallFunctionCreator(*CreateGraphInternalResult.FunctionGraph);
				UK2Node_CallFunction* CallFunctionNode = CallFunctionCreator.CreateNode();
				CallFunctionNode->SetFromFunction(WrapperField.GetFunction());
				CallFunctionNode->NodePosX = Result.bIsUbergraphPage ? 500 : 0;
				CallFunctionCreator.Finalize();

				SetterNode = CallFunctionNode;
			}

			// If we are not creating a setter for an ubergraph page the relevant node is the setter node
			if (!Result.bIsUbergraphPage)
			{
				Result.WrappedNode = SetterNode;
			}

			Result.NamedNodes.Add(Private::NamedNodes::GeneratedSetter, SetterNode);
			Private::MarkAsConversionFunction(Result.WrappedNode, Result.NewGraph);
		}

		// Make link Entry -> CallFunction || Entry -> Return
		if (CreateGraphInternalResult.FunctionResult)
		{
			UEdGraphPin* FunctionEntryThenPin = InParams.bCreateUbergraphPage ? CreateGraphInternalResult.EventEntry->GetThenPin() : CreateGraphInternalResult.FunctionEntry->GetThenPin();
			UEdGraphPin* FunctionResultExecPin = CreateGraphInternalResult.FunctionResult->GetExecPin();

			if (SetterNode->IsNodePure())
			{
				GraphSchema->TryCreateConnection(FunctionEntryThenPin, FunctionResultExecPin);
			}
			else
			{
				GraphSchema->TryCreateConnection(FunctionEntryThenPin, SetterNode->GetExecPin());
				GraphSchema->TryCreateConnection(SetterNode->GetThenPin(), FunctionResultExecPin);

				SetterNode->NodePosY = 0;
			}
		}

		// Build the path to get the container to get to the wrapper.
		UEdGraphPin* SplitPin = nullptr;
		if (ensure(Fields.Num() > 0))
		{
			UEdGraphPin* WrapperSelfPin = SetterNode->FindPinChecked(UEdGraphSchema_K2::PSC_Self);
			int32 ArgumentIndex = SetterNode->Pins.IndexOfByKey(WrapperSelfPin);
			const float PosX = SetterNode->NodePosX;
			const float PosY = SetterNode->NodePosY + ArgumentIndex * 100.0f;

			EMVVMBlueprintFieldPathSource PropertyPathSource = PropertyPath.GetSource(Blueprint);
			int32 NumberOfFieldExcludingThePropertyPathSource = PropertyPath.IsComponent() || PropertyPathSource == EMVVMBlueprintFieldPathSource::SelfContext ? CanCreateSetterGraphResult.LocalContainerPathIndex + 1 : CanCreateSetterGraphResult.LocalContainerPathIndex;
			TValueOrError<TArray<UEdGraphPin*>, void> PropertyPathPinsResult = Private::BuildPropertyPath(Blueprint, CreateGraphInternalResult.FunctionGraph, PropertyPath, NumberOfFieldExcludingThePropertyPathSource, FVector2f(PosX, PosY));
			if (PropertyPathPinsResult.HasError())
			{
				Private::RemoveGraph(Blueprint, bAddToBlueprint, CreateGraphInternalResult);
				return MakeError(LOCTEXT("SetterGraph_BuildPropertyPathFail", "Can't build the path to the setter."));
			}

			// Link the last data pin to the Conversation Function Pin
			int32 DeltaIndex = CanCreateSetterGraphResult.bSplitPin ? -1 : 0;
			int32 WrapperPinIndex = PropertyPathPinsResult.GetValue().Num() - 1 + DeltaIndex;
			UEdGraphPin* ToLinkPin = PropertyPathPinsResult.GetValue()[WrapperPinIndex];

			GraphSchema->TryCreateConnection(ToLinkPin, WrapperSelfPin);

			SplitPin = CanCreateSetterGraphResult.bSplitPin ? PropertyPathPinsResult.GetValue().Last() : nullptr;
		}

		// If the path continue, split the pins and make the link
		if (CanCreateSetterGraphResult.bSplitPin && SplitPin)
		{
			// need to break the pin
			UK2Node* CurrentNode = SetterNode;
			check(CurrentNode);
			TArrayView<UEdGraphPin*> InPins = CurrentNode->Pins;
			TArrayView<UEdGraphPin*> OutPins = TArrayView<UEdGraphPin*>(&SplitPin, 1);
			FName PinName;
			for (int32 Index = CanCreateSetterGraphResult.LocalContainerPathIndex; Index < Fields.Num(); ++Index)
			{
				const FMVVMConstFieldVariant& NewWrapperField = Fields[Index];

				if (PinName.IsNone())
				{
					PinName = NewWrapperField.GetName();
				}
				else
				{
					PinName = FName(*FString::Printf(TEXT("%s_%s"), *PinName.ToString(), *NewWrapperField.GetName().ToString()));
					for (int32 SubPinIndex = 0; SubPinIndex < InPins.Num(); ++SubPinIndex)
					{
						if (OutPins[SubPinIndex]->GetFName() != PinName)
						{
							GraphSchema->TryCreateConnection(OutPins[SubPinIndex], InPins[SubPinIndex]);
							InPins[SubPinIndex]->bHidden = true;
						}
					}
				}

				bool bLastItem = Index == Fields.Num() - 1;
				if (!bLastItem)
				{
					UEdGraphPin** FoundInPinPtr = InPins.FindByPredicate([PinName](const UEdGraphPin* Other) { return Other->GetFName() == PinName && Other->Direction == EGPD_Input && !Other->bHidden; });
					UEdGraphPin** FoundOutPinPtr = OutPins.FindByPredicate([PinName](const UEdGraphPin* Other) { return Other->GetFName() == PinName && Other->Direction == EGPD_Output && !Other->bHidden; });
					if (FoundInPinPtr == nullptr
						|| FoundOutPinPtr == nullptr
						|| !GraphSchema->CanSplitStructPin(**FoundInPinPtr)
						|| !GraphSchema->CanSplitStructPin(**FoundOutPinPtr))
					{
						Private::RemoveGraph(Blueprint, bAddToBlueprint, CreateGraphInternalResult);
						return MakeError(FText::Format(LOCTEXT("SetterGraph_CantSplitPin", "The pin {0} can't be split."), FText::FromName(NewWrapperField.GetName())));
					}

					GraphSchema->SplitPin(*FoundInPinPtr, false);
					GraphSchema->SplitPin(*FoundOutPinPtr, false);

					InPins = (*FoundInPinPtr)->SubPins;
					OutPins = (*FoundOutPinPtr)->SubPins;
				}
			}
		}

		return MakeValue(MoveTemp(Result));
	}

	void RemoveNodesFromPin(UEdGraph* FunctionGraph, UEdGraphPin* PathPin)
	{
		TArray<TTuple<UEdGraphNode*, UEdGraphPin*>> AllNodesForPath = GetPropertyPathGraphNode(PathPin);
		for (const TTuple<UEdGraphNode*, UEdGraphPin*>& Pair : AllNodesForPath)
		{
			UEdGraphNode* Node = Pair.Get<UEdGraphNode*>();
			FunctionGraph->RemoveNode(Node, true);
		}
	}

	void LinkAllNodes(UEdGraph* FunctionGraph, UEdGraphPin* EntryPin, UEdGraphNode* Wrapper, UK2Node_FunctionResult* FunctionResult)
	{
		check(FunctionGraph && EntryPin && Wrapper && FunctionResult);
		const UEdGraphSchema* GraphSchema = GetDefault<UMVVMConversionFunctionGraphSchema>();

		UEdGraphPin* ThenPin = EntryPin;
		check(ThenPin);
		ThenPin->BreakAllPinLinks();

		// Break Pins for every pins
		for (UEdGraphPin* Pin : Wrapper->Pins)
		{
			TArray<TTuple<UEdGraphNode*, UEdGraphPin*>> AllNodesForPath = Private::GetPropertyPathGraphNode(Pin);
			for (const TTuple<UEdGraphNode*, UEdGraphPin*>& Pair : AllNodesForPath)
			{
				UEdGraphNode* PathNode = Pair.Get<UEdGraphNode*>();
				if (UK2Node_CallFunction* CallFunction = Cast<UK2Node_CallFunction>(PathNode))
				{
					// if it not a pure node
					if (UEdGraphPin* ExecPin = CallFunction->FindPin(UEdGraphSchema_K2::PN_Execute))
					{
						ThenPin->BreakAllPinLinks();
						GraphSchema->TryCreateConnection(ThenPin, ExecPin);
						ThenPin = CallFunction->FindPinChecked(UEdGraphSchema_K2::PN_Then);
					}
				}
			}
		}

		// Make pin to the conversion node or to the return node
		if (UEdGraphPin* CallFunctionExecPin = Wrapper->FindPin(UEdGraphSchema_K2::PN_Execute))
		{
			GraphSchema->TryCreateConnection(ThenPin, CallFunctionExecPin);
		}
		else
		{
			UEdGraphPin* FunctionResultExecPin = FunctionResult->GetExecPin();
			GraphSchema->TryCreateConnection(ThenPin, FunctionResultExecPin);
		}
	}

	void LinkAllNodesForEvent(UEdGraph* FunctionGraph, UK2Node_Event* EventEntry, UEdGraphNode* Wrapper)
	{
		check(FunctionGraph && EventEntry && Wrapper);
		const UEdGraphSchema* GraphSchema = GetDefault<UMVVMConversionFunctionGraphSchema>();

		UEdGraphPin* ThenPin = EventEntry->FindPinChecked(UEdGraphSchema_K2::PN_Then);
		check(ThenPin);
		ThenPin->BreakAllPinLinks();

		// Break Pins for every pins
		for (UEdGraphPin* Pin : Wrapper->Pins)
		{
			TArray<TTuple<UEdGraphNode*, UEdGraphPin*>> AllNodesForPath = Private::GetPropertyPathGraphNode(Pin);
			for (const TTuple<UEdGraphNode*, UEdGraphPin*>& Pair : AllNodesForPath)
			{
				UEdGraphNode* PathNode = Pair.Get<UEdGraphNode*>();
				if (UK2Node_CallFunction* CallFunction = Cast<UK2Node_CallFunction>(PathNode))
				{
					// if it not a pure node
					if (UEdGraphPin* ExecPin = CallFunction->FindPin(UEdGraphSchema_K2::PN_Execute))
					{
						ThenPin->BreakAllPinLinks();
						GraphSchema->TryCreateConnection(ThenPin, ExecPin);
						ThenPin = CallFunction->FindPinChecked(UEdGraphSchema_K2::PN_Then);
					}
				}
			}
		}

		// Make pin to the conversion node or to the return node
		if (UEdGraphPin* CallFunctionExecPin = Wrapper->FindPin(UEdGraphSchema_K2::PN_Execute))
		{
			GraphSchema->TryCreateConnection(ThenPin, CallFunctionExecPin);
		}
	}
} //namespace

bool RequiresWrapper(const UFunction* ConversionFunction)
{
	if (ConversionFunction == nullptr)
	{
		return false;
	}

	TValueOrError<TArray<const FProperty*>, FText> ArgumentsResult = UE::MVVM::BindingHelper::TryGetArgumentsForConversionFunction(ConversionFunction);
	if (ArgumentsResult.HasValue())
	{
		return (ArgumentsResult.GetValue().Num() > 1);
	}
	return false;
}

bool IsInputPin(const UEdGraphPin* Pin)
{
	return Private::IsSystemInputPin(Pin) && Pin->PinName != UEdGraphSchema_K2::PN_Self;
}

FName CreateWrapperName(const FMVVMBlueprintViewBinding& Binding, bool bSourceToDestination)
{
	TStringBuilder<256> StringBuilder;
	StringBuilder << TEXT("__");
	StringBuilder << Binding.GetFName();
	StringBuilder << (bSourceToDestination ? TEXT("_SourceToDest") : TEXT("_DestToSource"));

	return FName(StringBuilder.ToString());
}

TValueOrError<void, FText> CanCreateSetterGraph(UBlueprint* Blueprint, const FMVVMBlueprintPropertyPath& PropertyPath)
{
	TArray<UE::MVVM::FMVVMConstFieldVariant> Fields = PropertyPath.GetCompleteFields(Blueprint);
	TValueOrError<Private::FCanSetterGraphResult, FText> Result = Private::CanCreateSetterGraph(Blueprint, Fields, false);
	if (Result.HasError())
	{
		return MakeError(Result.StealError());
	}
	return MakeValue();
}

TValueOrError<FCreateGraphResult, FText> CreateSetterGraph(UBlueprint* Blueprint, FName GraphName, const UFunction* Signature, const FMVVMBlueprintPropertyPath& PropertyPath, FCreateGraphParams InParams)
{
	TArray<UE::MVVM::FMVVMConstFieldVariant> Fields = PropertyPath.GetCompleteFields(Blueprint);
	TValueOrError<Private::FCanSetterGraphResult, FText> CanCreateSetterGraphResult = Private::CanCreateSetterGraph(Blueprint, Fields, InParams.bIsForEvent);
	if (CanCreateSetterGraphResult.HasError())
	{
		return MakeError(CanCreateSetterGraphResult.StealError());
	}

	const bool bIsEditable = false;
	const bool bAddToBlueprint = !InParams.bTransient;

	Private::FCreateGraphParams CreateGraphParams;
	CreateGraphParams.bIsConst = InParams.bIsConst;
	CreateGraphParams.bIsEditable = bIsEditable;
	CreateGraphParams.bAddToBlueprint = bAddToBlueprint;

	Private::FCreateGraphResult CreateGraphInternalResult = Private::CreateGraph(Blueprint, GraphName, Signature, CreateGraphParams);

	if (CreateGraphInternalResult.FunctionGraph == nullptr || CreateGraphInternalResult.FunctionEntry == nullptr)
	{
		Private::RemoveGraph(Blueprint, bAddToBlueprint, CreateGraphInternalResult);
		return MakeError(LOCTEXT("SetterGraph_CreateGraphFail", "Can create the graph object."));
	}

	FCreateGraphResult Result;
	Result.NewGraph = CreateGraphInternalResult.FunctionGraph;
	Result.WrappedNode = nullptr;

	return Private::CreateSetterAndLinkNodes(PropertyPath, Blueprint, InParams, Result, CanCreateSetterGraphResult.GetValue(), CreateGraphInternalResult);
}

TValueOrError<FCreateGraphResult, FText> CreateSetterGraph(UBlueprint* Blueprint, FName GraphName, const TSubclassOf<UK2Node> Node, const FMVVMBlueprintPropertyPath& PropertyPath, FCreateGraphParams InParams)
{
	TArray<UE::MVVM::FMVVMConstFieldVariant> Fields = PropertyPath.GetCompleteFields(Blueprint);
	TValueOrError<Private::FCanSetterGraphResult, FText> CanCreateSetterGraphResult = Private::CanCreateSetterGraph(Blueprint, Fields, InParams.bIsForEvent);
	if (CanCreateSetterGraphResult.HasError())
	{
		return MakeError(CanCreateSetterGraphResult.StealError());
	}

	const bool bIsEditable = false;
	const bool bAddToBlueprint = !InParams.bTransient;

	Private::FCreateGraphParams CreateGraphParams;
	CreateGraphParams.bIsConst = InParams.bIsConst;
	CreateGraphParams.bIsEditable = bIsEditable;
	CreateGraphParams.bAddToBlueprint = bAddToBlueprint;
	CreateGraphParams.bCreateUbergraphPage = InParams.bCreateUbergraphPage;

	Private::FCreateGraphResult CreateGraphInternalResult = Private::CreateGraph(Blueprint, GraphName, nullptr, CreateGraphParams);

	if (CreateGraphInternalResult.FunctionGraph == nullptr || (!CreateGraphInternalResult.FunctionEntry && !CreateGraphInternalResult.EventEntry))
	{
		Private::RemoveGraph(Blueprint, bAddToBlueprint, CreateGraphInternalResult);
		return MakeError(LOCTEXT("SetterGraph_CreateGraphFail", "Can create the graph object."));
	}

	FCreateGraphResult Result = CreateCallFunctionAndLinkNodes(Node, InParams, CreateGraphInternalResult, [](UK2Node*) {});

	// Create the wrapper setter node
	TValueOrError<FCreateGraphResult, FText> LinkedSetterGraph = Private::CreateSetterAndLinkNodes(PropertyPath, Blueprint, InParams, Result, CanCreateSetterGraphResult.GetValue(), CreateGraphInternalResult);

	const UMVVMAsyncConversionFunctionGraphSchema* GraphSchema = GetDefault<UMVVMAsyncConversionFunctionGraphSchema>();

	if (!LinkedSetterGraph.HasError())
	{
		UK2Node** SetterNodeItr = LinkedSetterGraph.GetValue().NamedNodes.Find(Private::NamedNodes::GeneratedSetter);
		UK2Node** CallFunctionNodeItr = LinkedSetterGraph.GetValue().NamedNodes.Find(Private::NamedNodes::GeneratedCallFunction);

		if (SetterNodeItr && CallFunctionNodeItr)
		{
			UK2Node* SetterNode = *SetterNodeItr;
			UK2Node* CallFunctionNode = *CallFunctionNodeItr;
			UEdGraphPin* AsyncCompletedPin = CallFunctionNode->FindPinChecked(UEdGraphSchema_K2::PN_Completed);

			// Link the Async result Exec with the setter, adding a valid binding check should sources expire after the async
			if (AsyncCompletedPin)
			{
				UMVVMK2Node_AreSourcesValidForBinding* BranchNode = nullptr;
				{
					FGraphNodeCreator<UMVVMK2Node_AreSourcesValidForBinding> BranchNodeCreator(*LinkedSetterGraph.GetValue().NewGraph);
					BranchNode = BranchNodeCreator.CreateNode(false, UMVVMK2Node_AreSourcesValidForBinding::StaticClass());
					BranchNode->NodePosX = SetterNode->NodePosX;
					BranchNode->NodePosY = SetterNode->NodePosY - 300;
					BranchNodeCreator.Finalize();
				}

				if (ensure(BranchNode))
				{
					GraphSchema->TryCreateConnection(AsyncCompletedPin, BranchNode->GetExecPin());
					GraphSchema->TryCreateConnection(BranchNode->GetThenPin(), SetterNode->GetExecPin());
				}
			}

			// Link the Async result value with the setter args
			if (AsyncCompletedPin)
			{
				UClass* BlueprintClass = Blueprint->SkeletonGeneratedClass ? Blueprint->SkeletonGeneratedClass : Blueprint->GeneratedClass;

				if (UEdGraphPin* SetterParamPin = SetterNode->FindPin(PropertyPath.GetFieldNames(BlueprintClass).Last()))
				{
					GraphSchema->TryCreateConnection(CallFunctionNode->GetAllPins().Last(), SetterParamPin);
				}
				else
				{
					// @TODO: UE-221370 - Find a more robust way to link to multiple function args. Maybe enforce functions to have one arg for now if async.
					GraphSchema->TryCreateConnection(CallFunctionNode->GetAllPins().Last(), SetterNode->GetAllPins().Last());
				}

			}
		}
	}

	return LinkedSetterGraph;
}

FCreateGraphResult CreateGraph(UBlueprint* Blueprint, FName GraphName, const UFunction* Signature, const UFunction* FunctionToWrap, FCreateGraphParams InParams)
{
	bool bIsEditable = false;
	bool bAddToBlueprint = !InParams.bTransient;
	bool bIsUbergraphPage = false;

	Private::FCreateGraphParams CreateGraphParams;
	CreateGraphParams.bIsConst = InParams.bIsConst;
	CreateGraphParams.bIsEditable = bIsEditable;
	CreateGraphParams.bAddToBlueprint = bAddToBlueprint;
	CreateGraphParams.bCreateUbergraphPage = bIsUbergraphPage;

	Private::FCreateGraphResult NewGraph = Private::CreateGraph(Blueprint, GraphName, Signature, CreateGraphParams);
	const UMVVMConversionFunctionGraphSchema* GraphSchema = GetDefault<UMVVMConversionFunctionGraphSchema>();

	// create return value pin
	const FProperty* ReturnProperty = UE::MVVM::BindingHelper::GetReturnProperty(FunctionToWrap);
	if (ReturnProperty)
	{
		TSharedPtr<FUserPinInfo> PinInfo = MakeShared<FUserPinInfo>();
		GraphSchema->ConvertPropertyToPinType(ReturnProperty, PinInfo->PinType);
		PinInfo->PinName = ReturnProperty->GetFName();
		PinInfo->DesiredPinDirection = EGPD_Input;
		NewGraph.FunctionResult->UserDefinedPins.Add(PinInfo);
		NewGraph.FunctionResult->ReconstructNode();
	}
	
	UK2Node_CallFunction* CallFunctionNode = nullptr;
	{
		FGraphNodeCreator<UK2Node_CallFunction> CallFunctionCreator(*NewGraph.FunctionGraph);
		CallFunctionNode = CallFunctionCreator.CreateNode();
		CallFunctionNode->SetFromFunction(FunctionToWrap);
		CallFunctionNode->NodePosX = 0;
		CallFunctionCreator.Finalize();
		Private::MarkAsConversionFunction(CallFunctionNode, NewGraph.FunctionGraph);
	}

	// Make link Entry -> CallFunction || Entry -> Return
	{
		UEdGraphPin* FunctionEntryThenPin = NewGraph.FunctionEntry->GetThenPin();
		UEdGraphPin* FunctionResultExecPin = NewGraph.FunctionResult->GetExecPin();
		
		if (!CallFunctionNode->IsNodePure())
		{
			UEdGraphPin* CallFunctionExecPin = CallFunctionNode->GetExecPin();
			UEdGraphPin* CallFunctionThenPin = CallFunctionNode->GetThenPin();

			GraphSchema->TryCreateConnection(FunctionEntryThenPin, CallFunctionExecPin);
			GraphSchema->TryCreateConnection(CallFunctionThenPin, FunctionResultExecPin);

			CallFunctionNode->NodePosY = 0;
		}
		else
		{
			GraphSchema->TryCreateConnection(FunctionEntryThenPin, FunctionResultExecPin);
			CallFunctionNode->NodePosY = 100;
		}
	}

	if (ReturnProperty)
	{
		UEdGraphPin* FunctionReturnPin = CallFunctionNode->FindPin(ReturnProperty->GetName(), EGPD_Output);
		UEdGraphPin* FunctionResultPin = NewGraph.FunctionResult->FindPin(ReturnProperty->GetFName(), EGPD_Input);
		check(FunctionResultPin && FunctionReturnPin);
		GraphSchema->TryCreateConnection(FunctionReturnPin, FunctionResultPin);
	}

	return FCreateGraphResult{ NewGraph.FunctionGraph , CallFunctionNode, TMap<FName, UK2Node*>{}, bIsUbergraphPage };
}

FCreateGraphResult CreateGraph(UBlueprint* Blueprint, FName GraphName, const UFunction* Signature, TSubclassOf<UK2Node> NodeType, FCreateGraphParams InParams, TFunctionRef<void(UK2Node*)> InitNodeCallback)
{
	bool bIsEditable = false;
	bool bAddToBlueprint = !InParams.bTransient;

	Private::FCreateGraphParams CreateGraphParams;
	CreateGraphParams.bIsConst = InParams.bIsConst;
	CreateGraphParams.bIsEditable = bIsEditable;
	CreateGraphParams.bAddToBlueprint = bAddToBlueprint;
	CreateGraphParams.bCreateUbergraphPage = InParams.bCreateUbergraphPage;

	Private::FCreateGraphResult CreateGraphInternalResult = Private::CreateGraph(Blueprint, GraphName, Signature, CreateGraphParams);
	FCreateGraphResult Result = CreateCallFunctionAndLinkNodes(NodeType, InParams, CreateGraphInternalResult, InitNodeCallback);

	return Result;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

TValueOrError<FCreateGraphResult, FText> CreateSetterGraph(UBlueprint* WidgetBlueprint, FName GraphName, const UFunction* Signature, const FMVVMBlueprintPropertyPath& PropertyPath, bool bIsConst, bool bTransient, const bool bIsForEvent)
{
	FCreateGraphParams Params;
	Params.bIsConst = bIsConst;
	Params.bTransient = bTransient;
	Params.bIsForEvent = bIsForEvent;

	return CreateSetterGraph(WidgetBlueprint, GraphName, Signature, PropertyPath, Params);
}

FCreateGraphResult CreateGraph(UBlueprint* WidgetBlueprint, FName GraphName, const UFunction* Signature, const UFunction* FunctionToWrap, bool bIsConst, bool bTransient)
{
	FCreateGraphParams Params;
	Params.bIsConst = bIsConst;
	Params.bTransient = bTransient;

	return CreateGraph(WidgetBlueprint, GraphName, Signature, FunctionToWrap, Params);
}

FCreateGraphResult CreateGraph(UBlueprint* WidgetBlueprint, FName GraphName, const UFunction* Signature, const TSubclassOf<UK2Node> Node, bool bIsConst, bool bTransient, TFunctionRef<void(UK2Node*)> InitNodeCallback)
{
	FCreateGraphParams Params;
	Params.bIsConst = bIsConst;
	Params.bTransient = bTransient;

	return CreateGraph(WidgetBlueprint, GraphName, Signature, Node, Params, InitNodeCallback);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

UK2Node* InsertEarlyExitBranchNode(UEdGraph* Graph, TSubclassOf<UK2Node> BranchNodeType)
{
	if (Graph == nullptr || BranchNodeType.Get() == nullptr)
	{
		return nullptr;
	}

	UK2Node_FunctionEntry* FunctionEntry = Private::FindFunctionEntry(Graph);
	if (FunctionEntry == nullptr)
	{
		return nullptr;
	}

	UK2Node* BranchNode = nullptr;
	{
		FGraphNodeCreator<UK2Node> BranchNodeCreator(*Graph);
		BranchNode = BranchNodeCreator.CreateNode(true, BranchNodeType.Get());
		BranchNode->NodePosX = FunctionEntry->NodePosX;
		BranchNode->NodePosY = FunctionEntry->NodePosY+100;
		BranchNodeCreator.Finalize();
	}

	const UEdGraphSchema* GraphSchema = GetDefault<UMVVMConversionFunctionGraphSchema>();
	{
		UEdGraphPin* EntryThenPin = FunctionEntry->FindPinChecked(UEdGraphSchema_K2::PN_Then);
		UEdGraphPin* BranchThenPin = BranchNode->FindPinChecked(UEdGraphSchema_K2::PN_Then, EEdGraphPinDirection::EGPD_Output);
		GraphSchema->MovePinLinks(*EntryThenPin, *BranchThenPin, true, false);

		UEdGraphPin* BranchInputPin = BranchNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute, EEdGraphPinDirection::EGPD_Input);
		GraphSchema->TryCreateConnection(EntryThenPin, BranchInputPin);
	}

	return BranchNode;
}

UK2Node* GetWrapperNode(const UEdGraph* Graph)
{
	if (Graph == nullptr)
	{
		return nullptr;
	}

	FName ConversionFunctionMetadataKey = Private::ConversionFunctionMetadataKey.Resolve();
	for (UEdGraphNode* Node : Graph->Nodes)
	{
		// check if we've set any metadata on the nodes to figure out which one it is
		if (Cast<UK2Node>(Node) && Node->GetPackage()->GetMetaData().HasValue(Node, ConversionFunctionMetadataKey))
		{
			return CastChecked<UK2Node>(Node);
		}
	}

	if (UK2Node_FunctionResult* FunctionResult = Private::FindFunctionResult(Graph))
	{
		for (UEdGraphPin* GraphPin : FunctionResult->Pins)
		{
			if (GraphPin->GetFName() != UEdGraphSchema_K2::PN_Execute && GraphPin->LinkedTo.Num() == 1)
			{
				if (UK2Node* Node = Cast<UK2Node>(GraphPin->LinkedTo[0]->GetOwningNode()))
				{
					Private::MarkAsConversionFunction(Node, Graph);
					return Node;
				}
			}
		}
	}

	return nullptr;
}

FMVVMBlueprintPropertyPath GetPropertyPathForPin(const UBlueprint* Blueprint, const UEdGraphPin* Pin, bool bSkipResolve)
{
	if (Pin == nullptr || Pin->LinkedTo.Num() == 0)
	{
		return FMVVMBlueprintPropertyPath();
	}

	return Private::GetPropertyPathForPin(Blueprint, Pin, bSkipResolve);
}

void SetPropertyPathForPin(const UBlueprint* Blueprint, const FMVVMBlueprintPropertyPath& PropertyPath, UEdGraphPin* PathPin)
{
	if (PathPin == nullptr)
	{
		return;
	}

	UEdGraphNode* ConversionNode = PathPin->GetOwningNode();
	UEdGraph* FunctionGraph = ConversionNode ? ConversionNode->GetGraph() : nullptr;
	const UEdGraphSchema* Schema = GetDefault<UMVVMConversionFunctionGraphSchema>();

	UK2Node* K2ConversionNode = Cast<UK2Node>(ConversionNode);
	if (K2ConversionNode && FunctionGraph)
	{
		if (UE::MVVM::ConversionFunctionHelper::IsAsyncNode(K2ConversionNode->GetClass()))
		{
			UK2Node_Event* ConversionEventEntry = Private::FindEventEntry(FunctionGraph);

			// Remove previous nodes
			Private::RemoveNodesFromPin(FunctionGraph, PathPin);

			// Add new nodes
			if (PropertyPath.IsValid())
			{
				const int32 ArgumentIndex = K2ConversionNode->Pins.IndexOfByPredicate([PathPin](const UEdGraphPin* Other){ return Other == PathPin; });
				if (!ensure(K2ConversionNode->Pins.IsValidIndex(ArgumentIndex)))
				{
					return;
				}

				const int32 NumberOfFields = PropertyPath.IsComponent() ? PropertyPath.GetCompleteFields(Blueprint).Num() : PropertyPath.GetFieldPaths().Num();
				const float PosX = K2ConversionNode->NodePosX;
				const float PosY = K2ConversionNode->NodePosY + ArgumentIndex * 100;
				TValueOrError<TArray<UEdGraphPin*>, void> BuildPropertyPathResult = Private::BuildPropertyPath(Blueprint, FunctionGraph, PropertyPath, NumberOfFields, FVector2f(PosX, PosY));
				if (BuildPropertyPathResult.HasError())
				{
					return;
				}

				// Link the last data pin to the Conversation Function Pin
				Schema->TryCreateConnection(BuildPropertyPathResult.GetValue().Last(), PathPin);
			}

			// Link Then / Exec pin
			Private::LinkAllNodesForEvent(FunctionGraph, ConversionEventEntry, K2ConversionNode);
		}
	}

	if (!FunctionGraph || !ConversionNode)
	{
		return;
	}

	UK2Node_FunctionEntry* ConversionFunctionEntry = Private::FindFunctionEntry(FunctionGraph);
	UK2Node_FunctionResult* ConversionFunctionResult = Private::FindFunctionResult(FunctionGraph);

	if (!ConversionFunctionEntry || !ConversionFunctionResult)
	{
		return;
	}

	// Remove previous nodes
	Private::RemoveNodesFromPin(FunctionGraph, PathPin);

	UEdGraphPin* EntryPin = ConversionFunctionEntry->FindPin(UEdGraphSchema_K2::PN_Then);
	
	if (UEdGraphPin* ExecutePin = ConversionNode->FindPin(UEdGraphSchema_K2::PN_Execute))
	{
		if (!ExecutePin->LinkedTo.IsEmpty())
		{
			EntryPin = ExecutePin->LinkedTo[0];
		}
	}

	// Add new nodes
	if (PropertyPath.IsValid())
	{
		const int32 ArgumentIndex = ConversionNode->Pins.IndexOfByPredicate([PathPin](const UEdGraphPin* Other){ return Other == PathPin; });
		if (!ensure(ConversionNode->Pins.IsValidIndex(ArgumentIndex)))
		{
			return;
		}

		const int32 NumberOfFields = PropertyPath.IsComponent() ? PropertyPath.GetCompleteFields(Blueprint).Num() : PropertyPath.GetFieldPaths().Num();
		const float PosX = ConversionNode->NodePosX;
		const float PosY = ConversionNode->NodePosY + ArgumentIndex * 100;
		TValueOrError<TArray<UEdGraphPin*>, void> BuildPropertyPathResult = Private::BuildPropertyPath(Blueprint, FunctionGraph, PropertyPath, NumberOfFields, FVector2f(PosX, PosY));
		if (BuildPropertyPathResult.HasError())
		{
			return;
		}

		// Link the last data pin to the Conversation Function Pin
		Schema->TryCreateConnection(BuildPropertyPathResult.GetValue().Last(), PathPin);
	}

	// Link Then / Exec pin
	Private::LinkAllNodes(FunctionGraph, EntryPin, ConversionNode, ConversionFunctionResult);
}

FMVVMBlueprintPropertyPath GetPropertyPathForArgument(const UBlueprint* WidgetBlueprint, const UK2Node_CallFunction* FunctionNode, FName ArgumentName, bool bSkipResolve)
{
	const UEdGraphPin* ArgumentPin = FunctionNode->FindPin(ArgumentName, EGPD_Input);
	return GetPropertyPathForPin(WidgetBlueprint, ArgumentPin, bSkipResolve);
}

UEdGraphPin* FindPin(const UEdGraph* Graph, const TArrayView<const FName> PinNames)
{
	if (PinNames.Num() == 0 || Graph == nullptr)
	{
		return nullptr;
	}

	const UEdGraphNode* CurrentGraphNode = GetWrapperNode(Graph);
	return FindPin(CurrentGraphNode, PinNames);
}

UEdGraphPin* FindPin(const UEdGraphNode* Node, const TArrayView<const FName> PinNames)
{
	if (PinNames.Num() == 0 || Node == nullptr)
	{
		return nullptr;
	}

	for (int32 Index = 0; Index < PinNames.Num() - 1; ++Index)
	{
		FName PinName = PinNames[Index];
		const UEdGraphPin* Pin = Node->FindPin(PinName);
		if (Pin == nullptr || Pin->LinkedTo.Num() != 1)
		{
			return nullptr;
		}
		Node = Pin->LinkedTo[0]->GetOwningNode();
		if (Node == nullptr)
		{
			return nullptr;
		}
	}

	return Node ? Node->FindPin(PinNames.Last()) : nullptr;

}

TArray<FName> FindPinId(const UEdGraphPin* GraphPin)
{
	if (GraphPin == nullptr)
	{
		return TArray<FName>();
	}

	const UEdGraphNode* ConversionFunctionNode = GetWrapperNode(GraphPin->GetOwningNode()->GetGraph());
	if (ConversionFunctionNode == nullptr)
	{
		return TArray<FName>();
	}

	TArray<FName> Result;
	while (GraphPin)
	{
		Result.Insert(GraphPin->GetFName(), 0);
		const UEdGraphNode* CurrentGraphNode = GraphPin->GetOwningNode();
		if (ConversionFunctionNode == CurrentGraphNode)
		{
			break;
		}
		const UEdGraphPin* OutputPin = Private::FindNewOutputPin(CurrentGraphNode);
		if (OutputPin->LinkedTo.Num() != 1)
		{
			break;
		}
		GraphPin = OutputPin->LinkedTo[0];
	}
	return Result;
}

TArray<UEdGraphPin*> FindInputPins(const UK2Node* Node)
{
	TArray<UEdGraphPin*> Result;
	if (Node == nullptr)
	{
		return Result;
	}

	Result.Reserve(Node->Pins.Num());
	for (UEdGraphPin* GraphPin : Node->Pins)
	{
		if (IsInputPin(GraphPin))
		{
			Result.Add(GraphPin);
		}
	}
	return Result;
}

UEdGraphPin* FindOutputPin(const UK2Node* Node)
{
	return Node? Private::FindNewOutputPin(Node) : nullptr;
}

void SetMetaData(UEdGraph* NewGraph, FName MetaData, FStringView Value)
{
	if (NewGraph == nullptr || MetaData.IsNone())
	{
		return;
	}
	if (UK2Node_FunctionEntry* FunctionEntry = Private::FindFunctionEntry(NewGraph))
	{
		FunctionEntry->MetaData.SetMetaData(MetaData, Value);
	}
}

void MarkNodeAsAutoPromote(UEdGraphNode* Node)
{
	if (Node && !Node->NodeComment.Contains(Private::AutoPromoteFunctionMetadataKey))
	{
		Node->NodeComment.Append(Private::AutoPromoteFunctionMetadataKey);
	}
}

bool IsAutoPromoteNode(const UEdGraphNode* Node)
{
	return Node && Node->NodeComment.Contains(Private::AutoPromoteFunctionMetadataKey);
}

bool IsAsyncNode(const TSubclassOf<UK2Node> Node)
{
	// Note: Currently exists in MVVM as we would like for the kismet-level solution to not use CDO's.
	// CDO's may give false readings if they are conditionally async (Ex: CallFunction).

	if (!Node)
	{
		return false;
	}

	return Node->GetDefaultObject<UK2Node>()->IsCompatibleWithGraph(UMVVMFakeTestUbergraph::StaticClass()->GetDefaultObject<UEdGraph>())
		&& !Node->GetDefaultObject<UK2Node>()->IsCompatibleWithGraph(UMVVMFakeTestFunctiongraph::StaticClass()->GetDefaultObject<UEdGraph>());
}

void MarkNodeToKeepConnections(const UK2Node* Node)
{
}

bool IsNodeMarkedToKeepConnections(const UK2Node* Node)
{
	return false;
}

} // UE::MVVM::ConversionFunctionHelper

#undef LOCTEXT_NAMESPACE
