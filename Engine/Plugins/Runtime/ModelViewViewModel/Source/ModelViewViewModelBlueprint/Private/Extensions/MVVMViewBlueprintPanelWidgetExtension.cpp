// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/MVVMViewBlueprintPanelWidgetExtension.h"

#include "Bindings/MVVMCompiledBindingLibraryCompiler.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Components/PanelWidget.h"
#include "Extensions/MVVMViewPanelWidgetExtension.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewCompilerInterface.h"
#include "MVVMBlueprintViewModelContext.h"
#include "MVVMPropertyPath.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "Slate/SObjectWidget.h"
#include "Templates/ValueOrError.h"
#include "UObject/UnrealType.h"
#include "View/MVVMViewClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMViewBlueprintPanelWidgetExtension)

#define LOCTEXT_NAMESPACE "MVVMViewBlueprintPanelWidgetExtension"

TArray<UE::MVVM::Compiler::FBlueprintViewUserWidgetProperty> UMVVMBlueprintViewExtension_PanelWidget::AddProperties()
{
	TArray<UE::MVVM::Compiler::FBlueprintViewUserWidgetProperty> PropertiesToAdd;

	if (!WidgetName.IsNone())
	{
		// Add the runtime panel widget extension as a variable
		PanelPropertyName = FName(FString::Printf(TEXT("%s_%s"), *WidgetName.ToString(), *FString(TEXT("Viewmodel_Extension"))));
		UE::MVVM::Compiler::FBlueprintViewUserWidgetProperty Property;
		Property.AuthoritativeClass = UMVVMPanelWidgetViewExtension::StaticClass();
		Property.DisplayName = FText::FromName(PanelPropertyName);
		Property.Name = PanelPropertyName;
		Property.CategoryName = TEXT("PanelWidgetExtension");
		Property.bReadOnly = true;
		PropertiesToAdd.Add(Property);
	}

	return PropertiesToAdd;
}

TArray<UE::MVVM::Compiler::FBlueprintViewUserWidgetWidgetProperty> UMVVMBlueprintViewExtension_PanelWidget::AddWidgetProperties()
{
	TArray<UE::MVVM::Compiler::FBlueprintViewUserWidgetWidgetProperty> WidgetPropertiesToAdd;

	if (!WidgetName.IsNone())
	{
		// Add the panel widget as a variable
		UE::MVVM::Compiler::FBlueprintViewUserWidgetWidgetProperty WidgetProperty;
		WidgetProperty.WidgetName = WidgetName;
		WidgetPropertiesToAdd.Add(WidgetProperty);
	}

	return WidgetPropertiesToAdd;
}

void UMVVMBlueprintViewExtension_PanelWidget::Precompile(UE::MVVM::Compiler::IMVVMBlueprintViewPrecompile* Compiler, UWidgetBlueprintGeneratedClass* Class)
{
	check (Compiler);
	auto VerifyViewmodelTypeMatch = [this, Compiler](const TSubclassOf<UUserWidget> InEntryWidgetClass, const FMVVMBlueprintPropertyPath& EntryViewModelPath, const FName& PropertyName)
	{
		bool bFoundSetterBinding = false;

		// Find bindings that have write fields with metadata "ViewmodelBlueprintWidgetExtension", these are setter functions for assigning a viewmodel array.
		// Use their corresponding read field to get the expected type of entry viewmodel, and compare it against the one that user selected in the details panel.
		for (const UE::MVVM::Compiler::FCompilerBindingHandle& Binding : Compiler->GetAllBindings())
		{
			TArray<UE::MVVM::FMVVMConstFieldVariant> WriteFields = Compiler->GetBindingWriteFields(Binding);
			for (int32 FieldIndex = 1; FieldIndex < WriteFields.Num(); FieldIndex++)
			{
				UE::MVVM::FMVVMConstFieldVariant& PathField = WriteFields[FieldIndex];
				UE::MVVM::FMVVMConstFieldVariant& ParentPathField = WriteFields[FieldIndex - 1];
				if (PathField.IsFunction() && ParentPathField.IsProperty() && PropertyName == ParentPathField.GetName())
				{
					const FString MetaData = PathField.GetFunction()->GetMetaData("ViewmodelBlueprintWidgetExtension");
					if (MetaData.Equals("EntryViewmodel", ESearchCase::IgnoreCase))
					{
						if (const FProperty* SourceProperty = Compiler->GetBindingSourceProperty(Binding))
						{
							if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(SourceProperty))
							{
								// This is the viewmodel that the user selected from the details panel
								TArray<UE::MVVM::FMVVMConstFieldVariant> SelectedVMFields = EntryViewModelPath.GetFields(InEntryWidgetClass);
								if (SelectedVMFields.Num() > 0)
								{
									UE::MVVM::FMVVMConstFieldVariant SelectedVMPropertyField = SelectedVMFields.Last();

									if (SelectedVMPropertyField.IsProperty())
									{
										bFoundSetterBinding = true;
										const FProperty* SelectedVMProperty = SelectedVMPropertyField.GetProperty();
										if (!ArrayProperty->Inner->SameType(SelectedVMProperty))
										{
											const FText UserSelectedViewmodelType = FText::FromString(SelectedVMProperty->GetCPPType());
											const FText ArraySetterFunctionViewmodelType = FText::FromString(ArrayProperty->Inner->GetCPPType());
											Compiler->AddMessageForBinding(Binding, FText::Format(LOCTEXT("EntryViewModelTypeMismatch", "The entry viewmodel type {0} does not match the array of viewmodels of type {1}."), UserSelectedViewmodelType, ArraySetterFunctionViewmodelType), UE::MVVM::Compiler::EMessageType::Error);
											Compiler->MarkPrecompileStepInvalid();
											break;
										}
									}
								}
							}
						}

						// No need to look into the next fields of this write path, we have found the field with ViewmodelBlueprintWidgetExtension metadata.
						// We will go to the next binding
						break;
					}
				}
			}
		}

		if (!bFoundSetterBinding && EntryViewModelId.IsValid())
		{
			Compiler->AddMessage(FText::Format(LOCTEXT("PreCompileMVVMWidgetExtensionNoSetterBindingFound", "No binding found from an array of viewmodels to {0} -> SetItems. Please find {0} on the root widget and add this binding or remove the Viewmodel extension on widget {1} from its details panel."), FText::FromName(PanelPropertyName), FText::FromName(WidgetName)), UE::MVVM::Compiler::EMessageType::Warning);
		}
	};
	 
	if (UWidget* const* FoundPanel = Compiler->GetWidgetNameToWidgetPointerMap().Find(WidgetName))
	{
		if (UPanelWidget* PanelWidget = Cast<UPanelWidget>(*FoundPanel))
		{
			if (EntryWidgetClass.Get())
			{
				UE::MVVM::Compiler::IMVVMBlueprintViewPrecompile::FObjectFieldPathArgs AddObjFieldPathArgs(Class, WidgetName.ToString(), UPanelWidget::StaticClass());
				TValueOrError<UE::MVVM::FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> FieldPathResult = Compiler->AddObjectFieldPath(AddObjFieldPathArgs);
				if (FieldPathResult.HasError() || !FieldPathResult.GetValue().IsValid())
				{
					Compiler->AddMessage(FText::Format(LOCTEXT("CouldNotCreateSourceFieldPath", "Couldn't create the source field path '{0}'. {1}. Make sure '{0}' is marked as 'Is Variable'."), FText::FromName(WidgetName), FieldPathResult.GetError()), UE::MVVM::Compiler::EMessageType::Error);
					Compiler->MarkPrecompileStepInvalid();
					return;
				}
				WidgetPathHandle = FieldPathResult.StealValue();

				if (const UWidgetBlueprint* EntryBlueprint = Cast<UWidgetBlueprint>(EntryWidgetClass->ClassGeneratedBy))
				{
					if (const UMVVMBlueprintView* EntryBPView = GetEntryWidgetBlueprintView(EntryBlueprint))
					{
						if (const FMVVMBlueprintViewModelContext* ViewModelContext = EntryBPView->FindViewModel(EntryViewModelId))
						{
							const FName EntryViewModelName = ViewModelContext->GetViewModelName();

							// find the property path of the entry viewmodel
							FMVVMBlueprintPropertyPath SelectedViewModel;
							SelectedViewModel.SetViewModelId(EntryViewModelId);
							FProperty* ViewModelProperty = FindFProperty<FProperty>(*EntryWidgetClass, EntryViewModelName);
							SelectedViewModel.AppendPropertyPath(EntryBlueprint, UE::MVVM::FMVVMConstFieldVariant(ViewModelProperty));

							VerifyViewmodelTypeMatch(EntryWidgetClass, SelectedViewModel, PanelPropertyName);
						}
						else
						{
							// If the stored viewmodel ID invalid, it means the viewmodel was deleted so we clear the value.
							// We need to manually clear this value here because the panel widget doesn't get notified when the entry widget viewmodels change.
							EntryViewModelId = FGuid();
							Compiler->AddMessage(FText::Format(LOCTEXT("PreCompileMVVMWidgetExtensionEntryVMDeleted", "No viewmodel selected for entry widget {0}. Please select a viewmodel for it via the details panel or remove the Viewmodel extension on the containing widget {1}."), EntryWidgetClass->GetDisplayNameText(), FText::FromName(WidgetName)), UE::MVVM::Compiler::EMessageType::Error);

						}
					}
					else
					{
						// If no view is found in the entry widget, we clear the entry viewmodel value.
						// We need to manually clear this value here because the panel widget doesn't get notified when the entry widget view is added/removed.
						EntryViewModelId = FGuid();
						Compiler->AddMessage(FText::Format(LOCTEXT("PreCompileMVVMWidgetExtensionEntryHasNoView", "Entry widget {0} doesn't have a View. Consider adding a binding to it or remove the MVVM extension on the containing widget {1}."), EntryWidgetClass->GetDisplayNameText(), FText::FromName(WidgetName)), UE::MVVM::Compiler::EMessageType::Error);
					}
				}
				else
				{
					Compiler->AddMessage(FText::Format(LOCTEXT("PreCompileMVVMWidgetExtensionEntryWidgetBlueprintNotFound", "No widget blueprint was found for entry widget {0}. The MVVM extension on the containing widget {1} won't work."), EntryWidgetClass->GetDisplayNameText(), FText::FromName(WidgetName)), UE::MVVM::Compiler::EMessageType::Error);
				}
			}
			else
			{
				Compiler->AddMessage(FText::Format(LOCTEXT("PreCompileMVVMWidgetExtensionWidgetNoEntryClass", "Widget {0} doesn't have an entry widget class. Consider assigning it in the details panel or remove the Viewmodel extension on the widget."), FText::FromName(WidgetName)), UE::MVVM::Compiler::EMessageType::Error);
			}
		}
		else
		{
			Compiler->AddMessage(FText::Format(LOCTEXT("PreCompileMVVMWidgetExtensionWidgetNotPanelWidget", "Widget {0} is not a UPanelWidget but has a MVVMViewBlueprintPanelWidgetExtension."), FText::FromName(WidgetName)), UE::MVVM::Compiler::EMessageType::Error);
		}
	}
	else
	{
		Compiler->AddMessage(FText::Format(LOCTEXT("PreCompileMVVMWidgetExtensionInvalidWidgetName", "Widget with name {0} doesn't exist in the widget blueprint but a viewmodel widget extension exists that is attached to it."), FText::FromName(WidgetName)),UE::MVVM::Compiler::EMessageType::Error);
	}
}

void UMVVMBlueprintViewExtension_PanelWidget::Compile(UE::MVVM::Compiler::IMVVMBlueprintViewCompile* Compiler, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension)
{
	check(Compiler);

	if (UWidget* const* FoundPanel = Compiler->GetWidgetNameToWidgetPointerMap().Find(WidgetName))
	{
		if (UPanelWidget* PanelWidget = Cast<UPanelWidget>(*FoundPanel))
		{
			if (EntryWidgetClass.Get())
			{
				check(WidgetPathHandle.IsValid());

				// Verify the widget property
				const TValueOrError<FMVVMVCompiledFieldPath, void> CompiledFieldPath = Compiler->GetFieldPath(WidgetPathHandle);
				if (CompiledFieldPath.HasError())
				{
					Compiler->AddMessage(FText::Format(LOCTEXT("CompiledFieldPathForWidgetNotGenerated", "The field path for widget {0} was not generated."), FText::FromName(WidgetName)), UE::MVVM::Compiler::EMessageType::Error);
					Compiler->MarkCompileStepInvalid();
					return;
				}

				// Check that we have a valid entry view and valid entry viewmodel
				// Otherwise, no runtime extension will be created.
				if (const UWidgetBlueprint* EntryBlueprint = Cast<UWidgetBlueprint>(EntryWidgetClass->ClassGeneratedBy))
				{
					if (const UMVVMBlueprintView* EntryBPView = GetEntryWidgetBlueprintView(EntryBlueprint))
					{
						if (const FMVVMBlueprintViewModelContext* ViewModelContext = EntryBPView->FindViewModel(EntryViewModelId))
						{
							// Create the corresponding runtime extension
							UMVVMViewClassExtension* NewExtensionObj = Compiler->CreateViewClassExtension(UMVVMViewPanelWidgetClassExtension::StaticClass());
							UMVVMViewPanelWidgetClassExtension* NewExtension = CastChecked<UMVVMViewPanelWidgetClassExtension>(NewExtensionObj);

							const FName EntryViewModelName = ViewModelContext->GetViewModelName();
							NewExtension->Initialize(UMVVMViewPanelWidgetClassExtension::FInitPanelWidgetExtensionArgs(WidgetName, EntryViewModelName, CompiledFieldPath.GetValue(), EntryWidgetClass, SlotObj, PanelPropertyName, ViewModelContext->NotifyFieldValueClass));
						}
					}
				}
			}
		}
	}	
}

const UMVVMBlueprintView* UMVVMBlueprintViewExtension_PanelWidget::GetEntryWidgetBlueprintView(const UWidgetBlueprint* EntryWidgetBlueprint) const
{
	if (EntryWidgetBlueprint)
	{
		if (const UMVVMWidgetBlueprintExtension_View* EntryWidgetExtension = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(EntryWidgetBlueprint))
		{
			if (const UMVVMBlueprintView* EntryWidgetView = EntryWidgetExtension->GetBlueprintView())
			{
				return EntryWidgetView;
			}
		}
	}
	return nullptr;
}

void UMVVMBlueprintViewExtension_PanelWidget::RefreshDesignerPreviewEntries(UPanelWidget* PanelWidget, TSubclassOf<UUserWidget> EntryWidgetClass, UPanelSlot* SlotTemplate, int32 NumDesignerPreviewEntries, bool bFullRebuild)
{
	if (ensure(PanelWidget))
	{
		if (bFullRebuild || !EntryWidgetClass)
		{
			PanelWidget->ClearChildren();

			if (EntryWidgetClass)
			{
				for (int32 EntryIndex = 0; EntryIndex < NumDesignerPreviewEntries; ++EntryIndex)
				{
					if (UUserWidget* EntryWidget = UUserWidget::CreateWidgetInstance(*PanelWidget, EntryWidgetClass, NAME_None))
					{
						PanelWidget->AddChild(EntryWidget, SlotTemplate);
					}
				}
			}
		}
		else if (NumDesignerPreviewEntries > PanelWidget->GetChildrenCount())
		{
			for (int32 NumToAdd = NumDesignerPreviewEntries - PanelWidget->GetChildrenCount(); NumToAdd > 0; --NumToAdd)
			{
				PanelWidget->AddChild(UUserWidget::CreateWidgetInstance(*PanelWidget, EntryWidgetClass, NAME_None), SlotTemplate);
			}
		}
		else if (ensure(NumDesignerPreviewEntries >= 0) && NumDesignerPreviewEntries < PanelWidget->GetChildrenCount())
		{
			for (int32 NumToRemove = PanelWidget->GetChildrenCount() - NumDesignerPreviewEntries; NumToRemove > 0; --NumToRemove)
			{
				PanelWidget->RemoveChildAt(PanelWidget->GetChildrenCount() - 1);
			}
		}
	}
}

bool UMVVMBlueprintViewExtension_PanelWidget::WidgetRenamed(FName OldName, FName NewName)
{
	if (WidgetName == OldName)
	{
		Modify();
		WidgetName = NewName;
		return true;
	}
	return false;
}

void UMVVMBlueprintViewExtension_PanelWidget::OnPreviewContentChanged(TSharedRef<SWidget> NewContent)
{
	if (NewContent != SNullWidget::NullWidget)
	{
		const SObjectWidget* ObjectWidget = StaticCastSharedPtr<SObjectWidget>(NewContent.ToSharedPtr()).Get();
		const UUserWidget* PreviewRoot = ObjectWidget ? ObjectWidget->GetWidgetObject() : nullptr;

		if (UPanelWidget* PreviewWidget = PreviewRoot ? Cast<UPanelWidget>(PreviewRoot->GetWidgetFromName(WidgetName)) : nullptr)
		{
			constexpr bool bFullRebuild = false;
			RefreshDesignerPreviewEntries(PreviewWidget, EntryWidgetClass, SlotObj, NumDesignerPreviewEntries, bFullRebuild);
		}
	}
}

#undef LOCTEXT_NAMESPACE