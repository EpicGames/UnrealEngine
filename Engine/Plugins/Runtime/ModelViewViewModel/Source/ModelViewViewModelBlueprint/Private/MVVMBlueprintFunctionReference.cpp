// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBlueprintFunctionReference.h"
#include "BlueprintCompilationManager.h"
#include "Engine/Blueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBlueprintFunctionReference)

#define LOCTEXT_NAMESPACE "MVVMBlueprintFunctionReference"

FMVVMBlueprintFunctionReference::FMVVMBlueprintFunctionReference(const UBlueprint* InContext, const UFunction* InFunction)
	: Type(EMVVMBlueprintFunctionReferenceType::None)
{
	UClass* OwnerClass = InFunction ? InFunction->GetOwnerClass() : nullptr;
	ensure(OwnerClass);
	if (!OwnerClass)
	{
		return;
	}

	Type = EMVVMBlueprintFunctionReferenceType::Function;
	FName FieldName = InFunction->GetFName();
	FGuid MemberGuid;
	UBlueprint::GetGuidFromClassByFieldName<UFunction>(OwnerClass, FieldName, MemberGuid);

	// Set the member reference
	bool bIsSelf = (InContext->GeneratedClass && InContext->GeneratedClass->IsChildOf(OwnerClass))
		|| (InContext->SkeletonGeneratedClass && InContext->SkeletonGeneratedClass->IsChildOf(OwnerClass));
	if (bIsSelf)
	{
		FunctionReference.SetSelfMember(FieldName, MemberGuid);
	}
	else
	{
		FGuid Guid;
		if (UBlueprint* VariableOwnerBP = Cast<UBlueprint>(OwnerClass->ClassGeneratedBy))
		{
			OwnerClass = VariableOwnerBP->SkeletonGeneratedClass ? VariableOwnerBP->SkeletonGeneratedClass : VariableOwnerBP->GeneratedClass;
		}

		FunctionReference.SetExternalMember(FieldName, OwnerClass, MemberGuid);
	}
}

FMVVMBlueprintFunctionReference::FMVVMBlueprintFunctionReference(FMemberReference InReference)
	: FunctionReference(MoveTemp(InReference))
	, Type(EMVVMBlueprintFunctionReferenceType::Function)
{}

FMVVMBlueprintFunctionReference::FMVVMBlueprintFunctionReference(TSubclassOf<UK2Node> InNode)
	: Node(InNode)
	, Type(InNode.Get() != nullptr ? EMVVMBlueprintFunctionReferenceType::Node : EMVVMBlueprintFunctionReferenceType::None)
{
}

const UFunction* FMVVMBlueprintFunctionReference::GetFunction(const UBlueprint* SelfContext) const
{
	UClass* ContextClass = SelfContext->SkeletonGeneratedClass ? SelfContext->SkeletonGeneratedClass : SelfContext->GeneratedClass;
	return GetFunction(ContextClass);
}

const UFunction* FMVVMBlueprintFunctionReference::GetFunction(const UClass* SelfContext) const
{
	if (Type != EMVVMBlueprintFunctionReferenceType::Function)
	{
		return nullptr;
	}

	if (FunctionReference.GetMemberName().IsNone())
	{
		return nullptr;
	}

	if (!FBlueprintCompilationManager::IsGeneratedClassLayoutReady())
	{
		SelfContext = FBlueprintEditorUtils::GetSkeletonClass(FunctionReference.GetMemberParentClass(const_cast<UClass*>(SelfContext)));
	}
	return FunctionReference.ResolveMember<UFunction>(const_cast<UClass*>(SelfContext), false);
}

TSubclassOf<UK2Node> FMVVMBlueprintFunctionReference::GetNode() const
{
	return Type == EMVVMBlueprintFunctionReferenceType::Node ? Node : TSubclassOf<UK2Node>();
}

bool FMVVMBlueprintFunctionReference::IsValid(const UBlueprint* SelfContext) const
{
	return GetFunction(SelfContext) != nullptr || GetNode().Get() != nullptr;
}

bool FMVVMBlueprintFunctionReference::IsValid(const UClass* SelfContext) const
{
	return GetFunction(SelfContext) != nullptr || GetNode().Get() != nullptr;
}

bool FMVVMBlueprintFunctionReference::operator==(const FMVVMBlueprintFunctionReference& Other) const
{
	if (Type != Other.Type)
	{
		return false;
	}
	switch (Type)
	{
	case EMVVMBlueprintFunctionReferenceType::Function:
		return FunctionReference.IsSameReference(Other.FunctionReference);

	case EMVVMBlueprintFunctionReferenceType::Node:
		return Node == Other.Node;

	case EMVVMBlueprintFunctionReferenceType::None:
		return true;

	default:
		check(false);
		return false;
	}
}

FString FMVVMBlueprintFunctionReference::ToString() const
{
	switch (Type)
	{
		case EMVVMBlueprintFunctionReferenceType::Function:
			return FunctionReference.GetMemberName().ToString();
		case EMVVMBlueprintFunctionReferenceType::Node:
			return Node.Get() ? Node.Get()->GetName() : FString();
	}
	return FString();
}


FName FMVVMBlueprintFunctionReference::GetName() const
{
	switch (Type)
	{
	case EMVVMBlueprintFunctionReferenceType::Function:
		return FunctionReference.GetMemberName();
	case EMVVMBlueprintFunctionReferenceType::Node:
		return Node.Get() ? Node.Get()->GetFName() : FName();
	}
	return FName();
}

#undef LOCTEXT_NAMESPACE
