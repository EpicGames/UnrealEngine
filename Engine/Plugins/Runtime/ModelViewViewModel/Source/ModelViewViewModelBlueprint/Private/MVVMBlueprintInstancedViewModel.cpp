// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBlueprintInstancedViewModel.h"

#include "MVVMBlueprintView.h"
#include "MVVMViewModelBase.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "ViewModel/MVVMInstancedViewModelGeneratedClass.h"

#include "BlueprintActionDatabase.h"
#include "Engine/Engine.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Hash/CityHash.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBlueprintInstancedViewModel)

/**
 * 
 */
UMVVMBlueprintInstancedViewModelBase::UMVVMBlueprintInstancedViewModelBase()
{
	ParentClass = UMVVMViewModelBase::StaticClass();
}

void UMVVMBlueprintInstancedViewModelBase::GenerateClass(bool bForceGeneration)
{
	if (ParentClass.Get() == nullptr)
	{
		ParentClass = UMVVMViewModelBase::StaticClass();
	}
	if (!FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass))
	{
		ParentClass = UMVVMViewModelBase::StaticClass();
	}
	if (GeneratedClassType.Get() == nullptr)
	{
		GeneratedClassType = UMVVMInstancedViewModelGeneratedClass::StaticClass();
	}

	PreloadObjectsForCompilation();

	UObject* PreviousClass = nullptr;
	if (GeneratedClass == nullptr)
	{
		UPackage* Outermost = GetOutermost();
		FName NewClassName = *FString::Printf(TEXT("InstanceViewmodel%d_IC"), GetFName().GetNumber());
		PreviousClass = StaticFindObjectFastInternal(nullptr, Outermost, NewClassName, EFindObjectFlags::ExactClass);
		if (PreviousClass)
		{
			SafeRename(PreviousClass);
		}
		GeneratedClass = NewObject<UMVVMInstancedViewModelGeneratedClass>(Outermost, GeneratedClassType.Get(), NewClassName);
	}

	if (bForceGeneration == false && !IsClassDirty())
	{
		return;
	}

	UObject* PreviousDefaultObject = GeneratedClass->GetDefaultObject(false);
	CleanClass();
	AddProperties();
	ConstructClass();

	// Relink the new default object
	if (PreviousClass)
	{
		FLinkerLoad::PRIVATE_PatchNewObjectIntoExport(PreviousClass, GeneratedClass);
	}
	if (PreviousDefaultObject)
	{
		FLinkerLoad::PRIVATE_PatchNewObjectIntoExport(PreviousDefaultObject, GeneratedClass->GetDefaultObject(true));
	}

	// Initialize the default object value
	SetDefaultValues();

	// Inform that the class changed
	{
		TMap<UObject*, UObject*> OldToNew;
		if (PreviousDefaultObject)
		{
			OldToNew.Emplace(PreviousDefaultObject, GeneratedClass->GetDefaultObject(true));
		}
		if (PreviousClass)
		{
			OldToNew.Emplace(PreviousClass, GeneratedClass);
		}
		OldToNew.Emplace(GeneratedClass, GeneratedClass);
		if (GEngine && OldToNew.Num() > 0)
		{
			GEngine->NotifyToolsOfObjectReplacement(OldToNew);
		}
	}

	// Rebuild the BP action
	if (FBlueprintActionDatabase* ActionDB = FBlueprintActionDatabase::TryGet())
	{
		// Notify Blueprints that there is a new class to add to the action list
		ActionDB->RefreshClassActions(GeneratedClass);
	}

	ClassGenerated();

	// Find a way to call DestroyPropertiesPendingDestruction but after everyone had the time to update.
	GeneratedClass->DestroyPropertiesPendingDestruction();
	GeneratedClass->Modify();
}

void UMVVMBlueprintInstancedViewModelBase::PreloadObjectsForCompilation()
{

}

bool UMVVMBlueprintInstancedViewModelBase::IsClassDirty() const
{
	return true;
}

void UMVVMBlueprintInstancedViewModelBase::CleanClass()
{
	UObject* PreviousDefaultObject = GeneratedClass->GetDefaultObject(false);
	if (PreviousDefaultObject)
	{
		SafeRename(PreviousDefaultObject);
	}
	for (TFieldIterator<UFunction> FunctionIter(GeneratedClass, EFieldIteratorFlags::ExcludeSuper); FunctionIter; ++FunctionIter)
	{
		FunctionIter->FunctionFlags &= ~FUNC_Native;
		SafeRename(*FunctionIter);
	}

	for (TFieldIterator<FField> FieldIter(GeneratedClass, EFieldIteratorFlags::ExcludeSuper); FieldIter; ++FieldIter)
	{
		FName NewName = *FString::Printf(TEXT("TRASH_%s"), *FieldIter->GetName());
		FieldIter->Rename(NewName);
	}

	// Clean class and reset basic properties
	GeneratedClass->PurgeClass(false);
	GeneratedClass->PropertyLink = ParentClass->PropertyLink;
	GeneratedClass->SetSuperStruct(ParentClass);
	GeneratedClass->ClassWithin = UObject::StaticClass();
	GeneratedClass->ClassConfigName = ParentClass->ClassConfigName;
	GeneratedClass->ClassFlags |= CLASS_NotPlaceable;
	GeneratedClass->NumReplicatedProperties = 0;
	GeneratedClass->FieldNotifies.Empty();
}

void UMVVMBlueprintInstancedViewModelBase::AddProperties()
{
}

void UMVVMBlueprintInstancedViewModelBase::ConstructClass()
{
	check(GeneratedClass);
	GeneratedClass->Bind();
	GeneratedClass->StaticLink(true);
	GeneratedClass->AssembleReferenceTokenStream();
	ensure(GeneratedClass->ClassGeneratedBy == nullptr);
	ensure(GeneratedClass->GetDefaultObject(false) == nullptr);
	GeneratedClass->GetDefaultObject(true);
	GeneratedClass->UpdateCustomPropertyListForPostConstruction();
	GeneratedClass->SetUpRuntimeReplicationData();
	// Initialize the fields for MVVM
	GeneratedClass->InitializeFieldNotifies();

	// The class is not public and can only be access inside the UMG. What about inherited UMG???
	GeneratedClass->ClearFlags(RF_Public | RF_Transactional);
	GeneratedClass->GetDefaultObject(true)->ClearFlags(RF_Public);
}

void UMVVMBlueprintInstancedViewModelBase::SetDefaultValues()
{

}

void UMVVMBlueprintInstancedViewModelBase::ClassGenerated()
{
	GetOutermost()->Modify(true);
}

bool UMVVMBlueprintInstancedViewModelBase::IsValidFieldName(const FName NewPropertyName) const
{
	return IsValidFieldName(NewPropertyName, GeneratedClass);
}


bool UMVVMBlueprintInstancedViewModelBase::IsValidFieldName(const FName NewPropertyName, UStruct* NewOwner) const
{
	if (!FName::IsValidXName(NewPropertyName, INVALID_NAME_CHARACTERS))
	{
		return false;
	}

	// Check if the name already exist. If it does, do not add the property.
	for (TFieldIterator<FField> PropertyIter(NewOwner, EFieldIteratorFlags::IncludeSuper); PropertyIter; ++PropertyIter)
	{
		if (PropertyIter->GetFName() == NewPropertyName)
		{
			return false;
		}
	}
	for (TFieldIterator<UField> FunctionIter(NewOwner, EFieldIteratorFlags::IncludeSuper); FunctionIter; ++FunctionIter)
	{
		if (FunctionIter->GetFName() == NewPropertyName)
		{
			return false;
		}
	}

	return true;
}

FProperty* UMVVMBlueprintInstancedViewModelBase::CreateProperty(const FProperty* FromProperty, UStruct* NewOwner)
{
	return CreateProperty(FromProperty, NewOwner, FromProperty->GetFName());
}

FProperty* UMVVMBlueprintInstancedViewModelBase::CreateProperty(const FProperty* FromProperty, UStruct* NewOwner, FName NewPropertyName)
{
	if (!IsValidFieldName(NewPropertyName))
	{
		return nullptr;
	}

	FProperty* NewProperty = CastFieldChecked<FProperty>(FField::Duplicate(FromProperty, NewOwner, NewPropertyName));
	FField::CopyMetaData(FromProperty, NewProperty);

	return NewProperty;
}

void UMVVMBlueprintInstancedViewModelBase::InitializeProperty(FProperty* NewProperty, FInitializePropertyArgs& Args)
{
	if (Args.bNetwork)
	{
		AddOnRepFunction(NewProperty);
	}

	EPropertyFlags NewFlags = NewProperty->GetPropertyFlags() | EPropertyFlags::CPF_Edit;
	if (Args.bNetwork)
	{
		NewFlags |= EPropertyFlags::CPF_Net;
		NewFlags |= !NewProperty->RepNotifyFunc.IsNone() ? EPropertyFlags::CPF_RepNotify : EPropertyFlags::CPF_None;
		++GeneratedClass->NumReplicatedProperties;
	}
	NewFlags |= !Args.bPrivate ? EPropertyFlags::CPF_BlueprintVisible : EPropertyFlags::CPF_None;
	NewFlags |= Args.bReadOnly ? EPropertyFlags::CPF_BlueprintReadOnly : EPropertyFlags::CPF_None;
	NewProperty->SetPropertyFlags(NewFlags);
	if (Args.bFieldNotify)
	{
		NewProperty->SetMetaData(FBlueprintMetadata::MD_FieldNotify, TEXT(""));
		GeneratedClass->FieldNotifies.Add(FFieldNotificationId(NewProperty->GetFName()));
	}
}

void UMVVMBlueprintInstancedViewModelBase::LinkProperty(FProperty* NewProperty) const
{
	LinkProperty(NewProperty, GeneratedClass);
}

void UMVVMBlueprintInstancedViewModelBase::LinkProperty(FProperty* NewProperty, UStruct* NewOwner) const
{
	NewOwner->AddCppProperty(NewProperty);
}

void UMVVMBlueprintInstancedViewModelBase::AddOnRepFunction(FProperty* NewProperty)
{
	FString OnRepCallFunctionName = FString::Printf(TEXT("__OnRep_%s"), *NewProperty->GetName());
	FName Name_OnRepCallFunctionName = *OnRepCallFunctionName;
	UObject* PreviousObj = StaticFindObjectFastInternal(nullptr, GeneratedClass, Name_OnRepCallFunctionName, EFindObjectFlags::ExactClass);
	ensure(PreviousObj == nullptr);
	if (PreviousObj)
	{
		// The function or property already exist. Something is wrong.
		return;
	}

	UFunction* Func = NewObject<UFunction>(GeneratedClass, Name_OnRepCallFunctionName);
	Func->FunctionFlags |= FUNC_Final | FUNC_Native | FUNC_Event;
	GeneratedClass->AddNativeFunction(*OnRepCallFunctionName, &UMVVMInstancedViewModelGeneratedClass::K2_CallNativeOnRep);
	GeneratedClass->AddFunctionToFunctionMap(Func, Func->GetFName());
	Func->Bind();
	Func->StaticLink(true);

	Func->Next = GeneratedClass->Children;
	GeneratedClass->Children = Func;

	NewProperty->RepNotifyFunc = Func->GetFName();

	GeneratedClass->AddNativeRepNotifyFunction(Func, NewProperty);
}

void UMVVMBlueprintInstancedViewModelBase::SafeRename(UObject* Object)
{
	ERenameFlags RenameFlags = REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors;
	FName TrashName = MakeUniqueObjectName(GetTransientPackage(), Object->GetClass(), *FString::Printf(TEXT("TRASH_%s"), *Object->GetName()));
	Object->Rename(*TrashName.ToString(), GetTransientPackage(), RenameFlags);
}

void UMVVMBlueprintInstancedViewModelBase::SetDefaultValue(const FProperty* SourceProperty, void const* SourceValuePtr, const FProperty* DestinationProperty)
{
	check(DestinationProperty);
	check(SourceProperty);
	check(SourceValuePtr);
	{
		void* DestinationPtr = DestinationProperty->ContainerPtrToValuePtr<void>(GeneratedClass->GetDefaultObject());
		DestinationProperty->CopyCompleteValue(DestinationPtr, SourceValuePtr);
	}
}

/**
 *
 */
namespace UE::MVVM::Private
{
uint64 GetObjectHash(const UObject* Object)
{
	const FString PathName = GetPathNameSafe(Object);
	return CityHash64((const char*)GetData(PathName), PathName.Len() * sizeof(TCHAR));
}

uint64 CalcPropertyDescHash(const FPropertyBagPropertyDesc& Desc)
{
	const UStruct* ValueTypeObject = Cast<const UStruct>(Desc.ValueTypeObject);
	FTopLevelAssetPath ValueTypeObjectPath = ValueTypeObject ? ValueTypeObject->GetStructPathName() : FTopLevelAssetPath();
#if WITH_EDITORONLY_DATA
	const uint32 Hashes[] = { GetTypeHash(ValueTypeObjectPath), GetTypeHash(Desc.ID), GetTypeHash(Desc.Name), GetTypeHash(Desc.ValueType), GetTypeHash(Desc.ContainerTypes), GetTypeHash(Desc.MetaData) };
#else
	const uint32 Hashes[] = { GetTypeHash(ValueTypeObjectPath), GetTypeHash(Desc.ID), GetTypeHash(Desc.Name), GetTypeHash(Desc.ValueType), GetTypeHash(Desc.ContainerTypes) };
#endif
	return CityHash64((const char*)Hashes, sizeof(Hashes));
}

uint64 CalcPropertyDescArrayHash(const TConstArrayView<FPropertyBagPropertyDesc> Descs)
{
	uint64 Hash = 0;
	for (const FPropertyBagPropertyDesc& Desc : Descs)
	{
		Hash = CityHash128to64(Uint128_64(Hash, CalcPropertyDescHash(Desc)));
	}
	// Should add the default values.
	return Hash;
}
} //namespace

bool UMVVMBlueprintInstancedViewModel_PropertyBag::IsClassDirty() const
{
	const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();
	if (PropertyBag)
	{
		return UE::MVVM::Private::CalcPropertyDescArrayHash(PropertyBag->GetPropertyDescs()) != PropertiesHash;
	}
	return PropertiesHash != 0;
}

void UMVVMBlueprintInstancedViewModel_PropertyBag::CleanClass()
{
	Super::CleanClass();

	FromPropertyToCreatedProperty.Empty();
}

void UMVVMBlueprintInstancedViewModel_PropertyBag::AddProperties()
{
	Super::AddProperties();

	for (TPropertyValueIterator<FProperty> It(GetSourceStruct(), GetSourceDefaults(), EPropertyValueIteratorFlags::NoRecursion, EFieldIteratorFlags::ExcludeDeprecated); It; ++It)
	{
		const FProperty* Property = It.Key();
		void const* ValuePtr = It.Value();

		FProperty* NewProperty = CreateProperty(Property, GeneratedClass);
		if (NewProperty)
		{
			FInitializePropertyArgs Args;
			Args.bFieldNotify = true;
			InitializeProperty(NewProperty, Args);

			LinkProperty(NewProperty);
			FromPropertyToCreatedProperty.Add(Property, NewProperty);
		}
	}
}

void UMVVMBlueprintInstancedViewModel_PropertyBag::SetDefaultValues()
{
	Super::SetDefaultValues();

	for (TPropertyValueIterator<FProperty> It(GetSourceStruct(), GetSourceDefaults(), EPropertyValueIteratorFlags::NoRecursion, EFieldIteratorFlags::ExcludeDeprecated); It; ++It)
	{
		const FProperty* Property = It.Key();
		void const* ValuePtr = It.Value();

		FProperty* NewProperty = nullptr;
		if (FProperty** NewPropertyPtr = FromPropertyToCreatedProperty.Find(Property))
		{
			NewProperty = *NewPropertyPtr;
		}
		else
		{
			NewProperty = GeneratedClass->FindPropertyByName(Property->GetFName());
		}

		if (NewProperty)
		{
			SetDefaultValue(Property, ValuePtr, NewProperty);
		}
	}
}

void UMVVMBlueprintInstancedViewModel_PropertyBag::ClassGenerated()
{
	Super::ClassGenerated();

	FromPropertyToCreatedProperty.Empty();
	const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();
	PropertiesHash = PropertyBag ? UE::MVVM::Private::CalcPropertyDescArrayHash(PropertyBag->GetPropertyDescs()) : 0;
}

#if WITH_EDITOR
void UMVVMBlueprintInstancedViewModel_PropertyBag::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* VariableProperty = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMBlueprintInstancedViewModel_PropertyBag, Variables));
	FProperty* ParentClassProperty = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMBlueprintInstancedViewModel_PropertyBag, ParentClass));
	FEditPropertyChain::TDoubleLinkedListNode* CurrentPropertyNode = PropertyChangedEvent.PropertyChain.GetActiveMemberNode();
	while (CurrentPropertyNode)
	{
		if (CurrentPropertyNode->GetValue() == VariableProperty || CurrentPropertyNode->GetValue() == ParentClassProperty)
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(GetOuterUMVVMBlueprintView()->GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
			PropertiesHash = 0; // force class regeneration
			break;
		}
		CurrentPropertyNode = CurrentPropertyNode->GetNextNode();
	}
}
#endif