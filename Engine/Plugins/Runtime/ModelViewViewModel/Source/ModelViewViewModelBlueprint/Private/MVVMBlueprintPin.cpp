// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBlueprintPin.h"

#include "Algo/Reverse.h"
#include "Bindings/MVVMConversionFunctionHelper.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "MVVMConversionFunctionGraphSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBlueprintPin)

/**
 *
 */
FMVVMBlueprintPinId::FMVVMBlueprintPinId(const TArrayView<const FName> Names)
	: PinNames(Names)
{
}

FMVVMBlueprintPinId::FMVVMBlueprintPinId(TArray<FName>&& Names)
	: PinNames(Names)
{
}

bool FMVVMBlueprintPinId::IsValid() const
{
	return PinNames.Num() > 0 && !PinNames.Contains(FName());
}

bool FMVVMBlueprintPinId::IsChildOf(const FMVVMBlueprintPinId& Other) const
{
	if (Other.PinNames.Num() >= PinNames.Num())
	{
		return false;
	}

	for (int32 Index = 0; Index < Other.PinNames.Num(); ++Index)
	{
		if (Other.PinNames[Index] != PinNames[Index])
		{
			return false;
		}
	}
	return true;
}

bool FMVVMBlueprintPinId::IsDirectChildOf(const FMVVMBlueprintPinId& Other) const
{
	if (Other.PinNames.Num() != PinNames.Num() - 1)
	{
		return false;
	}

	for (int32 Index = 0; Index < Other.PinNames.Num(); ++Index)
	{
		if (Other.PinNames[Index] != PinNames[Index])
		{
			return false;
		}
	}
	return true;
}

bool FMVVMBlueprintPinId::operator==(const FMVVMBlueprintPinId& Other) const
{
	return PinNames == Other.PinNames;
}

bool FMVVMBlueprintPinId::operator==(const TArrayView<const FName> Other) const
{
	return PinNames.Num() == Other.Num() && CompareItems(GetData(Other), GetData(PinNames), PinNames.Num());
}

FString FMVVMBlueprintPinId::ToString() const
{
	TStringBuilder<256> Builder;
	for (FName Name : PinNames)
	{
		if (Builder.Len() > 0)
		{
			Builder << TEXT('.');
		}
		Builder << Name;
	}
	return Builder.ToString();
}

/**
 *
 */
FMVVMBlueprintPin::FMVVMBlueprintPin(FName InPinName)
	: Id(MakeArrayView(&InPinName, 1))
{
}

FMVVMBlueprintPin::FMVVMBlueprintPin(FMVVMBlueprintPinId InPinId)
	: Id(MoveTemp(InPinId))
{
}

FMVVMBlueprintPin::FMVVMBlueprintPin(const TArrayView<const FName> InPinNames)
	: Id(InPinNames)
{
}

FMVVMBlueprintPin FMVVMBlueprintPin::CreateFromPin(const UBlueprint* Blueprint, const UEdGraphPin* Pin)
{
	FMVVMBlueprintPin Result;
	Result.Id = FMVVMBlueprintPinId(UE::MVVM::ConversionFunctionHelper::FindPinId(Pin));
	Result.Path = UE::MVVM::ConversionFunctionHelper::GetPropertyPathForPin(Blueprint, Pin, true);
	Result.DefaultObject = Pin->DefaultObject;
	Result.DefaultString = Pin->DefaultValue;
	Result.DefaultText = Pin->DefaultTextValue;
	Result.bSplit = Pin->SubPins.Num() > 0;
	Result.Status = Pin->bOrphanedPin ? EMVVMBlueprintPinStatus::Orphaned : EMVVMBlueprintPinStatus::Valid;
	return Result;
}

void FMVVMBlueprintPin::SetDefaultValue(UObject* Value)
{
	Reset();
	DefaultObject = Value;
}

void FMVVMBlueprintPin::SetDefaultValue(const FText& Value)
{
	Reset();
	DefaultText = Value;
}

void FMVVMBlueprintPin::SetDefaultValue(const FString& Value)
{
	Reset();
	DefaultString = Value;
}

void FMVVMBlueprintPin::SetPath(const FMVVMBlueprintPropertyPath& Value)
{
	Reset();
	Path = Value;
}

FString FMVVMBlueprintPin::GetValueAsString(const UClass* SelfContext) const
{
	if (Path.IsValid())
	{
		return Path.GetPropertyPath(SelfContext);
	}
	else if (DefaultObject)
	{
		return DefaultObject.GetPathName();
	}
	else if (!DefaultText.IsEmpty())
	{
		FString TextAsString;
		FTextStringHelper::WriteToBuffer(TextAsString, DefaultText);
		return TextAsString;
	}
	else
	{
		return DefaultString;
	}
}

bool FMVVMBlueprintPin::IsInputPin(const UEdGraphPin* Pin)
{
	return UE::MVVM::ConversionFunctionHelper::IsInputPin(Pin);
}

TArray<FMVVMBlueprintPin> FMVVMBlueprintPin::CopyAndReturnMissingPins(UBlueprint* Blueprint, UEdGraphNode* GraphNode, const TArray<FMVVMBlueprintPin>& Pins)
{
	check(GraphNode);
	check(Blueprint);

	TSet<const UEdGraphPin*> AllGraphPins;
	AllGraphPins.Reserve(GraphNode->Pins.Num());
	for (const FMVVMBlueprintPin& Pin : Pins)
	{
		Pin.CopyTo(Blueprint, GraphNode);
		if (UEdGraphPin* GraphPin = Pin.FindGraphPin(GraphNode->GetGraph()))
		{
			AllGraphPins.Add(GraphPin);
		}
	}

	// Create the missing pins.
	TArray<FMVVMBlueprintPin> MissingPins;
	for (const UEdGraphPin* GraphPin : GraphNode->Pins)
	{
		if (IsInputPin(GraphPin) && !AllGraphPins.Contains(GraphPin))
		{
			MissingPins.Add(FMVVMBlueprintPin::CreateFromPin(Blueprint, GraphPin));
		}
	}
	return MissingPins;
}

void FMVVMBlueprintPin::CopyTo(const UBlueprint* Blueprint, UEdGraphNode* Node) const
{
	Status = EMVVMBlueprintPinStatus::Orphaned;
	if (UEdGraphPin* GraphPin = FindGraphPin(Node->GetGraph()))
	{
		if (IsInputPin(GraphPin))
		{
			Status = EMVVMBlueprintPinStatus::Valid;
			if (bSplit && GraphPin->SubPins.Num() == 0 && GetDefault<UEdGraphSchema_K2>()->CanSplitStructPin(*GraphPin))
			{
				GetDefault<UEdGraphSchema_K2>()->SplitPin(GraphPin, false);
			}

			if (Path.IsValid())
			{
				UE::MVVM::ConversionFunctionHelper::SetPropertyPathForPin(Blueprint, Path, GraphPin);
			}
			else if (DefaultObject)
			{
				GetDefault<UMVVMConversionFunctionGraphSchema>()->TrySetDefaultObject(*GraphPin, DefaultObject, false);
			}
			else if (!DefaultText.IsEmpty())
			{
				GetDefault<UMVVMConversionFunctionGraphSchema>()->TrySetDefaultText(*GraphPin, DefaultText, false);
			}
			else if (!DefaultString.IsEmpty())
			{
				GetDefault<UMVVMConversionFunctionGraphSchema>()->TrySetDefaultValue(*GraphPin, DefaultString, false);
			}
		}
	}
}

TArray<FMVVMBlueprintPin> FMVVMBlueprintPin::CreateFromNode(UBlueprint* Blueprint, UEdGraphNode* GraphNode)
{
	TArray<FMVVMBlueprintPin> Result;
	Result.Reserve(GraphNode->Pins.Num());
	for (const UEdGraphPin* GraphPin : GraphNode->Pins)
	{
		if (IsInputPin(GraphPin) && !GraphPin->bOrphanedPin)
		{
			Result.Add(FMVVMBlueprintPin::CreateFromPin(Blueprint, GraphPin));
		}
	}
	return Result;
}

UEdGraphPin* FMVVMBlueprintPin::FindGraphPin(const UEdGraph* Graph) const
{
	return UE::MVVM::ConversionFunctionHelper::FindPin(Graph, Id.GetNames());
}

void FMVVMBlueprintPin::Reset()
{
	Path = FMVVMBlueprintPropertyPath();
	DefaultString.Empty();
	DefaultText = FText();
	DefaultObject = nullptr;
	bSplit = false;
	Status = EMVVMBlueprintPinStatus::Valid;
}

void FMVVMBlueprintPin::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (!PinName_DEPRECATED.IsNone())
		{
			Id = FMVVMBlueprintPinId(MakeArrayView(&PinName_DEPRECATED, 1));
			PinName_DEPRECATED = FName();
		}
	}
}
