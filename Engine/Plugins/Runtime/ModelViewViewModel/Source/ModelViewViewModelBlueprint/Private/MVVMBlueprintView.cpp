// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBlueprintView.h"

#include "Components/Widget.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MVVMBlueprintInstancedViewModel.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMBlueprintViewCondition.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "WidgetBlueprint.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "View/MVVMViewModelContextResolver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBlueprintView)


UMVVMBlueprintView::UMVVMBlueprintView()
{
	Settings = CreateDefaultSubobject<UMVVMBlueprintViewSettings>("Settings");
	CompiledBindingLibraryId = FGuid::NewGuid();
}

FMVVMBlueprintViewModelContext* UMVVMBlueprintView::FindViewModel(FGuid ViewModelId)
{
	return AvailableViewModels.FindByPredicate([ViewModelId](const FMVVMBlueprintViewModelContext& Other)
		{
			return Other.GetViewModelId() == ViewModelId;
		});
}

const FMVVMBlueprintViewModelContext* UMVVMBlueprintView::FindViewModel(FGuid ViewModelId) const
{
	return const_cast<UMVVMBlueprintView*>(this)->FindViewModel(ViewModelId);
}

const FMVVMBlueprintViewModelContext* UMVVMBlueprintView::FindViewModel(FName ViewModel) const
{
	return AvailableViewModels.FindByPredicate([ViewModel](const FMVVMBlueprintViewModelContext& Other)
		{
			return Other.GetViewModelName() == ViewModel;
		});
}

void UMVVMBlueprintView::AddViewModel(const FMVVMBlueprintViewModelContext& NewContext)
{
	AvailableViewModels.Add(NewContext);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
	OnViewModelsUpdated.Broadcast();
}

namespace UE::MVVM::Private
{
	bool RemoveViewModelInternal(FGuid ViewModelId, TArray<FMVVMBlueprintViewModelContext>& ViewModelContexts)
	{
		bool bResult = false;
		for (int32 Index = ViewModelContexts.Num() - 1; Index >= 0; --Index)
		{
			if (ViewModelContexts[Index].GetViewModelId() == ViewModelId)
			{
				UMVVMBlueprintInstancedViewModelBase* InstancedViewModel = ViewModelContexts[Index].InstancedViewModel;
				if (InstancedViewModel)
				{
					auto RenameToTransient = [](UObject* ObjectToRename)
					{
						FName TrashName = MakeUniqueObjectName(GetTransientPackage(), ObjectToRename->GetClass(), *FString::Printf(TEXT("TRASH_%s"), *ObjectToRename->GetName()));
						ObjectToRename->Rename(*TrashName.ToString(), GetTransientPackage());
					};
					if (InstancedViewModel->GetGeneratedClass())
					{
						RenameToTransient(InstancedViewModel->GetGeneratedClass());
					}
					RenameToTransient(InstancedViewModel);
				}

				ViewModelContexts.RemoveAt(Index);
				bResult = true;
			}
		}
		return bResult;
	}
}

bool UMVVMBlueprintView::RemoveViewModel(FGuid ViewModelId)
{
	bool bRemoved = UE::MVVM::Private::RemoveViewModelInternal(ViewModelId, AvailableViewModels);
	if (bRemoved)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
		OnViewModelsUpdated.Broadcast();
	}
	return bRemoved;
}

int32 UMVVMBlueprintView::RemoveViewModels(const TArrayView<FGuid> ViewModelIds)
{
	int32 Count = 0;
	for (const FGuid& ViewModelId : ViewModelIds)
	{
		if (UE::MVVM::Private::RemoveViewModelInternal(ViewModelId, AvailableViewModels))
		{
			++Count;
		}
	}

	if (Count > 0)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
		OnViewModelsUpdated.Broadcast();
	}
	return Count;
}

bool UMVVMBlueprintView::RenameViewModel(FName OldViewModelName, FName NewViewModelName)
{
	FMVVMBlueprintViewModelContext* ViewModelContext = AvailableViewModels.FindByPredicate([OldViewModelName](const FMVVMBlueprintViewModelContext& Other)
			{
				return Other.GetViewModelName() == OldViewModelName;
			});
	if (ViewModelContext)
	{
		ViewModelContext->ViewModelName = NewViewModelName;

		FBlueprintEditorUtils::ReplaceVariableReferences(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint(), OldViewModelName, NewViewModelName);
		FBlueprintEditorUtils::ValidateBlueprintChildVariables(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint(), NewViewModelName);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());

		OnViewModelsUpdated.Broadcast();
	}
	return ViewModelContext != nullptr;
}

bool UMVVMBlueprintView::ReparentViewModel(FGuid ViewModelId, const UClass* ViewModelClass)
{
	if (ViewModelClass && ViewModelClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
	{
		FMVVMBlueprintViewModelContext* ViewModelContext = AvailableViewModels.FindByPredicate([ViewModelId](const FMVVMBlueprintViewModelContext& Other)
			{
				return Other.GetViewModelId() == ViewModelId;
			});
		if (ViewModelContext)
		{
			ViewModelContext->NotifyFieldValueClass = const_cast<UClass*>(ViewModelClass);
			TArray<EMVVMBlueprintViewModelContextCreationType> ValidCreationTypes = UE::MVVM::GetAllowedContextCreationType(ViewModelClass);
			if (!ValidCreationTypes.Contains(ViewModelContext->CreationType))
			{
				if (ensureMsgf(ValidCreationTypes.Num() > 0, TEXT("There is no valid creation type for this class.")))
				{
					ViewModelContext->CreationType = ValidCreationTypes[0];
				}
			}

			FBlueprintEditorUtils::MarkBlueprintAsModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());

			OnViewModelsUpdated.Broadcast();
			return true;
		}
	}
	return false;
}

const FMVVMBlueprintViewBinding* UMVVMBlueprintView::FindBinding(const UWidget* Widget, const FProperty* Property) const
{
	return const_cast<UMVVMBlueprintView*>(this)->FindBinding(Widget, Property);
}

FMVVMBlueprintViewBinding* UMVVMBlueprintView::FindBinding(const UWidget* Widget, const FProperty* Property)
{
	FName WidgetName = Widget->GetFName();
	return Bindings.FindByPredicate([WidgetBlueprint = GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint(), WidgetName, Property](const FMVVMBlueprintViewBinding& Binding)
		{
			return Binding.DestinationPath.GetWidgetName() == WidgetName &&
				Binding.DestinationPath.PropertyPathContains(WidgetBlueprint->GeneratedClass, UE::MVVM::FMVVMConstFieldVariant(Property));
		});
}

void UMVVMBlueprintView::RemoveBindingAt(int32 Index)
{
	if (Bindings.IsValidIndex(Index))
	{
		if (Bindings[Index].Conversion.SourceToDestinationConversion)
		{
			Bindings[Index].Conversion.SourceToDestinationConversion->RemoveWrapperGraph(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
		}
		if (Bindings[Index].Conversion.DestinationToSourceConversion)
		{
			Bindings[Index].Conversion.DestinationToSourceConversion->RemoveWrapperGraph(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
		}

		Bindings.RemoveAt(Index);
		OnBindingsUpdated.Broadcast();

		FBlueprintEditorUtils::MarkBlueprintAsModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
	}
}

void UMVVMBlueprintView::RemoveBinding(const FMVVMBlueprintViewBinding* Binding)
{
	int32 Index = 0;
	for (; Index < Bindings.Num(); ++Index)
	{
		if (&Bindings[Index] == Binding)
		{
			break;
		}
	}

	RemoveBindingAt(Index);
}

const FMVVMBlueprintViewBinding* UMVVMBlueprintView::DuplicateBinding(const FMVVMBlueprintViewBinding* Binding)
{
	FMVVMBlueprintViewBinding& NewBinding = Bindings.AddDefaulted_GetRef();

	// Copy all existing binding properties except the BindingId
	NewBinding = *Binding;
	NewBinding.BindingId = FGuid::NewGuid();

	UWidgetBlueprint* WidgetBlueprint = GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint();

	if (UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding->Conversion.SourceToDestinationConversion)
	{
		NewBinding.Conversion.SourceToDestinationConversion = DuplicateObject<UMVVMBlueprintViewConversionFunction>(ConversionFunction, WidgetBlueprint);
		NewBinding.Conversion.SourceToDestinationConversion->RecreateWrapperGraph(WidgetBlueprint);
	}

	if (UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding->Conversion.DestinationToSourceConversion)
	{
		NewBinding.Conversion.DestinationToSourceConversion = DuplicateObject<UMVVMBlueprintViewConversionFunction>(ConversionFunction, WidgetBlueprint);
		NewBinding.Conversion.DestinationToSourceConversion->RecreateWrapperGraph(WidgetBlueprint);
	}

	OnBindingsAdded.Broadcast();
	OnBindingsUpdated.Broadcast();

	FBlueprintEditorUtils::MarkBlueprintAsModified(WidgetBlueprint);

	return &NewBinding;
}

FMVVMBlueprintViewBinding& UMVVMBlueprintView::AddDefaultBinding()
{
	FMVVMBlueprintViewBinding& NewBinding = Bindings.AddDefaulted_GetRef();
	NewBinding.BindingId = FGuid::NewGuid();

	OnBindingsAdded.Broadcast();
	OnBindingsUpdated.Broadcast();

	FBlueprintEditorUtils::MarkBlueprintAsModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());

	return NewBinding;
}

FMVVMBlueprintViewBinding* UMVVMBlueprintView::GetBindingAt(int32 Index)
{
	if (Bindings.IsValidIndex(Index))
	{
		return &Bindings[Index];
	}
	return nullptr;
}

const FMVVMBlueprintViewBinding* UMVVMBlueprintView::GetBindingAt(int32 Index) const
{
	if (Bindings.IsValidIndex(Index))
	{
		return &Bindings[Index];
	}
	return nullptr;
}

FMVVMBlueprintViewBinding* UMVVMBlueprintView::GetBinding(FGuid Id)
{
	return Bindings.FindByPredicate([Id](const FMVVMBlueprintViewBinding& Binding){ return Id == Binding.BindingId; });
}

const FMVVMBlueprintViewBinding* UMVVMBlueprintView::GetBinding(FGuid Id) const
{
	return Bindings.FindByPredicate([Id](const FMVVMBlueprintViewBinding& Binding) { return Id == Binding.BindingId; });
}

UMVVMBlueprintViewEvent* UMVVMBlueprintView::AddDefaultEvent()
{
	UMVVMBlueprintViewEvent* Event = NewObject<UMVVMBlueprintViewEvent>(this);
	AddEvent(Event);

	return Event;
}

void UMVVMBlueprintView::AddEvent(UMVVMBlueprintViewEvent* Event)
{
	Events.Add(Event);

	OnBindingsAdded.Broadcast();
	OnBindingsUpdated.Broadcast();

	FBlueprintEditorUtils::MarkBlueprintAsModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
}

void UMVVMBlueprintView::RemoveEvent(UMVVMBlueprintViewEvent* Event)
{
	if (Events.RemoveAll([Event](TObjectPtr<UMVVMBlueprintViewEvent>& Other){ return Other == Event; }) > 0)
	{
		Event->RemoveWrapperGraph();
		OnBindingsUpdated.Broadcast();

		FBlueprintEditorUtils::MarkBlueprintAsModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
	}
}

void UMVVMBlueprintView::ReplaceEvent(UMVVMBlueprintViewEvent* OldEvent, UMVVMBlueprintViewEvent* NewEvent)
{
	for (TObjectPtr<UMVVMBlueprintViewEvent>& Event : Events)
	{
		if (Event == OldEvent)
		{
			Event = NewEvent;
			OldEvent->RemoveWrapperGraph();
			OnBindingsUpdated.Broadcast();

			FBlueprintEditorUtils::MarkBlueprintAsModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
		}
	}
}

UMVVMBlueprintViewEvent* UMVVMBlueprintView::DuplicateEvent(UMVVMBlueprintViewEvent* Event)
{
	if (Event != nullptr)
	{
		Event->SavePinValues();
	}

	UMVVMBlueprintViewEvent* NewEvent = DuplicateObject<UMVVMBlueprintViewEvent>(Event, this);
	NewEvent->RecreateWrapperGraph();
	
	AddEvent(NewEvent);

	return NewEvent;
}

UMVVMBlueprintViewCondition* UMVVMBlueprintView::AddDefaultCondition()
{
	UMVVMBlueprintViewCondition* Condition = NewObject<UMVVMBlueprintViewCondition>(this);
	AddCondition(Condition);

	return Condition;
}

void UMVVMBlueprintView::AddCondition(UMVVMBlueprintViewCondition* Condition)
{
	Conditions.Add(Condition);

	OnBindingsAdded.Broadcast();
	OnBindingsUpdated.Broadcast();

	FBlueprintEditorUtils::MarkBlueprintAsModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
}

void UMVVMBlueprintView::RemoveCondition(UMVVMBlueprintViewCondition* Condition)
{
	if (Conditions.RemoveAll([Condition](TObjectPtr<UMVVMBlueprintViewCondition>& Other) { return Other == Condition; }) > 0)
	{
		Condition->RemoveWrapperGraph();
		OnBindingsUpdated.Broadcast();

		FBlueprintEditorUtils::MarkBlueprintAsModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
	}
}

void UMVVMBlueprintView::ReplaceCondition(UMVVMBlueprintViewCondition* OldCondition, UMVVMBlueprintViewCondition* NewCondition)
{
	for (TObjectPtr<UMVVMBlueprintViewCondition>& Condition : Conditions)
	{
		if (Condition == OldCondition)
		{
			Condition = NewCondition;
			OldCondition->RemoveWrapperGraph();
			OnBindingsUpdated.Broadcast();

			FBlueprintEditorUtils::MarkBlueprintAsModified(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint());
		}
	}
}

UMVVMBlueprintViewCondition* UMVVMBlueprintView::DuplicateCondition(UMVVMBlueprintViewCondition* Condition)
{
	if (Condition != nullptr)
	{
		Condition->SavePinValues();
	}

	UMVVMBlueprintViewCondition* NewCondition = DuplicateObject<UMVVMBlueprintViewCondition>(Condition, this);
	NewCondition->RecreateWrapperGraph();

	AddCondition(NewCondition);

	return NewCondition;
}

TArray<FText> UMVVMBlueprintView::GetBindingMessages(FGuid Id, UE::MVVM::EBindingMessageType InMessageType) const
{
	TArray<FText> Results;

	if (BindingMessages.Contains(Id))
	{
		const TArray<UE::MVVM::FBindingMessage>& AllBindingMessages = BindingMessages[Id];
		for (const UE::MVVM::FBindingMessage& Message : AllBindingMessages)
		{
			if (Message.MessageType == InMessageType)
			{
				Results.Add(Message.MessageText);
			}
		}
	}
	return Results;
}

bool UMVVMBlueprintView::HasBindingMessage(FGuid Id) const
{
	return BindingMessages.Find(Id) != nullptr;
}

bool UMVVMBlueprintView::HasBindingMessage(FGuid Id, UE::MVVM::EBindingMessageType InMessageType) const
{
	if (const TArray<UE::MVVM::FBindingMessage>* FoundBindingMessages = BindingMessages.Find(Id))
	{
		for (const UE::MVVM::FBindingMessage& Message : *FoundBindingMessages)
		{
			if (Message.MessageType == InMessageType)
			{
				return true;
			}
		}
	}

	return false;
}

void UMVVMBlueprintView::AddMessageToBinding(FGuid Id, UE::MVVM::FBindingMessage MessageToAdd)
{
	TArray<UE::MVVM::FBindingMessage>& FoundBindingMessages = BindingMessages.FindOrAdd(Id);
	FoundBindingMessages.Add(MessageToAdd);
}

void UMVVMBlueprintView::ResetBindingMessages()
{
	BindingMessages.Reset();
}

void UMVVMBlueprintView::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);	
}

#if WITH_EDITOR
namespace UE::MVVM::Private
{
	enum EConditionPath
	{
		ConditionPath,
		DestinationPath
	};

	template<typename Predicate>
	void ForEachPropertyPath_Update(UMVVMBlueprintView* BlueprintView, Predicate Pred, bool bGenerateGraph)
	{
		UWidgetBlueprint* WidgetBlueprint = BlueprintView->GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint();
		auto PredPin = [&Pred](const FMVVMBlueprintPin& Pin) -> TOptional<FMVVMBlueprintPropertyPath>
		{
			if (Pin.UsedPathAsValue())
			{
				FMVVMBlueprintPropertyPath NewPinPath = Pin.GetPath();
				if (Pred(NewPinPath))
				{
					return NewPinPath;
				}
			}
			return TOptional<FMVVMBlueprintPropertyPath>();
		};

		for (FMVVMBlueprintViewBinding& Binding : BlueprintView->GetBindings())
		{
			Pred(Binding.SourcePath);
			Pred(Binding.DestinationPath);
			if (Binding.Conversion.DestinationToSourceConversion)
			{
				Binding.Conversion.DestinationToSourceConversion->ConditionalPostLoad();
				for (const FMVVMBlueprintPin& Pin : Binding.Conversion.DestinationToSourceConversion->GetPins())
				{
					TOptional<FMVVMBlueprintPropertyPath> NewPath = PredPin(Pin);
					if (NewPath.IsSet())
					{
						if (bGenerateGraph)
						{
							Binding.Conversion.DestinationToSourceConversion->SetGraphPin(WidgetBlueprint, Pin.GetId(), NewPath.GetValue());
						}
						else
						{
							const_cast<FMVVMBlueprintPin&>(Pin).SetPath(NewPath.GetValue());
						}
					}
				}
			}
			if (Binding.Conversion.SourceToDestinationConversion)
			{
				Binding.Conversion.SourceToDestinationConversion->ConditionalPostLoad();
				for (const FMVVMBlueprintPin& Pin : Binding.Conversion.SourceToDestinationConversion->GetPins())
				{
					TOptional<FMVVMBlueprintPropertyPath> NewPath = PredPin(Pin);
					if (NewPath.IsSet())
					{
						if (bGenerateGraph)
						{
							Binding.Conversion.SourceToDestinationConversion->SetGraphPin(WidgetBlueprint, Pin.GetId(), NewPath.GetValue());
						}
						else
						{
							const_cast<FMVVMBlueprintPin&>(Pin).SetPath(NewPath.GetValue());
						}
					}
				}
			}
		}

		for (UMVVMBlueprintViewEvent* Event : BlueprintView->GetEvents())
		{
			if (Event)
			{
				auto PredEventPath = [Event, &Pred, bGenerateGraph](const FMVVMBlueprintPropertyPath& PropertyPath, bool bEventPath)
				{
					FMVVMBlueprintPropertyPath NewPropertyPath = PropertyPath;
					if (Pred(NewPropertyPath))
					{
						if (bGenerateGraph)
						{
							if (bEventPath)
							{
								Event->SetEventPath(NewPropertyPath);
							}
							else
							{
								Event->SetDestinationPath(NewPropertyPath);
							}
						}
						else
						{
							const_cast<FMVVMBlueprintPropertyPath&>(PropertyPath) = NewPropertyPath;
						}
					}
				};

				TArray<TTuple<FMVVMBlueprintPinId, FMVVMBlueprintPropertyPath>> NewPins;
				for (const FMVVMBlueprintPin& Pin : Event->GetPins())
				{
					TOptional<FMVVMBlueprintPropertyPath> NewPath = PredPin(Pin);
					if (NewPath.IsSet())
					{
						NewPins.Emplace(Pin.GetId(), MoveTemp(NewPath.GetValue()));
					}
				}
				PredEventPath(Event->GetEventPath(), true);
				PredEventPath(Event->GetDestinationPath(), false);
				for (TTuple<FMVVMBlueprintPinId, FMVVMBlueprintPropertyPath>& Pin : NewPins)
				{
					if (bGenerateGraph)
					{
						Event->SetPinPath(Pin.Get<0>(), Pin.Get<1>());
					}
					else
					{
						Event->SetPinPathNoGraphGeneration(Pin.Get<0>(), Pin.Get<1>());
					}
				}
			}
		}

		for (UMVVMBlueprintViewCondition* Condition : BlueprintView->GetConditions())
		{
			if (Condition)
			{
				auto PredConditionPath = [Condition, &Pred, bGenerateGraph](const FMVVMBlueprintPropertyPath& PropertyPath, EConditionPath TypePath)
					{
						FMVVMBlueprintPropertyPath NewPropertyPath = PropertyPath;
						if (Pred(NewPropertyPath))
						{
							if (bGenerateGraph)
							{
								if (TypePath == EConditionPath::ConditionPath)
								{
									Condition->SetConditionPath(NewPropertyPath);
								}
								else if (TypePath == EConditionPath::DestinationPath)
								{
									Condition->SetDestinationPath(NewPropertyPath);
								}
							}
							else
							{
								const_cast<FMVVMBlueprintPropertyPath&>(PropertyPath) = NewPropertyPath;
							}
						}
					};

				TArray<TTuple<FMVVMBlueprintPinId, FMVVMBlueprintPropertyPath>> NewPins;
				for (const FMVVMBlueprintPin& Pin : Condition->GetPins())
				{
					TOptional<FMVVMBlueprintPropertyPath> NewPath = PredPin(Pin);
					if (NewPath.IsSet())
					{
						NewPins.Emplace(Pin.GetId(), MoveTemp(NewPath.GetValue()));
					}
				}
				PredConditionPath(Condition->GetConditionPath(), EConditionPath::ConditionPath);
				PredConditionPath(Condition->GetDestinationPath(), EConditionPath::DestinationPath);
				for (TTuple<FMVVMBlueprintPinId, FMVVMBlueprintPropertyPath>& Pin : NewPins)
				{
					if (bGenerateGraph)
					{
						Condition->SetPinPath(Pin.Get<0>(), Pin.Get<1>());
					}
					else
					{
						Condition->SetPinPathNoGraphGeneration(Pin.Get<0>(), Pin.Get<1>());
					}
				}
			}
		}
	}
}

void UMVVMBlueprintView::PostLoad()
{
	Super::PostLoad();

	for (FMVVMBlueprintViewBinding& Binding : Bindings)
	{
		if (!Binding.BindingId.IsValid())
		{
			Binding.BindingId = FGuid::NewGuid();
		}
	}

	GetOuterUMVVMWidgetBlueprintExtension_View()->ConditionalPostLoad();
	GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint()->ConditionalPostLoad();
	UWidgetBlueprint* ThisWidgetBlueprint = GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint();

	// Make sure all bindings uses the skeletal class
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MVVMConvertPropertyPathToSkeletalClass)
	{
		for (FMVVMBlueprintViewBinding& Binding : Bindings)
		{
			for (const FMVVMBlueprintFieldPath& FieldPath : Binding.SourcePath.GetFieldPaths())
			{
				const_cast<FMVVMBlueprintFieldPath&>(FieldPath).SetDeprecatedSelfReference(ThisWidgetBlueprint);
			}
			for (const FMVVMBlueprintFieldPath& FieldPath : Binding.DestinationPath.GetFieldPaths())
			{
				const_cast<FMVVMBlueprintFieldPath&>(FieldPath).SetDeprecatedSelfReference(ThisWidgetBlueprint);
			}
		}
	}

	for (FMVVMBlueprintViewBinding& Binding : Bindings)
	{
		Binding.Conversion.DeprecateViewConversionFunction(GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint(), Binding);
	}

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MVVMPropertyPathSelf)
	{
		auto DeprecateSelfPath = [ThisWidgetBlueprint](FMVVMBlueprintPropertyPath& Path) -> bool
		{
			Path.GetSource(ThisWidgetBlueprint);
			return true;
		};
		UE::MVVM::Private::ForEachPropertyPath_Update(this, DeprecateSelfPath, false);
	}

	for (FMVVMBlueprintViewModelContext& ViewModelContext : AvailableViewModels)
	{
		if (UMVVMViewModelContextResolver* ResolverObject = ViewModelContext.Resolver)
		{
			if (!ResolverObject->HasAnyFlags(RF_Public))
			{
				ResolverObject->SetFlags(RF_Public);
			}
			if (ResolverObject->GetOuter() == ThisWidgetBlueprint->GetPackage())
			{
				ResolverObject->Rename(nullptr, ThisWidgetBlueprint, REN_DontCreateRedirectors | REN_NonTransactional);
			}
			else
			{
				// We need to explicitly clear up resolver redirectors, they were created when the resolver was moved to the correct package. 
				// Now REN_DontCreateRedirectors flag is added to ensure these redirectors are not created. But we still need to clear the existing ones.
				// Redirectors are not needed here because the resolver was not RF_Public before, it's only being made public at the same time as the package change.
				UObjectRedirector* FoundResolverRedirector = nullptr;
				ForEachObjectWithPackage(ThisWidgetBlueprint->GetPackage(), [ResolverObject, &FoundResolverRedirector](UObject* Object)
					{
						if (UObjectRedirector* Redirector = Cast<UObjectRedirector>(Object))
						{
							if (Redirector->DestinationObject == ResolverObject)
							{
								Redirector->ClearFlags(RF_Public | RF_Standalone);
								Redirector->DestinationObject = nullptr;
								return false;
							}
						}
						return true;
					}, EGetObjectsFlags::None);
			}
		}
	}
}

void UMVVMBlueprintView::PreSave(FObjectPreSaveContext Context)
{
	// Cleanup Event and Conditions to prevent BP compilation issues
	for (int32 EventIndex = Conditions.Num() - 1; EventIndex >= 0; --EventIndex)
	{
		UMVVMBlueprintViewCondition* Condition = Conditions[EventIndex];
		if (Condition == nullptr)
		{
			Conditions.RemoveAt(EventIndex);
		}
	}

	for (int32 ConditionIndex = Events.Num() - 1; ConditionIndex >= 0; --ConditionIndex)
	{
		UMVVMBlueprintViewEvent* Event = Events[ConditionIndex];
		if (Event == nullptr)
		{
			Events.RemoveAt(ConditionIndex);
		}
	}

	UWidgetBlueprint* WidgetBlueprint = GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint();
	for (FMVVMBlueprintViewBinding& Binding : Bindings)
	{
		Binding.Conversion.SavePinValues(WidgetBlueprint);
	}
	for (UMVVMBlueprintViewEvent* Event : Events)
	{
		Event->SavePinValues();
	}

	Super::PreSave(Context);
}

void UMVVMBlueprintView::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMVVMBlueprintView, Bindings))
	{
		OnBindingsUpdated.Broadcast();
	}
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMVVMBlueprintView, AvailableViewModels))
	{
		OnViewModelsUpdated.Broadcast();
	}
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMVVMBlueprintView, Events))
	{
		OnEventsUpdated.Broadcast();
	}
}

void UMVVMBlueprintView::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChainEvent)
{
	Super::PostEditChangeChainProperty(PropertyChainEvent);
	if (PropertyChainEvent.PropertyChain.Contains(UMVVMBlueprintView::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMBlueprintView, Bindings))))
	{
		OnBindingsUpdated.Broadcast();
	}
	if (PropertyChainEvent.PropertyChain.Contains(UMVVMBlueprintView::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMBlueprintView, AvailableViewModels))))
	{
		OnViewModelsUpdated.Broadcast();
	}
	if (PropertyChainEvent.PropertyChain.Contains(UMVVMBlueprintView::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMBlueprintView, Events))))
	{
		OnEventsUpdated.Broadcast();
	}
}

void UMVVMBlueprintView::PostEditUndo()
{
	Super::PostEditUndo();

	OnBindingsUpdated.Broadcast();
	OnViewModelsUpdated.Broadcast();
	OnEventsUpdated.Broadcast();
	OnConditionsUpdated.Broadcast();
}

void UMVVMBlueprintView::AddAssetTags(TArray<FAssetRegistryTag>& OutTags) const
{
}

void UMVVMBlueprintView::AddAssetTags(FAssetRegistryTagsContext Context) const
{

}

void UMVVMBlueprintView::OnFieldRenamed(UClass* FieldOwnerClass, FName OldObjectName, FName NewObjectName)
{
	bool bRenamed = false;

	UWidgetBlueprint* WidgetBlueprint = GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint();

	auto UpdatePaths = [WidgetBlueprint, FieldOwnerClass, &bRenamed, OldObjectName, NewObjectName](FMVVMBlueprintPropertyPath& PropertyPath) -> bool
	{
		const bool bDidPathContainField = PropertyPath.OnFieldRenamed(WidgetBlueprint, FieldOwnerClass, OldObjectName, NewObjectName);
		bRenamed |= bDidPathContainField;
		return bDidPathContainField;
	};

	UE::MVVM::Private::ForEachPropertyPath_Update(this, UpdatePaths, true);

	if (bRenamed)
	{
		Modify();
		OnBindingsUpdated.Broadcast();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
	}
}
#endif

