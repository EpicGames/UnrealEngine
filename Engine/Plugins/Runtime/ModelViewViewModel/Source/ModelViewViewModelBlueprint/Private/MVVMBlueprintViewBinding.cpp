// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBlueprintViewBinding.h"

#include "Blueprint/WidgetTree.h"
#include "Bindings/MVVMConversionFunctionHelper.h"
#include "K2Node_CallFunction.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMWidgetBlueprintExtension_View.h"

#include "MVVMFunctionGraphHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBlueprintViewBinding)

#define LOCTEXT_NAMESPACE "MVVMBlueprintViewBinding"

void FMVVMBlueprintViewConversionPath::GenerateWrapper(UBlueprint* Blueprint)
{
	if (DestinationToSourceConversion)
	{
		DestinationToSourceConversion->GetOrCreateWrapperGraph(Blueprint);
	}
	if (SourceToDestinationConversion)
	{
		SourceToDestinationConversion->GetOrCreateWrapperGraph(Blueprint);
	}
}

void FMVVMBlueprintViewConversionPath::SavePinValues(UBlueprint* Blueprint)
{
	if (DestinationToSourceConversion)
	{
		DestinationToSourceConversion->SavePinValues(Blueprint);
	}
	if (SourceToDestinationConversion)
	{
		SourceToDestinationConversion->SavePinValues(Blueprint);
	}
}

void FMVVMBlueprintViewConversionPath::DeprecateViewConversionFunction(UBlueprint* Blueprint, FMVVMBlueprintViewBinding& Owner)
{
	auto Deprecate = [&Owner, Blueprint](bool bSourceToDestination, FName& Wrapper, FMemberReference& Reference, TObjectPtr<UMVVMBlueprintViewConversionFunction>& ConversionFunction, FMVVMBlueprintPropertyPath& Path)
	{
		if (ConversionFunction)
		{
			ConversionFunction->ConditionalPostLoad();
			if (ConversionFunction->GetWrapperGraphName().IsNone())
			{
				// The function was a simple conversion function but now we want to always have a wrapper
				FName GraphName = UE::MVVM::ConversionFunctionHelper::CreateWrapperName(Owner, bSourceToDestination);
				ConversionFunction->Deprecation_SetWrapperGraphName(Blueprint, GraphName, Path);
				Path = FMVVMBlueprintPropertyPath();
			}
		}
		else
		{
			if (!Wrapper.IsNone())
			{
				TObjectPtr<UEdGraph>* GraphPtr = Blueprint->FunctionGraphs.FindByPredicate([Wrapper](const UEdGraph* Other) { return Other->GetFName() == Wrapper; });
				if (GraphPtr)
				{
					ConversionFunction = NewObject<UMVVMBlueprintViewConversionFunction>(Blueprint);
					ConversionFunction->Deprecation_InitializeFromWrapperGraph(Blueprint, *GraphPtr);
				}
			}
			else if (!Reference.GetMemberName().IsNone())
			{
				ConversionFunction = NewObject<UMVVMBlueprintViewConversionFunction>(Blueprint);
				FName GraphName = UE::MVVM::ConversionFunctionHelper::CreateWrapperName(Owner, bSourceToDestination);
				ConversionFunction->Deprecation_InitializeFromMemberReference(Blueprint, GraphName, Reference, Path);
				Path = FMVVMBlueprintPropertyPath();
			}
			Wrapper = FName();
			Reference = FMemberReference();
		}
	};

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Deprecate(false, DestinationToSourceWrapper_DEPRECATED, DestinationToSourceFunction_DEPRECATED, DestinationToSourceConversion, Owner.DestinationPath);
	Deprecate(true, SourceToDestinationWrapper_DEPRECATED, SourceToDestinationFunction_DEPRECATED, SourceToDestinationConversion, Owner.SourcePath);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FName FMVVMBlueprintViewBinding::GetFName() const
{
	return *BindingId.ToString(EGuidFormats::DigitsWithHyphensLower);
}

namespace UE::MVVM::Private
{
	FString GetBindingPathName(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, bool bIsSource, bool bUseDisplayName, bool bAppendFunctionKeywords)
	{
		UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
		UMVVMBlueprintView* BlueprintView = ExtensionView->GetBlueprintView();

		TStringBuilder<256> NameBuilder;

		auto AddPath = [&](const FMVVMBlueprintPropertyPath& ArgumentPath)
		{
			NameBuilder << ArgumentPath.ToString(WidgetBlueprint, bUseDisplayName, bAppendFunctionKeywords);
		};

		if (UMVVMBlueprintViewConversionFunction* ViewConversionFunction = Binding.Conversion.GetConversionFunction(bIsSource))
		{
			FMVVMBlueprintFunctionReference ConversionFunction = ViewConversionFunction->GetConversionFunction();
			if (ConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
			{
				if (const UFunction* ConversionFunctionPtr = ConversionFunction.GetFunction(WidgetBlueprint))
				{
					if (bUseDisplayName)
					{
						NameBuilder << ConversionFunctionPtr->GetDisplayNameText().ToString();
					}
					else 
					{
						NameBuilder << ConversionFunctionPtr->GetFName();
					}

					if (bAppendFunctionKeywords)
					{
						FString FunctionKeywords = ConversionFunctionPtr->GetMetaData(FBlueprintMetadata::MD_FunctionKeywords);
						if (!FunctionKeywords.IsEmpty())
						{
							NameBuilder << TEXT(".");
							NameBuilder << FunctionKeywords;
						}
					}
				}
				else
				{
					NameBuilder << TEXT("<ERROR>");
				}
			}
			else if (ConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Node)
			{
				TSubclassOf<UK2Node> ConversionFunctionNode = ConversionFunction.GetNode();
				if (ConversionFunctionNode.Get())
				{
					if (bUseDisplayName)
					{
						NameBuilder << ConversionFunctionNode->GetDisplayNameText().ToString();
					}
					else
					{
						NameBuilder << ConversionFunctionNode->GetFName();
					}
				}
				else
				{
					NameBuilder << TEXT("<ERROR>");
				}
			}
			else
			{
				NameBuilder << TEXT("<ERROR>");
			}

			// AddPins
			{
				NameBuilder << TEXT("(");

				bool bFirst = true;
				for (const FMVVMBlueprintPin& Pin : ViewConversionFunction->GetPins())
				{
					if (!bFirst)
					{
						NameBuilder << TEXT(", ");
					}

					if (Pin.UsedPathAsValue())
					{
						AddPath(Pin.GetPath());
					}
					else
					{
						NameBuilder << Pin.GetValueAsString(WidgetBlueprint->SkeletonGeneratedClass);
					}

					bFirst = false;
				}

				NameBuilder << TEXT(")");
			}
		}
		else
		{
			const FMVVMBlueprintPropertyPath& PropertyPath = bIsSource ? Binding.SourcePath : Binding.DestinationPath;
			AddPath(PropertyPath);
		}

		FString Name = NameBuilder.ToString();
		if (Name.IsEmpty())
		{
			Name = TEXT("<none>");
		}

		return Name;
	}
}

FString FMVVMBlueprintViewBinding::GetDisplayNameString(const UWidgetBlueprint* WidgetBlueprint, bool bUseDisplayName) const
{
	check(WidgetBlueprint);

	TStringBuilder<256> NameBuilder;
	bool bAppendFunctionKeywords = false;
	NameBuilder << UE::MVVM::Private::GetBindingPathName(WidgetBlueprint, *this, false, bUseDisplayName, bAppendFunctionKeywords);

	if (BindingType == EMVVMBindingMode::TwoWay)
	{
		NameBuilder << TEXT(" <-> ");
	}
	else if (UE::MVVM::IsForwardBinding(BindingType))
	{
		NameBuilder << TEXT(" <- ");
	}
	else if (UE::MVVM::IsBackwardBinding(BindingType))
	{
		NameBuilder << TEXT(" -> ");
	}
	else
	{
		NameBuilder << TEXT(" ??? "); // shouldn't happen
	}

	NameBuilder << UE::MVVM::Private::GetBindingPathName(WidgetBlueprint, *this, true, bUseDisplayName, bAppendFunctionKeywords);

	return NameBuilder.ToString();
}

FString FMVVMBlueprintViewBinding::GetSearchableString(const UWidgetBlueprint* WidgetBlueprint) const
{
	check(WidgetBlueprint);

	// Create the function keywords string.
	TStringBuilder<258> Builder;
	bool bDisplayName = true;
	bool bAppendFunctionKeywords = true;
	Builder << UE::MVVM::Private::GetBindingPathName(WidgetBlueprint, *this, false, bDisplayName, bAppendFunctionKeywords);
	Builder << UE::MVVM::Private::GetBindingPathName(WidgetBlueprint, *this, true, bDisplayName, bAppendFunctionKeywords);

	FString Result = Builder.ToString();
	// Remove the extra formatting that we don't need for search.
	// We will include the formatted string as well in the second call to GetBindingPathName.
	Result.ReplaceInline(TEXT(")"), TEXT(" "));
	Result.ReplaceInline(TEXT("("), TEXT(" "));
	Result.ReplaceInline(TEXT(","), TEXT(" "));

	return Result;
}

#undef LOCTEXT_NAMESPACE