// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBlueprintViewCondition.h"
#include "MVVMBlueprintView.h"

#include "Bindings/MVVMConversionFunctionHelper.h"
#include "Bindings/MVVMBindingHelper.h"
#include "Bindings/MVVMFieldPathHelper.h"
#include "Engine/Blueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MVVMConversionFunctionGraphSchema.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "WidgetBlueprint.h"

#include "EdGraphSchema_K2.h"
#include "EdGraph/EdGraphPin.h"
#include "GraphEditAction.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_VariableSet.h"
#include "KismetCompiler.h"
#include "Node/MVVMK2Node_IsConditionValid.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBlueprintViewCondition)

#define LOCTEXT_NAMESPACE "MVVMBlueprintViewCondition"

void UMVVMBlueprintViewCondition::SetConditionPath(FMVVMBlueprintPropertyPath InConditionPath)
{
	if (InConditionPath == ConditionPath)
	{
		return;
	}

	UpdatePinValues();
	RemoveWrapperGraph(LeaveConversionFunctionCurrentValues);

	ConditionPath = MoveTemp(InConditionPath);
	GraphName = FName();

	if (ConditionPath.IsValid() || DestinationPath.IsValid())
	{
		TStringBuilder<256> StringBuilder;
		StringBuilder << TEXT("__");
		StringBuilder << FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower);
		GraphName = StringBuilder.ToString();
	}

	bNeedsToRegenerateChildren = true;

	CreateWrapperGraphInternal();
}

void UMVVMBlueprintViewCondition::SetDestinationPath(FMVVMBlueprintPropertyPath InDestinationPath)
{
	if (InDestinationPath == DestinationPath)
	{
		return;
	}

	// when we update the conversion function, we want to reset all the values to their default
	RemoveWrapperGraph(RemoveConversionFunctionCurrentValues);

	DestinationPath = MoveTemp(InDestinationPath);
	GraphName = FName();

	if (ConditionPath.IsValid() || DestinationPath.IsValid())
	{
		TStringBuilder<256> StringBuilder;
		StringBuilder << TEXT("__");
		StringBuilder << FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower);
		GraphName = StringBuilder.ToString();
	}

	bNeedsToRegenerateChildren = true;

	CreateWrapperGraphInternal();
	SavePinValues();
}

void UMVVMBlueprintViewCondition::SetOperation(EMVVMConditionOperation InOperation)
{
	if (InOperation == ConditionOperation)
	{
		return;
	}

	UpdatePinValues();
	RemoveWrapperGraph(LeaveConversionFunctionCurrentValues);
	ConditionOperation = InOperation;

	bNeedsToRegenerateChildren = true;

	CreateWrapperGraphInternal();
}

void UMVVMBlueprintViewCondition::SetOperationValue(float NewValue)
{
	//@TODO CompareWithEpsilon
	if (NewValue == Value)
	{
		return;
	}

	UpdatePinValues();
	RemoveWrapperGraph(LeaveConversionFunctionCurrentValues);
	Value = NewValue;

	bNeedsToRegenerateChildren = true;

	CreateWrapperGraphInternal();
}

void UMVVMBlueprintViewCondition::SetOperationMaxValue(float NewMaxValue) 
{
	//@TODO CompareWithEpsilon
	if (NewMaxValue == MaxValue)
	{
		return;
	}

	UpdatePinValues();
	RemoveWrapperGraph(LeaveConversionFunctionCurrentValues);
	MaxValue = NewMaxValue;

	bNeedsToRegenerateChildren = true;

	CreateWrapperGraphInternal();
}

void UMVVMBlueprintViewCondition::SetCachedWrapperGraphInternal(UEdGraph* Graph, UK2Node* Node, UMVVMK2Node_IsConditionValid* SourceNode)
{
	if (CachedWrapperDestinationNode && OnUserDefinedPinRenamedHandle.IsValid())
	{
		CachedWrapperDestinationNode->OnUserDefinedPinRenamed().Remove(OnUserDefinedPinRenamedHandle);
	}
	if (CachedWrapperGraph && OnGraphChangedHandle.IsValid())
	{
		CachedWrapperGraph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
	}

	CachedWrapperGraph = Graph;
	CachedWrapperDestinationNode = Node;
	CachedConditionValidNode = SourceNode;
	OnGraphChangedHandle.Reset();
	OnUserDefinedPinRenamedHandle.Reset();

	if (CachedWrapperGraph)
	{
		OnGraphChangedHandle = CachedWrapperGraph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateUObject(this, &UMVVMBlueprintViewCondition::HandleGraphChanged));
	}
	if (CachedWrapperDestinationNode)
	{
		OnUserDefinedPinRenamedHandle = CachedWrapperDestinationNode->OnUserDefinedPinRenamed().AddUObject(this, &UMVVMBlueprintViewCondition::HandleUserDefinedPinRenamed);
	}
	UpdateConditionKeyInternal();
}
	
UEdGraph* UMVVMBlueprintViewCondition::GetOrCreateWrapperGraph()
{
	if (CachedWrapperGraph)
	{
		return CachedWrapperGraph;
	}

	CreateWrapperGraphInternal();
	return CachedWrapperGraph;
}

void UMVVMBlueprintViewCondition::RecreateWrapperGraph()
{
	GraphName = FName();

	RemoveWrapperGraph(LeaveConversionFunctionCurrentValues);

	TStringBuilder<256> StringBuilder;
	StringBuilder << TEXT("__");
	StringBuilder << FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower);
	GraphName = StringBuilder.ToString();

	bNeedsToRegenerateChildren = true;

	CreateWrapperGraphInternal();
}

void UMVVMBlueprintViewCondition::RemoveWrapperGraph(ERemoveWrapperGraphParam ActionForCurrentValues)
{
	if (CachedWrapperGraph)
	{
		FBlueprintEditorUtils::RemoveGraph(GetWidgetBlueprintInternal(), CachedWrapperGraph);
		SetCachedWrapperGraphInternal(nullptr, nullptr, nullptr);
	}

	Messages.Empty();
	if (ActionForCurrentValues == RemoveConversionFunctionCurrentValues)
	{
		SavedPins.Empty();
	}
}

UEdGraphPin* UMVVMBlueprintViewCondition::GetOrCreateGraphPin(const FMVVMBlueprintPinId& PinId)
{
	GetOrCreateWrapperGraph();
	UEdGraphPin* FoundPin = nullptr;
	if (CachedWrapperDestinationNode != nullptr)
	{
		FoundPin = UE::MVVM::ConversionFunctionHelper::FindPin(CachedWrapperDestinationNode, PinId.GetNames());
	}
	return FoundPin;
}

void UMVVMBlueprintViewCondition::SavePinValues()
{
	if (!bLoadingPins) // While loading pins value, the node can trigger a notify that would then trigger a save.
	{
		SavedPins.Empty();
		if (CachedWrapperDestinationNode)
		{
			UWidgetBlueprint* Blueprint = GetWidgetBlueprintInternal();
			SavedPins = FMVVMBlueprintPin::CreateFromNode(Blueprint, CachedWrapperDestinationNode);
		}
	}
}

void UMVVMBlueprintViewCondition::UpdatePinValues()
{
	if (CachedWrapperDestinationNode == nullptr)
	{
		return;
	}
	UWidgetBlueprint* Blueprint = GetWidgetBlueprintInternal();
	SavedPins.RemoveAll([](const FMVVMBlueprintPin& Pin) { return Pin.GetStatus() != EMVVMBlueprintPinStatus::Orphaned; });

	if (CachedWrapperDestinationNode)
	{
		TArray<FMVVMBlueprintPin> TmpSavedPins = FMVVMBlueprintPin::CreateFromNode(Blueprint, CachedWrapperDestinationNode);
		SavedPins.Append(TmpSavedPins);
	}
}

bool UMVVMBlueprintViewCondition::HasOrphanedPin() const
{
	for (const FMVVMBlueprintPin& Pin : SavedPins)
	{
		if (Pin.GetStatus() == EMVVMBlueprintPinStatus::Orphaned)
		{
			return true;
		}
	}
	return false;
}

void UMVVMBlueprintViewCondition::UpdateConditionKeyInternal()
{
	if (CachedConditionValidNode)
	{
		CachedConditionValidNode->ConditionKey = ConditionKey;
	}
}

void UMVVMBlueprintViewCondition::UpdateConditionKey(FMVVMViewClass_ConditionKey InConditionKey)
{
	if (ConditionKey != InConditionKey)
	{
		ConditionKey = InConditionKey;
		UpdateConditionKeyInternal();
	}
}


FMVVMBlueprintPropertyPath UMVVMBlueprintViewCondition::GetPinPath(const FMVVMBlueprintPinId& PinId) const
{
	const FMVVMBlueprintPin* ViewPin = SavedPins.FindByPredicate([&PinId](const FMVVMBlueprintPin& Other) { return PinId == Other.GetId(); });
	return ViewPin ? ViewPin->GetPath() : FMVVMBlueprintPropertyPath();
}

void UMVVMBlueprintViewCondition::SetPinPath(const FMVVMBlueprintPinId& PinId, const FMVVMBlueprintPropertyPath& Path)
{
	UEdGraphPin* GraphPin = GetOrCreateGraphPin(PinId);

	if (GraphPin)
	{
		UBlueprint* Blueprint = GetWidgetBlueprintInternal();
		// Set the value and make the blueprint as dirty before creating the pin.
		FMVVMBlueprintPin* ViewPin = SavedPins.FindByPredicate([&PinId](const FMVVMBlueprintPin& Other) { return PinId == Other.GetId(); });
		if (!ViewPin)
		{
			ViewPin = &SavedPins.Add_GetRef(FMVVMBlueprintPin::CreateFromPin(Blueprint, GraphPin));
		}

		//A property (viewmodel or widget) may not be created yet and the skeletal needs to be recreated.
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

		UE::MVVM::ConversionFunctionHelper::SetPropertyPathForPin(Blueprint, Path, GraphPin);

		// Take the path built in BP, it may had some errors
		ViewPin->SetPath(UE::MVVM::ConversionFunctionHelper::GetPropertyPathForPin(Blueprint, GraphPin, false));
	}
}

void UMVVMBlueprintViewCondition::SetPinPathNoGraphGeneration(const FMVVMBlueprintPinId& PinId, const FMVVMBlueprintPropertyPath& Path)
{
	FMVVMBlueprintPin* ViewPin = SavedPins.FindByPredicate([&PinId](const FMVVMBlueprintPin& Other) { return PinId == Other.GetId(); });
	if (!ViewPin)
	{
		ViewPin = &SavedPins.Emplace_GetRef(PinId);
		ViewPin->SetPath(Path);
	}

	//A property (viewmodel or widget) may not be created yet and the skeletal needs to be recreated.
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetWidgetBlueprintInternal());
}

UWidgetBlueprint* UMVVMBlueprintViewCondition::GetWidgetBlueprintInternal() const
{
	return GetOuterUMVVMBlueprintView()->GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint();
}

const UFunction* UMVVMBlueprintViewCondition::GetDestinationSignature() const
{
	if (DestinationPath.IsValid() && DestinationPath.GetFieldPaths().Num() > 0)
	{
		const UWidgetBlueprint* WidgetBlueprint = GetWidgetBlueprintInternal();
		const FMVVMBlueprintFieldPath& LastPath = DestinationPath.GetFieldPaths().Last();
		UE::MVVM::FMVVMConstFieldVariant LastField = LastPath.GetField(WidgetBlueprint->SkeletonGeneratedClass);
		if (LastField.IsProperty())
		{
			if (const FMulticastDelegateProperty* Property = CastField<FMulticastDelegateProperty>(LastField.GetProperty()))
			{
				return Property->SignatureFunction.Get();
			}
		}
	}
	return nullptr;
}

UEdGraph* UMVVMBlueprintViewCondition::CreateWrapperGraphInternal()
{
	if (GraphName.IsNone() || !DestinationPath.IsValid())
	{
		return nullptr;
	}

	UWidgetBlueprint* WidgetBlueprint = GetWidgetBlueprintInternal();
	UE::MVVM::ConversionFunctionHelper::FCreateGraphParams Params;
	Params.bIsConst = false;
	Params.bTransient = true;
	Params.bIsForEvent = true;
	TValueOrError<UE::MVVM::ConversionFunctionHelper::FCreateGraphResult, FText> CreateSetterGraphResult = UE::MVVM::ConversionFunctionHelper::CreateSetterGraph(WidgetBlueprint, GraphName, nullptr, DestinationPath, Params);
	if (CreateSetterGraphResult.HasError())
	{
		SetCachedWrapperGraphInternal(nullptr, nullptr, nullptr);
		return nullptr;
	}

	static FName NAME_Hidden("Hidden");
	UE::MVVM::ConversionFunctionHelper::SetMetaData(CreateSetterGraphResult.GetValue().NewGraph, NAME_Hidden, FStringView());

	UMVVMK2Node_IsConditionValid* BranchNode = Cast<UMVVMK2Node_IsConditionValid>(UE::MVVM::ConversionFunctionHelper::InsertEarlyExitBranchNode(CreateSetterGraphResult.GetValue().NewGraph, UMVVMK2Node_IsConditionValid::StaticClass()));

	const UEdGraphSchema* GraphSchema = GetDefault<UMVVMConversionFunctionGraphSchema>();
	{
		GraphSchema->TrySetDefaultValue(*BranchNode->GetOperationPin(), UEnum::GetValueAsString(ConditionOperation));
		GraphSchema->TrySetDefaultValue(*BranchNode->GetCompareValuePin(), LexToString(Value));
		GraphSchema->TrySetDefaultValue(*BranchNode->GetCompareMaxValuePin(), LexToString(MaxValue));

		UE::MVVM::ConversionFunctionHelper::SetPropertyPathForPin(WidgetBlueprint, ConditionPath, BranchNode->GetValuePin());

	}

	SetCachedWrapperGraphInternal(CreateSetterGraphResult.GetValue().NewGraph, CreateSetterGraphResult.GetValue().WrappedNode, BranchNode);
	LoadPinValuesInternal();

	return CachedWrapperGraph;
}

void UMVVMBlueprintViewCondition::LoadPinValuesInternal()
{
	TGuardValue<bool> Tmp(bLoadingPins, true);
	if (CachedWrapperDestinationNode)
	{
		TArray<FMVVMBlueprintPin> MissingPins = FMVVMBlueprintPin::CopyAndReturnMissingPins(GetWidgetBlueprintInternal(), CachedWrapperDestinationNode, SavedPins);
		SavedPins.Append(MissingPins);
	}
}

TArray<FText> UMVVMBlueprintViewCondition::GetCompilationMessages(EMessageType InMessageType) const
{
	TArray<FText> Result;
	Result.Reset(Messages.Num());
	for (const FMessage& Msg : Messages)
	{
		if (Msg.MessageType == InMessageType)
		{
			Result.Add(Msg.MessageText);
		}
	}
	return Result;
}

bool UMVVMBlueprintViewCondition::HasCompilationMessage(EMessageType InMessageType) const
{
	return Messages.ContainsByPredicate([InMessageType](const FMessage& Other)
	{
		return Other.MessageType == InMessageType;
	});
}

void UMVVMBlueprintViewCondition::AddCompilationToBinding(FMessage MessageToAdd) const
{
	Messages.Add(MoveTemp(MessageToAdd));
}

void UMVVMBlueprintViewCondition::ResetCompilationMessages()
{
	Messages.Reset();
}

FText UMVVMBlueprintViewCondition::GetDisplayName(bool bUseDisplayName) const
{
	TArray<FText> JoinArgs;
	for (const FMVVMBlueprintPin& Pin : GetPins())
	{
		if (Pin.UsedPathAsValue())
		{
			JoinArgs.Add(Pin.GetPath().ToText(GetWidgetBlueprintInternal(), bUseDisplayName));
		}
	}

	return FText::Format(LOCTEXT("BlueprintViewEventDisplayNameFormat", "{0} => {1}({2})")
		, ConditionPath.ToText(GetWidgetBlueprintInternal(), bUseDisplayName)
		, DestinationPath.ToText(GetWidgetBlueprintInternal(), bUseDisplayName)
		, FText::Join(LOCTEXT("PathDelimiter", ", "), JoinArgs)
		);
}

FString UMVVMBlueprintViewCondition::GetSearchableString() const
{
	TStringBuilder<256> Builder;
	Builder << ConditionPath.ToString(GetWidgetBlueprintInternal(), true, true);
	Builder << TEXT(' ');
	Builder << DestinationPath.ToString(GetWidgetBlueprintInternal(), true, true);
	Builder << TEXT('(');
	bool bFirst = true;
	for (const FMVVMBlueprintPin& Pin : GetPins())
	{
		if (!bFirst)
		{
			Builder << TEXT(", ");
		}
		if (Pin.UsedPathAsValue())
		{
			Builder << Pin.GetPath().ToString(GetWidgetBlueprintInternal(), true, true);
		}
		bFirst = false;
	}
	Builder << TEXT(')');
	return Builder.ToString();
}

void UMVVMBlueprintViewCondition::HandleGraphChanged(const FEdGraphEditAction& EditAction)
{
	// #todo add stuff for condition node as well
	if (EditAction.Graph == CachedWrapperGraph && CachedWrapperGraph)
	{
		if (CachedWrapperDestinationNode && EditAction.Nodes.Contains(CachedWrapperDestinationNode))
		{
			if (EditAction.Action == EEdGraphActionType::GRAPHACTION_RemoveNode)
			{
				CachedWrapperDestinationNode = UE::MVVM::ConversionFunctionHelper::GetWrapperNode(CachedWrapperGraph);
				SavePinValues();
				OnWrapperGraphModified.Broadcast();
			}
			else if (EditAction.Action == EEdGraphActionType::GRAPHACTION_EditNode)
			{
				SavePinValues();
				OnWrapperGraphModified.Broadcast();
			}
		}
		else if (CachedWrapperDestinationNode == nullptr && EditAction.Action == EEdGraphActionType::GRAPHACTION_AddNode)
		{
			CachedWrapperDestinationNode = UE::MVVM::ConversionFunctionHelper::GetWrapperNode(CachedWrapperGraph);
			SavePinValues();
			OnWrapperGraphModified.Broadcast();
		}
	}
}

void UMVVMBlueprintViewCondition::HandleUserDefinedPinRenamed(UK2Node* InNode, FName OldPinName, FName NewPinName)
{
	if (InNode == CachedWrapperDestinationNode)
	{
		SavePinValues();
		OnWrapperGraphModified.Broadcast();
	}
}

void UMVVMBlueprintViewCondition::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChainEvent)
{
	Super::PostEditChangeChainProperty(PropertyChainEvent);
	if (bNeedsToRegenerateChildren)
	{
		GetOuterUMVVMBlueprintView()->OnConditionParametersRegenerate.Broadcast(this);
		bNeedsToRegenerateChildren = false;
	}
	GetOuterUMVVMBlueprintView()->OnConditionsUpdated.Broadcast();
}

#undef LOCTEXT_NAMESPACE
