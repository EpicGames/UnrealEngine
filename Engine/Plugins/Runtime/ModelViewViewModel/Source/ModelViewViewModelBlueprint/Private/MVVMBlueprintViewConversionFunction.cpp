// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMDeveloperProjectSettings.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Bindings/MVVMConversionFunctionHelper.h"
#include "Bindings/MVVMBindingHelper.h"
#include "MVVMFunctionGraphHelper.h"
#include "Engine/Blueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "EdGraphSchema_K2.h"
#include "EdGraph/EdGraphPin.h"
#include "GraphEditAction.h"
#include "K2Node_BaseAsyncTask.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "KismetCompiler.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/ValueOrError.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBlueprintViewConversionFunction)

namespace UE::MVVM::Private
{
	static FLazyName DefaultConversionFunctionName = TEXT("__ConversionFunction");
}

bool UMVVMBlueprintViewConversionFunction::IsValidConversionFunction(const UBlueprint* WidgetBlueprint, const UFunction* Function)
{
	check(WidgetBlueprint);
	check(Function);

	// functions in the widget blueprint can do anything they want, other functions have to be static functions in a BlueprintFunctionLibrary
	const UClass* FunctionClass = Function->GetOuterUClass();

	bool bIsFromWidgetBlueprint = WidgetBlueprint->GeneratedClass && WidgetBlueprint->GeneratedClass->IsChildOf(FunctionClass) && Function->HasAllFunctionFlags(FUNC_BlueprintPure | FUNC_Const);
	bool bIsFromSkeletonWidgetBlueprint = WidgetBlueprint->SkeletonGeneratedClass && WidgetBlueprint->SkeletonGeneratedClass->IsChildOf(FunctionClass) && Function->HasAllFunctionFlags(FUNC_BlueprintPure | FUNC_Const);
	bool bFromBlueprintFunctionLibrary = FunctionClass->IsChildOf<UBlueprintFunctionLibrary>() && Function->HasAllFunctionFlags(FUNC_Static | FUNC_BlueprintPure);
	if (!bIsFromWidgetBlueprint && !bIsFromSkeletonWidgetBlueprint && !bFromBlueprintFunctionLibrary)
	{
		return false;
	}

	TValueOrError<const FProperty*, FText> ReturnResult = UE::MVVM::BindingHelper::TryGetReturnTypeForConversionFunction(Function);
	if (ReturnResult.HasError())
	{
		return false;
	}

	const FProperty* ReturnProperty = ReturnResult.GetValue();
	if (ReturnProperty == nullptr)
	{
		return false;
	}

	TValueOrError<TArray<const FProperty*>, FText> ArgumentsResult = UE::MVVM::BindingHelper::TryGetArgumentsForConversionFunction(Function);
	if (ArgumentsResult.HasError())
	{
		return false;
	}

	return GetDefault<UMVVMDeveloperProjectSettings>()->IsConversionFunctionAllowed(WidgetBlueprint, Function);
}

bool UMVVMBlueprintViewConversionFunction::IsValidConversionNode(const UBlueprint* WidgetBlueprint, const TSubclassOf<UK2Node> Function)
{
	check(WidgetBlueprint);
	check(Function.Get());

	UEdGraphPin* ReturnPin = UE::MVVM::ConversionFunctionHelper::FindOutputPin(Function.GetDefaultObject());
	if (ReturnPin == nullptr)
	{
		return false;
	}

	TArray<UEdGraphPin*> ArgumentsResult = UE::MVVM::ConversionFunctionHelper::FindInputPins(Function.GetDefaultObject());
	if (ArgumentsResult.Num() == 0)
	{
		return false;
	}

	return GetDefault<UMVVMDeveloperProjectSettings>()->IsConversionFunctionAllowed(WidgetBlueprint, Function);
}

bool UMVVMBlueprintViewConversionFunction::IsValid(const UBlueprint* SelfContext) const
{
	if (ConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
	{
		UClass* SelfClass = SelfContext->SkeletonGeneratedClass ? SelfContext->SkeletonGeneratedClass : SelfContext->GeneratedClass;
		const UFunction* Function = ConversionFunction.GetFunction(SelfContext);
		if (Function)
		{
			return IsValidConversionFunction(SelfContext, Function);
		}
	}
	else if (ConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Node)
	{
		TSubclassOf<UK2Node> Node = ConversionFunction.GetNode();
		return Node.Get() && GetDefault<UMVVMDeveloperProjectSettings>()->IsConversionFunctionAllowed(SelfContext, Node);
	}
	return false;
}

bool UMVVMBlueprintViewConversionFunction::NeedsWrapperGraph(const UBlueprint* SelfContext) const
{
	UClass* SelfClass = SelfContext->SkeletonGeneratedClass ? SelfContext->SkeletonGeneratedClass : SelfContext->GeneratedClass;
	return NeedsWrapperGraphInternal(SelfClass);
}

bool UMVVMBlueprintViewConversionFunction::NeedsWrapperGraphInternal(const UClass* SkeletalSelfContext) const
{
	if (ConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
	{
		const UFunction* Function = ConversionFunction.GetFunction(SkeletalSelfContext);
		if (ensure(Function))
		{
			bool bNeedsWrapper = SavedPins.Num() > 1 || !UE::MVVM::BindingHelper::IsValidForSimpleRuntimeConversion(Function);
			if (!bNeedsWrapper)
			{
				// Confirms there are no autocast/autopromote node
				if (ensure(CachedWrapperGraph))
				{
					for (UEdGraphNode* Node : CachedWrapperGraph->Nodes)
					{
						if (UE::MVVM::ConversionFunctionHelper::IsAutoPromoteNode(Node))
						{
							bNeedsWrapper = true;
							break;
						}
					}
				}
			}

			return bNeedsWrapper;
		}
	}
	else if (ConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Node)
	{
		return true;
	}
	return false;
}

bool UMVVMBlueprintViewConversionFunction::IsWrapperGraphTransient() const
{
	return bWrapperGraphTransient;
}

bool UMVVMBlueprintViewConversionFunction::IsUbergraphPage() const
{
	return bIsUbergraphPage;
}

void UMVVMBlueprintViewConversionFunction::SetDestinationPath(FMVVMBlueprintPropertyPath InDestinationPath)
{
	if (InDestinationPath == DestinationPath)
	{
		return;
	}

	DestinationPath = MoveTemp(InDestinationPath);
}

const UFunction* UMVVMBlueprintViewConversionFunction::GetCompiledFunction(const UClass* SelfContext) const
{
	if (NeedsWrapperGraphInternal(SelfContext))
	{
		FMemberReference CompiledFunction;
		if (bIsUbergraphPage)
		{
			CompiledFunction.SetSelfMember(UE::MVVM::BindingHelper::GetDelegateSignatureName(GraphName));
		}
		else
		{
			CompiledFunction.SetSelfMember(GraphName);
		}
		return CompiledFunction.ResolveMember<UFunction>(const_cast<UClass*>(SelfContext));
	}

	//NeedsWrapperGraphInternal should return true.
	check(ConversionFunction.GetType() != EMVVMBlueprintFunctionReferenceType::Node);

	// If simple conversion function.
	if (ConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
	{
		return ConversionFunction.GetFunction(SelfContext);
	}
	return nullptr;
}

FName UMVVMBlueprintViewConversionFunction::GetCompiledFunctionName(const UClass* SelfContext) const
{
	if (NeedsWrapperGraphInternal(SelfContext))
	{
		ensure(!GraphName.IsNone());
		return GraphName;
	}
	return ConversionFunction.GetName();
}

FMVVMBlueprintFunctionReference UMVVMBlueprintViewConversionFunction::GetConversionFunction() const
{
	return ConversionFunction;
}

void UMVVMBlueprintViewConversionFunction::Reset()
{
	ConversionFunction = FMVVMBlueprintFunctionReference();
	GraphName = FName();
	bWrapperGraphTransient = true;
	bIsUbergraphPage = false;
	LatentEventNodeUUID = nullptr;
	SavedPins.Reset();
	SetCachedWrapperGraph(nullptr, nullptr, nullptr);
}

void UMVVMBlueprintViewConversionFunction::Initialize(UBlueprint* InContext, FName InGraphName, FMVVMBlueprintFunctionReference InFunction)
{
	Reset();

	if (ensure(InFunction.IsValid(InContext) && !InGraphName.IsNone()))
	{
		ConversionFunction = InFunction;
		check(GraphName.IsNone()); // the name needs to be set before a GetOrCreateWrapperGraph

		bIsUbergraphPage = IsConversionFunctionAsyncNode();

		if (bIsUbergraphPage)
		{
			TStringBuilder<256> StringBuilder;
			StringBuilder << InGraphName;
			StringBuilder << TEXT("_Async");

			GraphName = StringBuilder.ToString();
		}
		else
		{
			GraphName = InGraphName;
		}
		bWrapperGraphTransient = true;
		LatentEventNodeUUID = nullptr;
		CreateWrapperGraphInternal(InContext);
		SavePinValues(InContext);
	}
}

void UMVVMBlueprintViewConversionFunction::InitializeFromFunction(UBlueprint* InContext, FName InGraphName, const UFunction* InFunction)
{
	Reset();

	if (ensure(InFunction && !InGraphName.IsNone()))
	{
		ConversionFunction = FMVVMBlueprintFunctionReference(InContext, InFunction);
		check(GraphName.IsNone()); // the name needs to be set before a GetOrCreateWrapperGraph
		GraphName = InGraphName;
		bWrapperGraphTransient = true;
		bIsUbergraphPage = false;
		LatentEventNodeUUID = nullptr;
		CreateWrapperGraphInternal(InContext);
		SavePinValues(InContext);
	}
}

void UMVVMBlueprintViewConversionFunction::Deprecation_InitializeFromWrapperGraph(UBlueprint* SelfContext, UEdGraph* Graph)
{
	Reset();

	if (UK2Node* WrapperNode = UE::MVVM::ConversionFunctionHelper::GetWrapperNode(Graph))
	{
		if (UK2Node_CallFunction* CallFunction = Cast<UK2Node_CallFunction>(WrapperNode))
		{
			ConversionFunction = FMVVMBlueprintFunctionReference(CallFunction->FunctionReference);
		}
		else
		{
			ConversionFunction = FMVVMBlueprintFunctionReference(WrapperNode->GetClass());
		}
		check(WrapperNode->GetGraph());
		SetCachedWrapperGraph(SelfContext, WrapperNode->GetGraph(), WrapperNode);

		check(GraphName.IsNone());
		GraphName = CachedWrapperGraph->GetFName();
		bWrapperGraphTransient = true;

		SavePinValues(SelfContext);

		if (bWrapperGraphTransient && CachedWrapperNode)
		{
			SelfContext->FunctionGraphs.RemoveSingle(CachedWrapperGraph);
			CachedWrapperGraph->SetFlags(RF_Transient);
		}
	}
}

void UMVVMBlueprintViewConversionFunction::Deprecation_InitializeFromMemberReference(UBlueprint* SelfContext, FName InGraphName, FMemberReference MemberReference, const FMVVMBlueprintPropertyPath& Source)
{
	Reset();

	ConversionFunction = FMVVMBlueprintFunctionReference(MemberReference);

	check(GraphName.IsNone()); // the name needs to be set before a GetOrCreateWrapperGraph
	GraphName = InGraphName;
	bWrapperGraphTransient = true;

	// since it is a new object, we can't create a the graph right away
	UClass* GeneratedClass = SelfContext->SkeletonGeneratedClass ? SelfContext->SkeletonGeneratedClass : SelfContext->GeneratedClass;
	const UFunction* Function = ConversionFunction.GetFunction(GeneratedClass);
	const FProperty* PinProperty = Function ? UE::MVVM::BindingHelper::GetFirstArgumentProperty(Function) : nullptr;
	if (PinProperty)
	{
		FName PinPropertyName = PinProperty->GetFName();
		FMVVMBlueprintPin NewPin = FMVVMBlueprintPin(FMVVMBlueprintPinId(MakeArrayView(&PinPropertyName, 1)));
		NewPin.SetPath(Source);
		SavedPins.Add(MoveTemp(NewPin));
	}
	else
	{
		SavedPins.Reset();
	}
}

void UMVVMBlueprintViewConversionFunction::Deprecation_SetWrapperGraphName(UBlueprint* SelfContext, FName InGraphName, const FMVVMBlueprintPropertyPath& Source)
{
	if (ensure(SavedPins.Num() == 0) && ensure(GraphName.IsNone()))
	{
		GraphName = InGraphName;
		bWrapperGraphTransient = true;

		// since it is a new object, we can't create a the graph right away
		UClass* GeneratedClass = SelfContext->SkeletonGeneratedClass ? SelfContext->SkeletonGeneratedClass : SelfContext->GeneratedClass;
		const UFunction* Function = ConversionFunction.GetFunction(GeneratedClass);
		const FProperty* PinProperty = Function ? UE::MVVM::BindingHelper::GetFirstArgumentProperty(Function) : nullptr;
		if (PinProperty)
		{
			FName PinPropertyName = PinProperty->GetFName();
			FMVVMBlueprintPin NewPin  = FMVVMBlueprintPin(FMVVMBlueprintPinId(MakeArrayView(&PinPropertyName, 1)));
			NewPin.SetPath(Source);
			SavedPins.Add(MoveTemp(NewPin));
		}
		else
		{
			SavedPins.Reset();
		}
	}
}

void UMVVMBlueprintViewConversionFunction::SetCachedWrapperGraph(UBlueprint* Context, UEdGraph* Graph, UK2Node* Node)
{
	if (CachedWrapperNode && OnUserDefinedPinRenamedHandle.IsValid())
	{
		CachedWrapperNode->OnUserDefinedPinRenamed().Remove(OnUserDefinedPinRenamedHandle);
	}
	if (CachedWrapperGraph && OnGraphChangedHandle.IsValid())
	{
		CachedWrapperGraph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
	}

	CachedWrapperGraph = Graph;
	CachedWrapperNode = Node;
	OnGraphChangedHandle.Reset();
	OnUserDefinedPinRenamedHandle.Reset();

	if (CachedWrapperGraph && Context)
	{
		TWeakObjectPtr<UBlueprint> WeakContext = Context;
		OnGraphChangedHandle = CachedWrapperGraph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateUObject(this, &UMVVMBlueprintViewConversionFunction::HandleGraphChanged, WeakContext));
	}
	if (CachedWrapperNode && Context)
	{
		TWeakObjectPtr<UBlueprint> WeakContext = Context;
		OnUserDefinedPinRenamedHandle = CachedWrapperNode->OnUserDefinedPinRenamed().AddUObject(this, &UMVVMBlueprintViewConversionFunction::HandleUserDefinedPinRenamed, WeakContext);
	}
}

FName UMVVMBlueprintViewConversionFunction::GenerateUniqueGraphName() const
{
	TStringBuilder<256> StringBuilder;
	StringBuilder << UE::MVVM::Private::DefaultConversionFunctionName.Resolve();
	StringBuilder << FGuid::NewDeterministicGuid(GetFullName()).ToString(EGuidFormats::DigitsWithHyphensLower);
	return StringBuilder.ToString();
}

void UMVVMBlueprintViewConversionFunction::CreateWrapperGraphName()
{
	ensure(!GraphName.IsNone());
	if (GraphName.IsNone())
	{
		GraphName = GenerateUniqueGraphName();
	}
}

UEdGraph* UMVVMBlueprintViewConversionFunction::GetOrCreateIntermediateWrapperGraph(FKismetCompilerContext& Context)
{
	check(Context.NewClass);

	if (CachedWrapperGraph)
	{
		return CachedWrapperGraph;
	}

	if (ConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::None)
	{
		return nullptr;
	}

	TObjectPtr<UEdGraph>* FoundGraph = !GraphName.IsNone() ? Context.Blueprint->FunctionGraphs.FindByPredicate([GraphName = GetWrapperGraphName()](const UEdGraph* Other) { return Other->GetFName() == GraphName; }) : nullptr;
	if (FoundGraph)
	{
		UBlueprint* NullContext = nullptr; // do not register the callback
		SetCachedWrapperGraph(NullContext, *FoundGraph, UE::MVVM::ConversionFunctionHelper::GetWrapperNode(*FoundGraph));
		LoadPinValuesInternal(Context.Blueprint);

		// Conversion Function graph are not saved in the editor anymore.
		check(CachedWrapperGraph == *FoundGraph);
		Context.Blueprint->FunctionGraphs.RemoveSingle(CachedWrapperGraph);
		CachedWrapperGraph->SetFlags(RF_Transient);

		return CachedWrapperGraph;
	}
	else if (IsValid(Context.Blueprint))
	{
		CreateWrapperGraphName();
		return CreateWrapperGraphInternal(Context);
	}
	return nullptr;
}

UEdGraph* UMVVMBlueprintViewConversionFunction::GetOrCreateWrapperGraph(UBlueprint* Blueprint)
{
	check(Blueprint);

	if (CachedWrapperGraph)
	{
		return CachedWrapperGraph;
	}
	
	if (ConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::None)
	{
		return nullptr;
	}

	TObjectPtr<UEdGraph>* FoundGraph = Blueprint->FunctionGraphs.FindByPredicate([GraphName = GetWrapperGraphName()](const UEdGraph* Other) { return Other->GetFName() == GraphName; });
	if (FoundGraph)
	{
		const_cast<UMVVMBlueprintViewConversionFunction*>(this)->SetCachedWrapperGraph(Blueprint, *FoundGraph, UE::MVVM::ConversionFunctionHelper::GetWrapperNode(*FoundGraph));
		LoadPinValuesInternal(Blueprint);

		// Conversion Function graph are not saved in the editor anymore.
		check(CachedWrapperGraph == *FoundGraph);
		Blueprint->FunctionGraphs.RemoveSingle(CachedWrapperGraph);
		CachedWrapperGraph->SetFlags(RF_Transient);
	}
	else if (IsValid(Blueprint))
	{
		CreateWrapperGraphName();
		return CreateWrapperGraphInternal(Blueprint);
	}
	return nullptr;
}

void UMVVMBlueprintViewConversionFunction::RecreateWrapperGraph(UBlueprint* Blueprint)
{
	if (IsValid(Blueprint))
	{
		GraphName = GenerateUniqueGraphName();
		CreateWrapperGraphInternal(Blueprint);
	}
}

UEdGraphPin* UMVVMBlueprintViewConversionFunction::GetOrCreateGraphPin(UBlueprint* Blueprint, const FMVVMBlueprintPinId& PinId)
{
	GetOrCreateWrapperGraph(Blueprint);
	return CachedWrapperGraph ? UE::MVVM::ConversionFunctionHelper::FindPin(CachedWrapperGraph, PinId.GetNames()) : nullptr;
}

UEdGraph* UMVVMBlueprintViewConversionFunction::CreateWrapperGraphInternal(FKismetCompilerContext& Context)
{
	return CreateWrapperGraphInternal(Context.Blueprint);
}

UEdGraph* UMVVMBlueprintViewConversionFunction::CreateWrapperGraphInternal(UBlueprint* Blueprint)
{
	check(Blueprint);

	bool bConst = true;
	UE::MVVM::ConversionFunctionHelper::FCreateGraphParams Params;
	Params.bIsConst = bConst;
	Params.bTransient = bWrapperGraphTransient;
	Params.bCreateUbergraphPage = false;

	UE::MVVM::ConversionFunctionHelper::FCreateGraphResult Result;
	if (ConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
	{
		const UFunction* Function = ConversionFunction.GetFunction(Blueprint);
		check(Function);
		Result = UE::MVVM::ConversionFunctionHelper::CreateGraph(Blueprint, GraphName, nullptr, Function, Params);
	}
	else if (ConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Node)
	{
		TSubclassOf<UK2Node> Node = ConversionFunction.GetNode();
		check(Node.Get());

		if (IsConversionFunctionAsyncNode())
		{
			Params.bCreateUbergraphPage = true;
			TValueOrError<UE::MVVM::ConversionFunctionHelper::FCreateGraphResult, FText> SetterGraphResult = UE::MVVM::ConversionFunctionHelper::CreateSetterGraph(Blueprint
				, GraphName
				, Node
				, DestinationPath
				, Params);

			// @TODO: UE-221351 - Handle Error Case, although a bit difficult since no error log in this context
			if (!SetterGraphResult.HasError())
			{
				Result = SetterGraphResult.GetValue();
			}
		}
		else
		{
			Result = UE::MVVM::ConversionFunctionHelper::CreateGraph(Blueprint, GraphName, nullptr, Node, Params, [](UK2Node*) {});
		}
	}

	static FName NAME_Hidden("Hidden");
	UE::MVVM::ConversionFunctionHelper::SetMetaData(Result.NewGraph, NAME_Hidden, FStringView());

	bIsUbergraphPage = Result.bIsUbergraphPage;

	// Generate a non-transient node for latent manager to use when handling latents by Node UUID
	if (bIsUbergraphPage)
	{
		LatentEventNodeUUID = NewObject<UEdGraphNode>(this);
		// Create the NodeGuid deterministically. Use this->GetPathName(), for the guid instead of LatentEventNodeUUID->GetPathName(), 
		// to avoid possible indeterminism in the LatentEventNodeUUID's objectname.
		LatentEventNodeUUID->NodeGuid = FGuid::NewDeterministicGuid(this->GetPathName(), 1);
	}

	SetCachedWrapperGraph(Blueprint, Result.NewGraph, Result.WrappedNode);
	LoadPinValuesInternal(Blueprint);
	return CachedWrapperGraph;
}

void UMVVMBlueprintViewConversionFunction::RemoveWrapperGraph(UBlueprint* Blueprint)
{
	TObjectPtr<UEdGraph>* Result = Blueprint->FunctionGraphs.FindByPredicate([WrapperName = GraphName](const UEdGraph* GraphPtr) { return GraphPtr->GetFName() == WrapperName; });
	if (Result)
	{
		FBlueprintEditorUtils::RemoveGraph(Blueprint, Result->Get());
	}
	else
	{
		if (UObject* ExistingObject = StaticFindObject(nullptr, Blueprint, *GraphName.ToString(), EFindObjectFlags::ExactClass))
		{
			UE::MVVM::FunctionGraphHelper::RenameObjectToTransientPackage(ExistingObject);
		}
	}
	bIsUbergraphPage = false;
	SetCachedWrapperGraph(Blueprint, nullptr, nullptr);
}

void UMVVMBlueprintViewConversionFunction::SetGraphPin(UBlueprint* Blueprint, const FMVVMBlueprintPinId& PinId, const FMVVMBlueprintPropertyPath& Path)
{
	UEdGraphPin* GraphPin = GetOrCreateGraphPin(Blueprint, PinId);
	check(GraphPin);

	// Set the value and make the blueprint as dirty before creating the pin.
	//A property may not be created yet and the skeletal needs to be recreated.
	FMVVMBlueprintPin* Pin = SavedPins.FindByPredicate([&PinId](const FMVVMBlueprintPin& Other) { return PinId == Other.GetId(); });
	if (!Pin)
	{
		Pin = &SavedPins.Add_GetRef(FMVVMBlueprintPin::CreateFromPin(Blueprint, GraphPin));
	}
	Pin->SetPath(Path);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	UE::MVVM::ConversionFunctionHelper::SetPropertyPathForPin(Blueprint, Path, GraphPin);
}

bool UMVVMBlueprintViewConversionFunction::IsConversionFunctionAsyncNode()
{
	if (ConversionFunction.GetType() != EMVVMBlueprintFunctionReferenceType::Node)
	{
		return false;
	}

	TSubclassOf<UK2Node> Node = ConversionFunction.GetNode();
	check(Node.Get());

	return (UE::MVVM::ConversionFunctionHelper::IsAsyncNode(Node));
}

void UMVVMBlueprintViewConversionFunction::SavePinValues(UBlueprint* Blueprint)
{
	if (!bLoadingPins) // While loading pins value, the node can trigger a notify that would then trigger a save.
	{
		SavedPins.Empty();
		if (CachedWrapperNode)
		{
			SavedPins = FMVVMBlueprintPin::CreateFromNode(Blueprint, CachedWrapperNode);
		}
	}
}

void UMVVMBlueprintViewConversionFunction::UpdatePinValues(UBlueprint* Blueprint)
{
	if (CachedWrapperNode)
	{
		TArray<FMVVMBlueprintPin> TmpSavedPins = FMVVMBlueprintPin::CreateFromNode(Blueprint, CachedWrapperNode);
		SavedPins.RemoveAll([](const FMVVMBlueprintPin& Pin) { return Pin.GetStatus() != EMVVMBlueprintPinStatus::Orphaned; });
		SavedPins.Append(TmpSavedPins);
	}
}

bool UMVVMBlueprintViewConversionFunction::HasOrphanedPin() const
{
	for (const FMVVMBlueprintPin& Pin : SavedPins)
	{
		if (Pin.GetStatus() == EMVVMBlueprintPinStatus::Orphaned)
		{
			return true;
		}
	}
	return false;
}

void UMVVMBlueprintViewConversionFunction::LoadPinValuesInternal(UBlueprint* Blueprint)
{
	TGuardValue<bool> Tmp (bLoadingPins, true);
	if (CachedWrapperNode)
	{
		TArray<FMVVMBlueprintPin> MissingPins = FMVVMBlueprintPin::CopyAndReturnMissingPins(Blueprint, CachedWrapperNode, SavedPins);
		SavedPins.Append(MissingPins);
	}
}

void UMVVMBlueprintViewConversionFunction::HandleGraphChanged(const FEdGraphEditAction& EditAction, TWeakObjectPtr<UBlueprint> WeakBlueprint)
{
	UBlueprint* Blueprint = WeakBlueprint.Get();
	if (Blueprint && EditAction.Graph == CachedWrapperGraph && CachedWrapperGraph)
	{
		if (CachedWrapperNode && EditAction.Nodes.Contains(CachedWrapperNode))
		{
			if (EditAction.Action == EEdGraphActionType::GRAPHACTION_RemoveNode)
			{
				CachedWrapperNode = UE::MVVM::ConversionFunctionHelper::GetWrapperNode(CachedWrapperGraph);
				SavePinValues(Blueprint);
				OnWrapperGraphModified.Broadcast();
			}
			else if (EditAction.Action == EEdGraphActionType::GRAPHACTION_EditNode)
			{
				SavePinValues(Blueprint);
				OnWrapperGraphModified.Broadcast();
			}
		}
		else if (CachedWrapperNode == nullptr && EditAction.Action == EEdGraphActionType::GRAPHACTION_AddNode)
		{
			CachedWrapperNode = UE::MVVM::ConversionFunctionHelper::GetWrapperNode(CachedWrapperGraph);
			SavePinValues(Blueprint);
			OnWrapperGraphModified.Broadcast();
		}
	}
}

void UMVVMBlueprintViewConversionFunction::HandleUserDefinedPinRenamed(UK2Node* InNode, FName OldPinName, FName NewPinName, TWeakObjectPtr<UBlueprint> WeakBlueprint)
{
	UBlueprint* Blueprint = WeakBlueprint.Get();
	if (Blueprint && InNode == CachedWrapperNode)
	{
		SavePinValues(Blueprint);
		OnWrapperGraphModified.Broadcast();
	}
}

void UMVVMBlueprintViewConversionFunction::PostLoad()
{
	Super::PostLoad();

	if (FunctionNode_DEPRECATED.Get())
	{
		ConversionFunction = FMVVMBlueprintFunctionReference(FunctionNode_DEPRECATED.Get());
		FunctionNode_DEPRECATED = TSubclassOf<UK2Node>();
	}
	else if (!FunctionReference_DEPRECATED.GetMemberName().IsNone())
	{
		ConversionFunction = FMVVMBlueprintFunctionReference(FunctionReference_DEPRECATED);
		FunctionReference_DEPRECATED = FMemberReference();
	}
}
