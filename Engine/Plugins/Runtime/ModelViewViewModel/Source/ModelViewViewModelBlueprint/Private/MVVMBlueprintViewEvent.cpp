// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBlueprintViewEvent.h"
#include "MVVMBlueprintView.h"

#include "Bindings/MVVMConversionFunctionHelper.h"
#include "Bindings/MVVMBindingHelper.h"
#include "Bindings/MVVMFieldPathHelper.h"
#include "Engine/Blueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MVVMConversionFunctionGraphSchema.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "WidgetBlueprint.h"

#include "EdGraphSchema_K2.h"
#include "EdGraph/EdGraphPin.h"
#include "GraphEditAction.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_VariableSet.h"
#include "KismetCompiler.h"
#include "Node/MVVMK2Node_AreSourcesValidForEvent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBlueprintViewEvent)

#define LOCTEXT_NAMESPACE "MVVMBlueprintViewEvent"

void UMVVMBlueprintViewEvent::SetEventPath(FMVVMBlueprintPropertyPath InEventPath)
{
	if (InEventPath == EventPath)
	{
		return;
	}

	UpdatePinValues();
	RemoveWrapperGraph(LeaveConversionFunctionCurrentValues);

	EventPath = MoveTemp(InEventPath);
	GraphName = FName();

	if (EventPath.IsValid())
	{
		TStringBuilder<256> StringBuilder;
		StringBuilder << TEXT("__");
		StringBuilder << FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower);
		GraphName = StringBuilder.ToString();
	}

	bNeedsToRegenerateChildren = true;

	CreateWrapperGraphInternal();
}

void UMVVMBlueprintViewEvent::SetDestinationPath(FMVVMBlueprintPropertyPath InDestinationPath)
{
	if (InDestinationPath == DestinationPath)
	{
		return;
	}

	RemoveWrapperGraph(RemoveConversionFunctionCurrentValues);

	DestinationPath = MoveTemp(InDestinationPath);

	bNeedsToRegenerateChildren = true;

	CreateWrapperGraphInternal();
	SavePinValues();
}

void UMVVMBlueprintViewEvent::SetCachedWrapperGraphInternal(UEdGraph* Graph, UK2Node* Node, UMVVMK2Node_AreSourcesValidForEvent* SourceNode)
{
	if (CachedWrapperNode && OnUserDefinedPinRenamedHandle.IsValid())
	{
		CachedWrapperNode->OnUserDefinedPinRenamed().Remove(OnUserDefinedPinRenamedHandle);
	}
	if (CachedWrapperGraph && OnGraphChangedHandle.IsValid())
	{
		CachedWrapperGraph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
	}

	CachedWrapperGraph = Graph;
	CachedWrapperNode = Node;
	CachedSourceValidNode = SourceNode;
	OnGraphChangedHandle.Reset();
	OnUserDefinedPinRenamedHandle.Reset();

	if (CachedWrapperGraph)
	{
		OnGraphChangedHandle = CachedWrapperGraph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateUObject(this, &UMVVMBlueprintViewEvent::HandleGraphChanged));
	}
	if (CachedWrapperNode)
	{
		OnUserDefinedPinRenamedHandle = CachedWrapperNode->OnUserDefinedPinRenamed().AddUObject(this, &UMVVMBlueprintViewEvent::HandleUserDefinedPinRenamed);
	}
	UpdateEventKeyInternal();
}
	
UEdGraph* UMVVMBlueprintViewEvent::GetOrCreateWrapperGraph()
{
	if (CachedWrapperGraph)
	{
		return CachedWrapperGraph;
	}

	CreateWrapperGraphInternal();
	return CachedWrapperGraph;
}

void UMVVMBlueprintViewEvent::RecreateWrapperGraph()
{
	GraphName = FName();

	RemoveWrapperGraph(LeaveConversionFunctionCurrentValues);

	TStringBuilder<256> StringBuilder;
	StringBuilder << TEXT("__");
	StringBuilder << FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower);
	GraphName = StringBuilder.ToString();

	bNeedsToRegenerateChildren = true;

	CreateWrapperGraphInternal();
}

void UMVVMBlueprintViewEvent::RemoveWrapperGraph(ERemoveWrapperGraphParam ActionForCurrentValues)
{
	if (CachedWrapperGraph)
	{
		FBlueprintEditorUtils::RemoveGraph(GetWidgetBlueprintInternal(), CachedWrapperGraph);
		SetCachedWrapperGraphInternal(nullptr, nullptr, nullptr);
	}

	Messages.Empty();
	if(ActionForCurrentValues == RemoveConversionFunctionCurrentValues)
	{
		SavedPins.Empty();
	}
}

UEdGraphPin* UMVVMBlueprintViewEvent::GetOrCreateGraphPin(const FMVVMBlueprintPinId& PinId)
{
	GetOrCreateWrapperGraph();
	return CachedWrapperGraph ? UE::MVVM::ConversionFunctionHelper::FindPin(CachedWrapperGraph, PinId.GetNames()) : nullptr;
}

void UMVVMBlueprintViewEvent::SavePinValues()
{
	if (!bLoadingPins) // While loading pins value, the node can trigger a notify that would then trigger a save.
	{
		SavedPins.Empty();
		if (CachedWrapperNode)
		{
			UWidgetBlueprint* Blueprint = GetWidgetBlueprintInternal();
			SavedPins = FMVVMBlueprintPin::CreateFromNode(Blueprint, CachedWrapperNode);
		}
	}
}

void UMVVMBlueprintViewEvent::UpdatePinValues()
{
	if (CachedWrapperNode)
	{
		UWidgetBlueprint* Blueprint = GetWidgetBlueprintInternal();
		TArray<FMVVMBlueprintPin> TmpSavedPins = FMVVMBlueprintPin::CreateFromNode(Blueprint, CachedWrapperNode);
		SavedPins.RemoveAll([](const FMVVMBlueprintPin& Pin){ return Pin.GetStatus() != EMVVMBlueprintPinStatus::Orphaned; });
		SavedPins.Append(TmpSavedPins);
	}
}

bool UMVVMBlueprintViewEvent::HasOrphanedPin() const
{
	for (const FMVVMBlueprintPin& Pin : SavedPins)
	{
		if (Pin.GetStatus() == EMVVMBlueprintPinStatus::Orphaned)
		{
			return true;
		}
	}
	return false;
}

void UMVVMBlueprintViewEvent::UpdateEventKey(FMVVMViewClass_EventKey InEventKey)
{
	if (EventKey != InEventKey)
	{
		EventKey = InEventKey;
		UpdateEventKeyInternal();
	}
}

void UMVVMBlueprintViewEvent::UpdateEventKeyInternal()
{
	if (CachedSourceValidNode)
	{
		CachedSourceValidNode->EventKey = EventKey;
	}
}

FMVVMBlueprintPropertyPath UMVVMBlueprintViewEvent::GetPinPath(const FMVVMBlueprintPinId& PinId) const
{
	const FMVVMBlueprintPin* ViewPin = SavedPins.FindByPredicate([&PinId](const FMVVMBlueprintPin& Other) { return PinId == Other.GetId(); });
	return ViewPin ? ViewPin->GetPath() : FMVVMBlueprintPropertyPath();
}

void UMVVMBlueprintViewEvent::SetPinPath(const FMVVMBlueprintPinId& PinId, const FMVVMBlueprintPropertyPath& Path)
{
	UEdGraphPin* GraphPin = GetOrCreateGraphPin(PinId);

	if (GraphPin)
	{
		UBlueprint* Blueprint = GetWidgetBlueprintInternal();
		// Set the value and make the blueprint as dirty before creating the pin.
		FMVVMBlueprintPin* ViewPin = SavedPins.FindByPredicate([&PinId](const FMVVMBlueprintPin& Other) { return PinId == Other.GetId(); });
		if (!ViewPin)
		{
			ViewPin = &SavedPins.Add_GetRef(FMVVMBlueprintPin::CreateFromPin(Blueprint, GraphPin));
		}

		//A property (viewmodel or widget) may not be created yet and the skeletal needs to be recreated.
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

		UE::MVVM::ConversionFunctionHelper::SetPropertyPathForPin(Blueprint, Path, GraphPin);

		// Take the path built in BP, it may had some errors
		ViewPin->SetPath(UE::MVVM::ConversionFunctionHelper::GetPropertyPathForPin(Blueprint, GraphPin, false));
	}
}

void UMVVMBlueprintViewEvent::SetPinPathNoGraphGeneration(const FMVVMBlueprintPinId& PinId, const FMVVMBlueprintPropertyPath& Path)
{
	FMVVMBlueprintPin* ViewPin = SavedPins.FindByPredicate([&PinId](const FMVVMBlueprintPin& Other) { return PinId == Other.GetId(); });
	if (!ViewPin)
	{
		ViewPin = &SavedPins.Emplace_GetRef(PinId);
		ViewPin->SetPath(Path);
	}

	//A property (viewmodel or widget) may not be created yet and the skeletal needs to be recreated.
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetWidgetBlueprintInternal());
}

UWidgetBlueprint* UMVVMBlueprintViewEvent::GetWidgetBlueprintInternal() const
{
	return GetOuterUMVVMBlueprintView()->GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint();
}

bool UMVVMBlueprintViewEvent::Supports(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& PropertyPath)
{
	return GetDefault<UMVVMDeveloperProjectSettings>()->bAllowBindingEvent && GetEventSignature(WidgetBlueprint, PropertyPath) != nullptr;
}

const UFunction* UMVVMBlueprintViewEvent::GetEventSignature() const
{
	return GetEventSignature(GetWidgetBlueprintInternal(), EventPath);
}

const UFunction* UMVVMBlueprintViewEvent::GetEventSignature(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& PropertyPath)
{
	if (PropertyPath.IsValid() && PropertyPath.GetFieldPaths().Num() > 0)
	{
		const FMVVMBlueprintFieldPath& LastPath = PropertyPath.GetFieldPaths().Last();
		UE::MVVM::FMVVMConstFieldVariant LastField = LastPath.GetField(WidgetBlueprint->SkeletonGeneratedClass);
		if (LastField.IsProperty())
		{
			if (const FMulticastDelegateProperty* Property = CastField<FMulticastDelegateProperty>(LastField.GetProperty()))
			{
				return Property->SignatureFunction.Get();
			}
		}
	}
	return nullptr;
}

UEdGraph* UMVVMBlueprintViewEvent::CreateWrapperGraphInternal()
{
	if (GraphName.IsNone() || !DestinationPath.IsValid() || !EventPath.IsValid())
	{
		return nullptr;
	}

	const UFunction* DelegateSignature = GetEventSignature();
	if (DelegateSignature == nullptr)
	{
		return nullptr;
	}

	UWidgetBlueprint* WidgetBlueprint = GetWidgetBlueprintInternal();
	UE::MVVM::ConversionFunctionHelper::FCreateGraphParams Params;
	Params.bIsConst = false;
	Params.bTransient = true;
	Params.bIsForEvent = true;

	TValueOrError<UE::MVVM::ConversionFunctionHelper::FCreateGraphResult, FText> CreateSetterGraphResult = UE::MVVM::ConversionFunctionHelper::CreateSetterGraph(WidgetBlueprint, GraphName, DelegateSignature, DestinationPath, Params);
	if (CreateSetterGraphResult.HasError())
	{
		SetCachedWrapperGraphInternal(nullptr, nullptr, nullptr);
		return nullptr;
	}
	else
	{
		static FName NAME_Hidden("Hidden");
		UE::MVVM::ConversionFunctionHelper::SetMetaData(CreateSetterGraphResult.GetValue().NewGraph, NAME_Hidden, FStringView());

		UMVVMK2Node_AreSourcesValidForEvent* BranchNode = Cast<UMVVMK2Node_AreSourcesValidForEvent>(UE::MVVM::ConversionFunctionHelper::InsertEarlyExitBranchNode(CreateSetterGraphResult.GetValue().NewGraph, UMVVMK2Node_AreSourcesValidForEvent::StaticClass()));
		SetCachedWrapperGraphInternal(CreateSetterGraphResult.GetValue().NewGraph, CreateSetterGraphResult.GetValue().WrappedNode, BranchNode);
		LoadPinValuesInternal();
	}

	return CachedWrapperGraph;
}

void UMVVMBlueprintViewEvent::LoadPinValuesInternal()
{
	TGuardValue<bool> Tmp(bLoadingPins, true);
	if (CachedWrapperNode)
	{
		TArray<FMVVMBlueprintPin> MissingPins = FMVVMBlueprintPin::CopyAndReturnMissingPins(GetWidgetBlueprintInternal(), CachedWrapperNode, SavedPins);
		SavedPins.Append(MissingPins);
	}
}

TArray<FText> UMVVMBlueprintViewEvent::GetCompilationMessages(EMessageType InMessageType) const
{
	TArray<FText> Result;
	Result.Reset(Messages.Num());
	for (const FMessage& Msg : Messages)
	{
		if (Msg.MessageType == InMessageType)
		{
			Result.Add(Msg.MessageText);
		}
	}
	return Result;
}

bool UMVVMBlueprintViewEvent::HasCompilationMessage(EMessageType InMessageType) const
{
	return Messages.ContainsByPredicate([InMessageType](const FMessage& Other)
	{
		return Other.MessageType == InMessageType;
	});
}

void UMVVMBlueprintViewEvent::AddCompilationToBinding(FMessage MessageToAdd) const
{
	Messages.Add(MoveTemp(MessageToAdd));
}

void UMVVMBlueprintViewEvent::ResetCompilationMessages()
{
	Messages.Reset();
}

FText UMVVMBlueprintViewEvent::GetDisplayName(bool bUseDisplayName) const
{
	TArray<FText> JoinArgs;
	for (const FMVVMBlueprintPin& Pin : GetPins())
	{
		if (Pin.UsedPathAsValue())
		{
			JoinArgs.Add(Pin.GetPath().ToText(GetWidgetBlueprintInternal(), bUseDisplayName));
		}
	}

	return FText::Format(LOCTEXT("BlueprintViewEventDisplayNameFormat", "{0} => {1}({2})")
		, EventPath.ToText(GetWidgetBlueprintInternal(), bUseDisplayName)
		, DestinationPath.ToText(GetWidgetBlueprintInternal(), bUseDisplayName)
		, FText::Join(LOCTEXT("PathDelimiter", ", "), JoinArgs)
		);
}

FString UMVVMBlueprintViewEvent::GetSearchableString() const
{
	TStringBuilder<256> Builder;
	Builder << EventPath.ToString(GetWidgetBlueprintInternal(), true, true);
	Builder << TEXT(' ');
	Builder << DestinationPath.ToString(GetWidgetBlueprintInternal(), true, true);
	Builder << TEXT('(');
	bool bFirst = true;
	for (const FMVVMBlueprintPin& Pin : GetPins())
	{
		if (!bFirst)
		{
			Builder << TEXT(", ");
		}
		if (Pin.UsedPathAsValue())
		{
			Builder << Pin.GetPath().ToString(GetWidgetBlueprintInternal(), true, true);
		}
		bFirst = false;
	}
	Builder << TEXT(')');
	return Builder.ToString();
}

void UMVVMBlueprintViewEvent::HandleGraphChanged(const FEdGraphEditAction& EditAction)
{
	if (EditAction.Graph == CachedWrapperGraph && CachedWrapperGraph)
	{
		if (CachedWrapperNode && EditAction.Nodes.Contains(CachedWrapperNode))
		{
			if (EditAction.Action == EEdGraphActionType::GRAPHACTION_RemoveNode)
			{
				CachedWrapperNode = UE::MVVM::ConversionFunctionHelper::GetWrapperNode(CachedWrapperGraph);
				SavePinValues();
				OnWrapperGraphModified.Broadcast();
			}
			else if (EditAction.Action == EEdGraphActionType::GRAPHACTION_EditNode)
			{
				SavePinValues();
				OnWrapperGraphModified.Broadcast();
			}
		}
		else if (CachedWrapperNode == nullptr && EditAction.Action == EEdGraphActionType::GRAPHACTION_AddNode)
		{
			CachedWrapperNode = UE::MVVM::ConversionFunctionHelper::GetWrapperNode(CachedWrapperGraph);
			SavePinValues();
			OnWrapperGraphModified.Broadcast();
		}
	}
}

void UMVVMBlueprintViewEvent::HandleUserDefinedPinRenamed(UK2Node* InNode, FName OldPinName, FName NewPinName)
{
	if (InNode == CachedWrapperNode)
	{
		SavePinValues();
		OnWrapperGraphModified.Broadcast();
	}
}

void UMVVMBlueprintViewEvent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChainEvent)
{
	Super::PostEditChangeChainProperty(PropertyChainEvent);
	if (bNeedsToRegenerateChildren)
	{
		GetOuterUMVVMBlueprintView()->OnEventParametersRegenerate.Broadcast(this);
		bNeedsToRegenerateChildren = false;
	}
	GetOuterUMVVMBlueprintView()->OnEventsUpdated.Broadcast();
}

#undef LOCTEXT_NAMESPACE
