// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBlueprintViewModelContext.h"
#include "MVVMDeveloperProjectSettings.h"
#include "HAL/IConsoleManager.h"
#include "View/MVVMViewModelContextResolver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBlueprintViewModelContext)

FMVVMBlueprintViewModelContext::FMVVMBlueprintViewModelContext(const UClass* InClass, FName InViewModelName)
{
	if (InClass && InClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
	{
#if WITH_EDITOR
		TArray<EMVVMBlueprintViewModelContextCreationType> CreationTypes = UE::MVVM::GetAllowedContextCreationType(InClass);
		if (CreationTypes.Num() > 0)
#endif
		{
			CreationType = CreationTypes[0];
			ViewModelContextId = FGuid::NewGuid();
			NotifyFieldValueClass = const_cast<UClass*>(InClass);
			ViewModelName = InViewModelName;
			bExposeInstanceInEditor = GetDefault<UMVVMDeveloperProjectSettings>()->bExposeViewModelInstanceInEditor;
		}
	}
}


FText FMVVMBlueprintViewModelContext::GetDisplayName() const
{
	return FText::FromName(ViewModelName);
}

#if WITH_EDITOR
TObjectPtr<UMVVMViewModelContextResolver> FMVVMBlueprintViewModelContext::CreateDefaultResolver(UObject* Outer) const
{
	TObjectPtr<UMVVMViewModelContextResolver> DefaultResolver;

	if (CreationType == EMVVMBlueprintViewModelContextCreationType::Resolver)
	{
		if (const UClass* DefaultClass = GetDefault<UMVVMDeveloperProjectSettings>()->DefaultResolverValue.LoadSynchronous())
		{
			if (UMVVMViewModelContextResolver* DefaultResolverClass = Cast<UMVVMViewModelContextResolver>(DefaultClass->GetDefaultObject()))
			{
				if (DefaultResolverClass->DoesSupportViewModelClass(GetViewModelClass()))
				{
					DefaultResolver = NewObject<UMVVMViewModelContextResolver>(Outer, DefaultClass, NAME_None, RF_Public);
				}
			}
		}
	}

	return DefaultResolver;
}

namespace UE::MVVM
{
namespace Private
{
static const FName NAME_MVVMAllowedContextCreationType = TEXT("MVVMAllowedContextCreationType");
static const FName NAME_MVVMDisallowedContextCreationType = TEXT("MVVMDisallowedContextCreationType");

bool GSupportUseAsInterfaceSetting = false;
static FAutoConsoleVariableRef CVarSupportUseAsInterfaceSetting(
	TEXT("MVVM.SupportUseAsInterfaceSetting"),
	GSupportUseAsInterfaceSetting,
	TEXT("Enable in order to have the UseAsInterface option available in UMG's viewmodel."),
	ECVF_ReadOnly
);

// Find the Allowed and Disallowed from first class in the hierarchy that defines it.
void GenerateAllowedList(const UClass* Class, TArray<FString>& AllowedStrings, TArray<FString>& DisallowedStrings)
{
	if (AllowedStrings.Num() == 0 && Class->HasMetaData(NAME_MVVMAllowedContextCreationType))
	{
		Class->GetMetaData(NAME_MVVMAllowedContextCreationType).ParseIntoArray(AllowedStrings, TEXT("|"));
	}
	if (DisallowedStrings.Num() == 0 && Class->HasMetaData(NAME_MVVMDisallowedContextCreationType))
	{
		Class->GetMetaData(NAME_MVVMDisallowedContextCreationType).ParseIntoArray(DisallowedStrings, TEXT("|"));
	}

	UClass* ParentClass = Class->GetSuperClass();
	if (ParentClass && AllowedStrings.Num() == 0 && DisallowedStrings.Num() == 0)
	{
		GenerateAllowedList(ParentClass, AllowedStrings, DisallowedStrings);
	}
}

TArray<EMVVMBlueprintViewModelContextCreationType> GenerateAllowedList(const UClass* Class)
{
	const UEnum* EnumCreationType = StaticEnum<EMVVMBlueprintViewModelContextCreationType>();
	TArray<FString> AllowedStrings;
	TArray<FString> DisallowedStrings;
	GenerateAllowedList(Class, AllowedStrings, DisallowedStrings);

	if (AllowedStrings.Num() != 0)
	{
		for (FString& Trim : AllowedStrings)
		{
			Trim.TrimStartAndEndInline();
		}
		AllowedStrings.RemoveSwap(FString()); // Remove empty lines
	}
	if (DisallowedStrings.Num() != 0)
	{
		for (FString& Trim : DisallowedStrings)
		{
			Trim.TrimStartAndEndInline();
		}
		DisallowedStrings.RemoveSwap(FString()); // Remove empty lines
	}

	TArray<EMVVMBlueprintViewModelContextCreationType, TInlineAllocator<8>> FullList;
	ensure(EnumCreationType->NumEnums() < 8); // just upgrade the FullList InlineAllocator to a bigger number.
	for (int32 Index = 0; Index < EnumCreationType->NumEnums() - 1; ++Index)
	{
		EMVVMBlueprintViewModelContextCreationType CreationType = static_cast<EMVVMBlueprintViewModelContextCreationType>(EnumCreationType->GetValueByIndex(Index));

		const bool bIsHidden = EnumCreationType->HasMetaData(TEXT("Hidden"), Index);
		const bool bAllowed = GetDefault<UMVVMDeveloperProjectSettings>()->IsContextCreationTypeAllowed(CreationType);

		if (!bIsHidden && bAllowed)
		{
			FullList.Add(CreationType);
		}
	}

	TArray<EMVVMBlueprintViewModelContextCreationType> Result;
	Result.Reserve(EnumCreationType->NumEnums());
	if (AllowedStrings.Num() != 0 || DisallowedStrings.Num() != 0)
	{
		auto RunAllowed = [&Result, &AllowedStrings, &FullList, EnumCreationType]()
		{
			for (const FString& Allowed : AllowedStrings)
			{
				if (Allowed == TEXT("all"))
				{
					Result = FullList;
					break;
				}

				int32 FoundIndex = EnumCreationType->GetIndexByName(*Allowed);
				if (ensure(FoundIndex != INDEX_NONE))
				{
					Result.AddUnique(static_cast<EMVVMBlueprintViewModelContextCreationType>(EnumCreationType->GetValueByIndex(FoundIndex)));
				}
			}
		};
		auto RunDisallowed = [&Result, &DisallowedStrings, EnumCreationType]()
		{
			for (const FString& Disallowed : DisallowedStrings)
			{
				int32 FoundIndex = EnumCreationType->GetIndexByName(*Disallowed);
				if (ensure(FoundIndex != INDEX_NONE))
				{
					Result.RemoveSingleSwap(static_cast<EMVVMBlueprintViewModelContextCreationType>(EnumCreationType->GetValueByIndex(FoundIndex)));
				}
			}
		};

		if (AllowedStrings.Num() != 0 && DisallowedStrings.Num() == 0)
		{
			RunAllowed();
		}
		else if (AllowedStrings.Num() == 0 && DisallowedStrings.Num() != 0)
		{
			Result = FullList;
			RunDisallowed();
		}
		else
		{
			ensureMsgf(false, TEXT("Disallowed and Allowed list found for class '%s'. Only provide one of the options."), *Class->GetFullName());
			RunAllowed();
			RunDisallowed();
		}
	}
	else
	{
		Result = FullList;
	}

	return Result;
}
} //namespace Private

TArray<EMVVMBlueprintViewModelContextCreationType> GetAllowedContextCreationType(const UClass* Class)
{
	return Private::GenerateAllowedList(Class);
}
}//namespace UE::MVVM
#endif