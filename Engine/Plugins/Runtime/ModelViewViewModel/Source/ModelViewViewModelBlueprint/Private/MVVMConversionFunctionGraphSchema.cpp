// Copyright Epic Games, Inc. All Rights Reserved.


#include "MVVMConversionFunctionGraphSchema.h"
#include "Bindings/MVVMConversionFunctionHelper.h"
#include "EdGraphSchema_K2_Actions.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_DynamicCast.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMConversionFunctionGraphSchema)


bool UMVVMConversionFunctionGraphSchema::TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const
{
	bool bResult = Super::TryCreateConnection(A, B);

	if (bResult && !A->LinkedTo.Contains(B))
	{
		// ConversionNode or Promotion node was created between the 2 graph pins
		if (A->LinkedTo.Num() == 1)
		{
			if (UEdGraphNode* AutogeneratedNode = A->LinkedTo[0]->GetOwningNode())
			{
				for (UEdGraphPin* AutogeneratedPin : AutogeneratedNode->Pins)
				{
					if (AutogeneratedPin->LinkedTo.Contains(B))
					{
						UE::MVVM::ConversionFunctionHelper::MarkNodeAsAutoPromote(AutogeneratedNode);
						break;
					}
				}
			}
		}
	}
	return bResult;
}

const FPinConnectionResponse UMVVMConversionFunctionGraphSchema::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	check(PinA);
	check(PinB);
	if (PinA->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject && PinB->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
	{
		FPinConnectionResponse Response(CONNECT_RESPONSE_DISALLOW, FString::Printf(TEXT("Conversion disallowed from %s to %s"), *UEdGraphSchema_K2::PC_SoftObject.ToString(), *UEdGraphSchema_K2::PC_Object.ToString()));
		Response.SetFatal();
		return Response;
	}

	return Super::CanCreateConnection(PinA, PinB);
}

bool UMVVMAsyncConversionFunctionGraphSchema::CreateAutomaticConversionNodeAndConnections(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	// Determine which pin is an input and which pin is an output
	UEdGraphPin* InputPin = nullptr;
	UEdGraphPin* OutputPin = nullptr;
	if (!CategorizePinsByDirection(PinA, PinB, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return false;
	}

	check(InputPin);
	check(OutputPin);

	UK2Node* TemplateConversionNode = nullptr;

	if (TOptional<FFindSpecializedConversionNodeResults> ConversionResult = FindSpecializedConversionNode(OutputPin->PinType, *InputPin, true))
	{
		TemplateConversionNode = ConversionResult->TargetNode;
	}

	// Note: We return true if graph is modified, thus we should do all our desired checks, then chose to modify or not & return true / false appropriately
	if (TemplateConversionNode != nullptr && TemplateConversionNode->IsA(UK2Node_DynamicCast::StaticClass()))
	{
		if (UK2Node_DynamicCast* DynamicCastNode = Cast<UK2Node_DynamicCast>(TemplateConversionNode))
		{
			if (UK2Node* InputPinNode = Cast<UK2Node>(InputPin->GetOwningNode()))
			{
				UEdGraphPin* InputExecPin = InputPinNode->GetExecPin();

				if (InputExecPin && InputExecPin->LinkedTo.Num() == 1)
				{
					// By default Dynamic Cast Nodes generated by auto cast will be pure, convert to nonpure and handle exec connections 
					DynamicCastNode->SetPurity(false);

					// Determine where to position the new node (assuming it isn't going to get beaded)
					FVector2D AverageLocation = CalculateAveragePositionBetweenNodes(InputPin, OutputPin);

					// Connect the cast node up to the output/input pins
					UK2Node* ConversionNode = FEdGraphSchemaAction_K2NewNode::SpawnNodeFromTemplate<UK2Node>(InputPin->GetOwningNode()->GetGraph(), TemplateConversionNode, AverageLocation);
					AutowireConversionNode(InputPin, OutputPin, ConversionNode);

					// Reroute the exec connection on our target pin's node to have the dynamic cast. It must already be set.
					UEdGraphPin* ToInputExecPin = InputExecPin->LinkedTo[0];
					UEdGraphPin* CastExecPin = ConversionNode->GetExecPin();
					UEdGraphPin* CastThenPin = ConversionNode->GetThenPin();

					ToInputExecPin->BreakLinkTo(InputExecPin);
					TryCreateConnection(ToInputExecPin, CastExecPin);
					TryCreateConnection(CastThenPin, InputExecPin);

					return true;
				}
			}
		}

		// We couldn't handle the dynamic cast as desired, fail connections
		return false;
	}

	return Super::CreateAutomaticConversionNodeAndConnections(PinA, PinB);
}

TOptional<UEdGraphSchema_K2::FFindSpecializedConversionNodeResults> UMVVMAsyncConversionFunctionGraphSchema::FindSpecializedConversionNode(const FEdGraphPinType& OutputPinType, const UEdGraphPin& InputPin, bool bCreateNode) const
{
	TOptional<FFindSpecializedConversionNodeResults> Result = Super::FindSpecializedConversionNode(OutputPinType, InputPin, bCreateNode);

	if (!Result.IsSet())
	{
		// In MVVM, allow autocast for object types where trivial
		if (!OutputPinType.IsContainer())
		{
			FEdGraphPinType InputPinType = InputPin.PinType;

			// Note: Cast nodes do not support a ForEach-style expansion, so we exclude container types here.
			const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(InputPin.GetOwningNode());
			UClass* BlueprintClass = (Blueprint->GeneratedClass != nullptr) ? Blueprint->GeneratedClass : Blueprint->ParentClass;

			UClass* InputClass = Cast<UClass>(InputPin.PinType.PinSubCategoryObject.Get());

			if ((InputClass == nullptr) && (InputPin.PinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self))
			{
				InputClass = BlueprintClass;
			}

			const UClass* OutputClass = Cast<UClass>(OutputPinType.PinSubCategoryObject.Get());

			if ((OutputClass == nullptr) && (OutputPinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self))
			{
				OutputClass = BlueprintClass;
			}

			bool bNeedsDynamicCast = false;
			if ((OutputPinType.PinCategory == PC_Interface) && (InputPinType.PinCategory == PC_Object))
			{
				bNeedsDynamicCast = (InputClass && OutputClass) && (InputClass->ImplementsInterface(OutputClass) || OutputClass->IsChildOf(InputClass));
			}
			else if (OutputPinType.PinCategory == PC_Object)
			{
				if ((InputPinType.PinCategory == PC_Object))
				{
					bNeedsDynamicCast = (InputClass && OutputClass) && InputClass->IsChildOf(OutputClass);
				}
			}

			if (bNeedsDynamicCast)
			{
				Result = { nullptr };
				if (bCreateNode)
				{
					UK2Node_DynamicCast* DynCastNode = NewObject<UK2Node_DynamicCast>();
					DynCastNode->TargetType = InputClass;
					DynCastNode->SetPurity(true);
					Result->TargetNode = DynCastNode;
				}
			}
		}
	}

	return Result;
}

///////////////////////////////////////////////////////////////////////////////

EGraphType UMVVMFakeTestUbergraphSchema::GetGraphType(const UEdGraph* TestEdGraph) const
{
	return EGraphType::GT_Ubergraph;
}

UMVVMFakeTestUbergraph::UMVVMFakeTestUbergraph()
{
	Schema = UMVVMFakeTestUbergraphSchema::StaticClass();
}

UMVVMFakeTestFunctiongraph::UMVVMFakeTestFunctiongraph()
{
	Schema = UMVVMConversionFunctionGraphSchema::StaticClass();
}
