// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphSchema_K2.h"

#include "MVVMConversionFunctionGraphSchema.generated.h"

/**
 * Schema for conversion functions, adds pin metadata needed on connections for MVVM
 */
UCLASS()
class UMVVMConversionFunctionGraphSchema : public UEdGraphSchema_K2
{
	GENERATED_BODY()

public:
	//~ Begin EdGraphSchema Interface
	virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	//~ End EdGraphSchema Interface
};

/**
 * Schema used by async conversion functions, currently same as regular graph schema. But will autocast objects.
 */
UCLASS()
class UMVVMAsyncConversionFunctionGraphSchema : public UMVVMConversionFunctionGraphSchema
{
	GENERATED_BODY()

public:
	//~ Begin EdGraphSchema Interface
	virtual bool CreateAutomaticConversionNodeAndConnections(UEdGraphPin* A, UEdGraphPin* B) const override;
	[[nodiscard]] virtual TOptional<FFindSpecializedConversionNodeResults> FindSpecializedConversionNode(const FEdGraphPinType& OutputPinType, const UEdGraphPin& InputPin, bool bCreateNode) const override;
	//~ End EdGraphSchema Interface
};

/**
 * Schema used to test if a node is async or not
 * 
 * Note: Keep in private header.
 */
UCLASS(Hidden, HideDropDown)
class UMVVMFakeTestUbergraphSchema : public UEdGraphSchema_K2
{
	GENERATED_BODY()

public:
	virtual EGraphType GetGraphType(const UEdGraph* TestEdGraph) const override;
};

/**
 * Graph used to test if a node is async or not
 *
 * Note: Keep in private header.
 */
UCLASS(Hidden, HideDropDown)
class UMVVMFakeTestUbergraph : public UEdGraph
{
	GENERATED_BODY()

public:
	UMVVMFakeTestUbergraph();
};

/**
 * Graph used to test if a node is async or not
 *
 * Note: Keep in private header.
 */
UCLASS(Hidden, HideDropDown)
class UMVVMFakeTestFunctiongraph : public UEdGraph
{
	GENERATED_BODY()

public:
	UMVVMFakeTestFunctiongraph();
};
