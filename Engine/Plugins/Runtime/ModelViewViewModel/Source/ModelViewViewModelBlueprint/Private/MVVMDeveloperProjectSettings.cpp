// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMDeveloperProjectSettings.h"

#include "BlueprintEditorSettings.h"
#include "Engine/Blueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MVVMBlueprintViewModelContext.h"
#include "PropertyPermissionList.h"
#include "Types/MVVMExecutionMode.h"
#include "UObject/UnrealType.h"

#include "K2Node_FormatText.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "Components/HorizontalBox.h"
#include "Components/ListView.h"
#include "Components/ScrollBox.h"
#include "Components/StackBox.h"
#include "Components/VerticalBox.h"
#include "Components/WrapBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMDeveloperProjectSettings)

#define LOCTEXT_NAMESPACE "MVVMDeveloperProjectSettings"

UMVVMDeveloperProjectSettings::UMVVMDeveloperProjectSettings()
{
	AllowedExecutionMode.Add(EMVVMExecutionMode::Immediate);
	AllowedExecutionMode.Add(EMVVMExecutionMode::Delayed);
	AllowedExecutionMode.Add(EMVVMExecutionMode::Tick);
	AllowedExecutionMode.Add(EMVVMExecutionMode::DelayedWhenSharedElseImmediate);
	
	AllowedContextCreationType.Add(EMVVMBlueprintViewModelContextCreationType::Manual);
	AllowedContextCreationType.Add(EMVVMBlueprintViewModelContextCreationType::CreateInstance);
	AllowedContextCreationType.Add(EMVVMBlueprintViewModelContextCreationType::GlobalViewModelCollection);
	AllowedContextCreationType.Add(EMVVMBlueprintViewModelContextCreationType::PropertyPath);
	AllowedContextCreationType.Add(EMVVMBlueprintViewModelContextCreationType::PropertyPath);
	AllowedContextCreationType.Add(EMVVMBlueprintViewModelContextCreationType::Resolver);
	AllowedContextCreationType.Add(EMVVMBlueprintViewModelContextCreationType::Context);

	FTopLevelAssetPath BlueprintFunctionLibrary = FTopLevelAssetPath("/Script/Engine", "BlueprintFunctionLibrary");
	FTopLevelAssetPath FormatText = FTopLevelAssetPath("/Script/BlueprintGraph", "K2Node_FormatText");
	FTopLevelAssetPath GenericToText = FTopLevelAssetPath("/Script/BlueprintGraph", "K2Node_GenericToText");
	FTopLevelAssetPath LoadAsset = FTopLevelAssetPath("/Script/BlueprintGraph", "K2Node_LoadAsset");
	AllowedClassForConversionFunctions.Add(FSoftClassPath(BlueprintFunctionLibrary.ToString()));
	AllowedClassForConversionFunctions.Add(FSoftClassPath(FormatText.ToString()));
	AllowedClassForConversionFunctions.Add(FSoftClassPath(GenericToText.ToString()));
	AllowedClassForConversionFunctions.Add(FSoftClassPath(LoadAsset.ToString()));

	SupportedListViewBaseClassesForExtension.Add(UListView::StaticClass());
	SupportedPanelClassesForExtension.Add(UHorizontalBox::StaticClass());
	SupportedPanelClassesForExtension.Add(UVerticalBox::StaticClass());
	SupportedPanelClassesForExtension.Add(UScrollBox::StaticClass());
	SupportedPanelClassesForExtension.Add(UStackBox::StaticClass());
	SupportedPanelClassesForExtension.Add(UWrapBox::StaticClass());
}

FName UMVVMDeveloperProjectSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

FText UMVVMDeveloperProjectSettings::GetSectionText() const
{
	return LOCTEXT("MVVMProjectSettings", "UMG Model View Viewmodel");
}

#if WITH_EDITOR
void UMVVMDeveloperProjectSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.Property->GetFName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UMVVMDeveloperProjectSettings, ConversionFunctionFilter)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UMVVMDeveloperProjectSettings, AllowedClassForConversionFunctions)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UMVVMDeveloperProjectSettings, DeniedClassForConversionFunctions)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UMVVMDeveloperProjectSettings, DeniedModuleForConversionFunctions))
	{
		OnLibrarySettingChanged.Broadcast();
	}
}
#endif

bool UMVVMDeveloperProjectSettings::PropertyHasFiltering(const UStruct* ObjectStruct, const FProperty* Property) const
{
	check(ObjectStruct);
	check(Property);

	const UClass* AuthoritativeClass = Cast<const UClass>(ObjectStruct);
	ObjectStruct = AuthoritativeClass ? AuthoritativeClass->GetAuthoritativeClass() : ObjectStruct;
	if (!FPropertyEditorPermissionList::Get().HasFiltering(ObjectStruct))
	{
		return false;
	}

	TStringBuilder<512> StringBuilder;
	Property->GetOwnerClass()->GetPathName(nullptr, StringBuilder);
	FSoftClassPath StructPath;
	StructPath.SetPath(StringBuilder);

	if (ObjectStruct)
	{
		for (const TPair<FSoftClassPath, FMVVMDeveloperProjectWidgetSettings>& PermissionItem : FieldSelectorPermissions)
		{
			if (UClass* ConcreteClass = PermissionItem.Key.ResolveClass())
			{
				if (ObjectStruct->IsChildOf(ConcreteClass))
				{
					const FMVVMDeveloperProjectWidgetSettings& Settings = PermissionItem.Value;
					if (Settings.DisallowedFieldNames.Contains(Property->GetFName()))
					{
						return false;
					}
				}
			}
		}
	}
	return true;
}

namespace UE::MVVM::Private
{
//class ClassA { int A };
//class ClassB { };
//MyClassB.A; Maybe ClassB doesn't have the permission to use ClassA::A. Maybe MyClassB has the persmission but MyClassA doesn't have it.

//GeneratingFor: the blueprint it's is executed from
//AccessorOwner: the ClassB
//FieldClassOwner: ClassA
bool ShouldDoFieldEditorPermission(const UBlueprint* GeneratingFor, const UClass* AccessorOwner, const UClass* FieldClassOwner)
{
	if (GeneratingFor && FieldClassOwner)
	{
		const UClass* UpToDateClass = FBlueprintEditorUtils::GetMostUpToDateClass(FieldClassOwner);
		return GeneratingFor->SkeletonGeneratedClass != UpToDateClass;
	}
	return true;
}
}//namespace

bool UMVVMDeveloperProjectSettings::IsPropertyAllowed(const UBlueprint* GeneratingFor, const UStruct* ObjectStruct, const FProperty* Property) const
{
	check(GeneratingFor);
	check(ObjectStruct);
	check(Property);

	const UClass* AuthoritativeClass = Cast<const UClass>(ObjectStruct);
	AuthoritativeClass = AuthoritativeClass ? AuthoritativeClass->GetAuthoritativeClass() : nullptr;

	const bool bDoPropertyEditorPermission = UE::MVVM::Private::ShouldDoFieldEditorPermission(GeneratingFor, AuthoritativeClass, Property->GetOwnerClass());
	if (bDoPropertyEditorPermission)
	{
		if (FPropertyEditorPermissionList::Get().IsEnabledForPackage(GeneratingFor->GetPackage()) && !FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(AuthoritativeClass, Property->GetFName()))
		{
			return false;
		}
	}

	if (AuthoritativeClass)
	{
		TStringBuilder<512> StringBuilder;
		AuthoritativeClass->GetPathName(nullptr, StringBuilder);
		FSoftClassPath StructPath;
		StructPath.SetPath(StringBuilder.ToView());

		for (const TPair<FSoftClassPath, FMVVMDeveloperProjectWidgetSettings>& PermissionItem : FieldSelectorPermissions)
		{
			if (UClass* ConcreteClass = PermissionItem.Key.ResolveClass())
			{
				if (AuthoritativeClass->IsChildOf(ConcreteClass))
				{
					const FMVVMDeveloperProjectWidgetSettings& Settings = PermissionItem.Value;
					if (Settings.DisallowedFieldNames.Contains(Property->GetFName()))
					{
						return false;
					}
				}
			}
		}
	}
	return true;
}

bool UMVVMDeveloperProjectSettings::IsFunctionAllowed(const UBlueprint* GeneratingFor, const UClass* ObjectClass, const UFunction* Function) const
{
	check(GeneratingFor);
	check(ObjectClass);
	check(Function);

	const UClass* AuthoritativeClass = ObjectClass->GetAuthoritativeClass();
	if (AuthoritativeClass == nullptr)
	{
		return false;
	}

	const FPathPermissionList& FunctionPermissions = GetMutableDefault<UBlueprintEditorSettings>()->GetFunctionPermissions();
	if (FunctionPermissions.HasFiltering())
	{
		const bool bDoPropertyEditorPermission = UE::MVVM::Private::ShouldDoFieldEditorPermission(GeneratingFor, AuthoritativeClass, Function->GetOwnerClass());
		if (bDoPropertyEditorPermission)
		{
			const UFunction* FunctionToTest = AuthoritativeClass->FindFunctionByName(Function->GetFName());
			if (FunctionToTest == nullptr)
			{
				return false;
			}

			TStringBuilder<512> StringBuilder;
			FunctionToTest->GetPathName(nullptr, StringBuilder);
			if (!FunctionPermissions.PassesFilter(StringBuilder.ToView()))
			{
				return false;
			}
		}
	}

	{
		TStringBuilder<512> StringBuilder;
		AuthoritativeClass->GetPathName(nullptr, StringBuilder);
		FSoftClassPath StructPath;
		StructPath.SetPath(StringBuilder);

		for (const TPair<FSoftClassPath, FMVVMDeveloperProjectWidgetSettings>& PermissionItem : FieldSelectorPermissions)
		{
			if (UClass* ConcreteClass = PermissionItem.Key.ResolveClass())
			{
				if (AuthoritativeClass->IsChildOf(ConcreteClass))
				{
					const FMVVMDeveloperProjectWidgetSettings& Settings = PermissionItem.Value;
					if (Settings.DisallowedFieldNames.Contains(Function->GetFName()))
					{
						return false;
					}
				}
			}
		}
	}

	return true;
}

namespace UE::MVVM::Private
{
bool IsConversionFunctionAllowed(const TSet<FSoftClassPath>& AllowedClasses, const TSet<FSoftClassPath>& DeniedClasses, const TSet<FName>& DeniedModules, UClass* CurrentClass)
{
	bool bIsModuleDenied = DeniedModules.Contains(CurrentClass->GetClassPathName().GetPackageName());
	if (bIsModuleDenied)
	{
		return false;
	}
	while (CurrentClass)
	{
		TStringBuilder<512> FunctionClassPath;
		CurrentClass->GetPathName(nullptr, FunctionClassPath);
		TStringBuilder<512> ToTestClassPath;


		for (const FSoftClassPath& SoftClass : DeniedClasses)
		{
			SoftClass.ToString(ToTestClassPath);
			if (ToTestClassPath.ToView() == FunctionClassPath.ToView())
			{
				return false;
			}
			ToTestClassPath.Reset();
		}

		for (const FSoftClassPath& SoftClass : AllowedClasses)
		{
			SoftClass.ToString(ToTestClassPath);
			if (ToTestClassPath.ToView() == FunctionClassPath.ToView())
			{
				return true;
			}
			ToTestClassPath.Reset();
		}


		CurrentClass = CurrentClass->GetSuperClass();
	}
	return false;
}
} //namespace

bool UMVVMDeveloperProjectSettings::IsConversionFunctionAllowed(const UBlueprint* GeneratingFor, const UFunction* Function) const
{
	if (ConversionFunctionFilter == EMVVMDeveloperConversionFunctionFilterType::BlueprintActionRegistry)
	{
		return IsFunctionAllowed(GeneratingFor, Function->GetOwnerClass(), Function);
	}
	else
	{
		check(ConversionFunctionFilter == EMVVMDeveloperConversionFunctionFilterType::AllowedList);

		// Optimization. Static are for functions inside the AllowedClassForConversionFunctions.
		if (Function->HasAllFunctionFlags(FUNC_Static))
		{
			UClass* CurrentClass = Function->GetOwnerClass();
			return UE::MVVM::Private::IsConversionFunctionAllowed(AllowedClassForConversionFunctions, DeniedClassForConversionFunctions, DeniedModuleForConversionFunctions, CurrentClass);
		}
		else
		{
			// The function is on self (WidgetBlueprint) and may be filtered.
			return IsFunctionAllowed(GeneratingFor, Function->GetOwnerClass(), Function);
		}
	}
}

bool UMVVMDeveloperProjectSettings::IsConversionFunctionAllowed(const UBlueprint* Context, const TSubclassOf<UK2Node> Function) const
{
	if (ConversionFunctionFilter == EMVVMDeveloperConversionFunctionFilterType::BlueprintActionRegistry)
	{
		return !Function.Get()->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists);
	}
	else
	{
		check(ConversionFunctionFilter == EMVVMDeveloperConversionFunctionFilterType::AllowedList);

		return UE::MVVM::Private::IsConversionFunctionAllowed(AllowedClassForConversionFunctions, DeniedClassForConversionFunctions, DeniedModuleForConversionFunctions, Function.Get());
	}
}

TArray<const UClass*> UMVVMDeveloperProjectSettings::GetAllowedConversionFunctionClasses() const
{
	TArray<const UClass*> Result;
	for (const FSoftClassPath& SoftClass : AllowedClassForConversionFunctions)
	{
		if (UClass* Class = SoftClass.ResolveClass())
		{
			Result.Add(Class);
		}
	}

	return Result;
}

TArray<const UClass*> UMVVMDeveloperProjectSettings::GetDeniedConversionFunctionClasses() const
{
	TArray<const UClass*> Result;
	for (const FSoftClassPath& SoftClass : DeniedClassForConversionFunctions)
	{
		if (UClass* Class = SoftClass.ResolveClass())
		{
			Result.Add(Class);
		}
	}

	return Result;
}

bool UMVVMDeveloperProjectSettings::IsExtensionSupportedForPanelClass(TSubclassOf<UPanelWidget> ClassToSupport) const
{
	if (ClassToSupport.Get())
	{
		for (const TSoftClassPtr<UPanelWidget>& SoftClass : SupportedPanelClassesForExtension)
		{
			if (UClass* Class = SoftClass.Get())
			{
				if (ClassToSupport->IsChildOf(Class))
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool UMVVMDeveloperProjectSettings::IsExtensionSupportedForListViewBaseClass(TSubclassOf<UListViewBase> ClassToSupport) const
{
	if (ClassToSupport.Get())
	{
		for (const TSoftClassPtr<UListViewBase>& SoftClass : SupportedListViewBaseClassesForExtension)
		{
			if (UClass* Class = SoftClass.Get())
			{
				if (ClassToSupport->IsChildOf(Class))
				{
					return true;
				}
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
