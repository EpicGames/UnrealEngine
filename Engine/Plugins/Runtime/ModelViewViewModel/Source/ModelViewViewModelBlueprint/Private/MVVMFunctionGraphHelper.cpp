// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMFunctionGraphHelper.h"

#include "EdGraph/EdGraph.h"
#include "K2Node_CallFunction.h"
#include "K2Node_DynamicCast.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "KismetCompiler.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "View/MVVMView.h"
#include "MVVMViewModelBase.h"

#if WITH_EDITOR
#include "EdGraphSchema_K2.h"
#endif

namespace UE::MVVM::FunctionGraphHelper
{

namespace Private
{

UK2Node_FunctionEntry* FindFunctionEntry(const UEdGraph* Graph)
{
	for (UEdGraphNode* Node : Graph->Nodes)
	{
		UK2Node_FunctionEntry* FunctionEntry = Cast<UK2Node_FunctionEntry>(Node);
		if (FunctionEntry)
		{
			return FunctionEntry;
		}
	}
	return nullptr;
}

} //namespace

UEdGraph* CreateFunctionGraph(UBlueprint* InBlueprint, FStringView InFunctionName, EFunctionFlags ExtraFunctionFlag, const FStringView Category, bool bIsEditable)
{
	FName UniqueFunctionName = FBlueprintEditorUtils::FindUniqueKismetName(InBlueprint, InFunctionName.GetData());
	UEdGraph* FunctionGraph = FBlueprintEditorUtils::CreateNewGraph(InBlueprint, UniqueFunctionName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());

	// This is sometime needed. Needs more investigation.
	//InBlueprint->FunctionGraphs.Add(FunctionGraph);

	const UEdGraphSchema_K2* Schema = CastChecked<UEdGraphSchema_K2>(FunctionGraph->GetSchema());
	Schema->MarkFunctionEntryAsEditable(FunctionGraph, bIsEditable);
	Schema->CreateDefaultNodesForGraph(*FunctionGraph);

	// function entry node
	FGraphNodeCreator<UK2Node_FunctionEntry> FunctionEntryCreator(*FunctionGraph);
	UK2Node_FunctionEntry* FunctionEntry = FunctionEntryCreator.CreateNode();
	FunctionEntry->FunctionReference.SetSelfMember(FunctionGraph->GetFName());
	FunctionEntry->AddExtraFlags(ExtraFunctionFlag);
	FunctionEntry->bIsEditable = bIsEditable;
	FunctionEntry->MetaData.Category = FText::FromStringView(Category);
	FunctionEntryCreator.Finalize();

	return FunctionGraph;
}


UEdGraph* CreateIntermediateFunctionGraph(FKismetCompilerContext& InContext, FStringView InFunctionName, EFunctionFlags ExtraFunctionFlag, const FStringView Category, bool bIsEditable)
{
	UEdGraph* FunctionGraph = InContext.SpawnIntermediateFunctionGraph(InFunctionName.GetData());
	const UEdGraphSchema* Schema = FunctionGraph->GetSchema();
	Schema->CreateDefaultNodesForGraph(*FunctionGraph);

	// function entry node
	UK2Node_FunctionEntry* FunctionEntry = Private::FindFunctionEntry(FunctionGraph);
	check(FunctionEntry);
	{
		FunctionEntry->AddExtraFlags(ExtraFunctionFlag);
		FunctionEntry->bIsEditable = bIsEditable;
		FunctionEntry->MetaData.Category = FText::FromStringView(Category);

		// Add function input
		FunctionEntry->AllocateDefaultPins();
	}
	return FunctionGraph;
}


void SetVariableNodeMember(UK2Node_Variable* InVariableNode, const FProperty* InProperty, UBlueprint* InTargetBlueprint)
{
	if (InProperty->GetOwnerStruct() && InTargetBlueprint->GeneratedClass && InTargetBlueprint->GeneratedClass->IsChildOf(InProperty->GetOwnerStruct()))
	{
		FGuid Guid = FBlueprintEditorUtils::FindMemberVariableGuidByName(InTargetBlueprint, InProperty->GetFName());
		InVariableNode->VariableReference.SetSelfMember(InProperty->GetFName(), Guid);
	}
	else
	{
		UEdGraphSchema_K2::ConfigureVarNode(InVariableNode, InProperty->GetFName(), InProperty->GetOwnerStruct(), InTargetBlueprint);
	}
}

bool AddFunctionArgument(UEdGraph* InFunctionGraph, TSubclassOf<UObject> InArgument, FName InArgumentName)
{
	for (UEdGraphNode* Node : InFunctionGraph->Nodes)
	{
		if (UK2Node_FunctionEntry* FunctionEntry = Cast<UK2Node_FunctionEntry>(Node))
		{
			TSharedPtr<FUserPinInfo> PinInfo = MakeShared<FUserPinInfo>();
			PinInfo->PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
			PinInfo->PinType.PinSubCategoryObject = InArgument.Get();
			PinInfo->PinName = InArgumentName;
			PinInfo->DesiredPinDirection = EGPD_Output;
			FunctionEntry->UserDefinedPins.Add(PinInfo);

			FunctionEntry->ReconstructNode();
			return true;
		}
	}
	return false;
}


bool AddFunctionArgument(UEdGraph* InFunctionGraph, const FProperty* InArgument, FName InArgumentName)
{
	for (UEdGraphNode* Node : InFunctionGraph->Nodes)
	{
		if (UK2Node_FunctionEntry* FunctionEntry = Cast<UK2Node_FunctionEntry>(Node))
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

			TSharedPtr<FUserPinInfo> PinInfo = MakeShared<FUserPinInfo>();
			K2Schema->ConvertPropertyToPinType(InArgument, PinInfo->PinType);
			PinInfo->PinName = InArgumentName;
			PinInfo->DesiredPinDirection = EGPD_Output;
			FunctionEntry->UserDefinedPins.Add(PinInfo);

			FunctionEntry->ReconstructNode();
			return true;
		}
	}
	return false;
}


bool GenerateViewModelSetter(FKismetCompilerContext& InContext, UEdGraph* InFunctionGraph, FName InViewModelName)
{
	check(InFunctionGraph);

	UK2Node_FunctionEntry* FunctionEntry = Private::FindFunctionEntry(InFunctionGraph);
	if (FunctionEntry == nullptr)
	{
		return false;
	}

	const UEdGraphSchema* Schema = InFunctionGraph->GetSchema();
	if (Schema == nullptr)
	{
		return false;
	}

	bool bResult = true;
	UK2Node_CallFunction* CallSetViewModelNode = InContext.SpawnIntermediateNode<UK2Node_CallFunction>(FunctionEntry, InFunctionGraph);
	{
		CallSetViewModelNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UMVVMView, SetViewModel), UMVVMView::StaticClass());
		CallSetViewModelNode->AllocateDefaultPins();

		UEdGraphPin* NamePin = CallSetViewModelNode->FindPin(TEXT("ViewModelName"), EGPD_Input);
		bResult = bResult && NamePin != nullptr;
		if (ensure(NamePin))
		{
			bool bMarkAsModified = false;
			Schema->TrySetDefaultValue(*NamePin, InViewModelName.ToString(), bMarkAsModified);
		}
	}
	UK2Node_CallFunction* CallGetExtensionlNode = InContext.SpawnIntermediateNode<UK2Node_CallFunction>(FunctionEntry, InFunctionGraph);
	{
		CallGetExtensionlNode->FunctionReference.SetSelfMember(TEXT("GetExtension"));
		CallGetExtensionlNode->AllocateDefaultPins();

		UEdGraphPin* ExtensionTypePin = CallGetExtensionlNode->FindPin(TEXT("ExtensionType"), EGPD_Input);
		bResult = bResult && ExtensionTypePin != nullptr;
		if (ensure(ExtensionTypePin))
		{
			bool bMarkAsModified = false;
			Schema->TrySetDefaultObject(*ExtensionTypePin, UMVVMView::StaticClass(), bMarkAsModified);
		}
	}
	// Entry -> SetViewModel
	{
		UEdGraphPin* ThenPin = FunctionEntry->FindPin(UEdGraphSchema_K2::PN_Then, EGPD_Output);
		UEdGraphPin* EntryViewModelPin = FunctionEntry->FindPin(TEXT("ViewModel"), EGPD_Output);
		UEdGraphPin* ExecPin = CallSetViewModelNode->FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input);
		UEdGraphPin* SetViewModelPin = CallSetViewModelNode->FindPin(TEXT("ViewModel"), EGPD_Input);
		if (ensure(ThenPin && ExecPin))
		{
			ensure(Schema->TryCreateConnection(ThenPin, ExecPin));
		}
		if (ensure(EntryViewModelPin && SetViewModelPin))
		{
			ensure(Schema->TryCreateConnection(EntryViewModelPin, SetViewModelPin));
		}
		bResult = bResult && ThenPin && ExecPin && EntryViewModelPin && SetViewModelPin;
	}
	// GetExtension -> SetViewModel
	{
		UEdGraphPin* ResultPin = CallGetExtensionlNode->FindPin(UEdGraphSchema_K2::PN_ReturnValue, EGPD_Output);
		UEdGraphPin* ObjectPin = CallSetViewModelNode->FindPin(UEdGraphSchema_K2::PN_Self, EGPD_Input);
		if (ensure(ResultPin && ObjectPin))
		{
			ensure(Schema->TryCreateConnection(ResultPin, ObjectPin));
		}
		bResult = bResult && ResultPin && ObjectPin;
	}

	return bResult;
}


bool GenerateViewModelFieldNotifySetter(FKismetCompilerContext& InContext, UEdGraph* InFunctionGraph, FProperty* InProperty, FName InInputPinName)
{
	check(InFunctionGraph);

	UK2Node_FunctionEntry* FunctionEntry = Private::FindFunctionEntry(InFunctionGraph);
	if (FunctionEntry == nullptr)
	{
		return false;
	}

	const UEdGraphSchema* Schema = InFunctionGraph->GetSchema();
	if (Schema == nullptr)
	{
		return false;
	}

	UK2Node_VariableGet* OldValueGetNode = InContext.SpawnIntermediateNode<UK2Node_VariableGet>(FunctionEntry, InFunctionGraph);
	{
		bool bSelfContext = true;
		OldValueGetNode->VariableReference.SetFromField<FProperty>(InProperty, bSelfContext);
		OldValueGetNode->AllocateDefaultPins();
	}

	UK2Node_CallFunction* CallSetPropertyValue = InContext.SpawnIntermediateNode<UK2Node_CallFunction>(FunctionEntry, InFunctionGraph);
	{
		CallSetPropertyValue->FunctionReference.SetExternalMember("K2_SetPropertyValue", UMVVMViewModelBase::StaticClass());
		CallSetPropertyValue->AllocateDefaultPins();
	}

	bool bResult = true;
	// Entry -> SetProperty
	{
		UEdGraphPin* ThenPin = FunctionEntry->FindPin(UEdGraphSchema_K2::PN_Then, EGPD_Output);
		UEdGraphPin* ExecPin = CallSetPropertyValue->FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input);
		if (ensure(ThenPin && ExecPin))
		{
			ensure(Schema->TryCreateConnection(ThenPin, ExecPin));
		}
		bResult = bResult && ThenPin && ExecPin;
	}
	// OldValue -> SetProperty(OldValue)
	{
		UEdGraphPin* ViewModelValuePin = OldValueGetNode->FindPin(InProperty->GetFName(), EGPD_Output);
		UEdGraphPin* OldValuePin = CallSetPropertyValue->FindPin(TEXT("OldValue"), EGPD_Input);
		if (ensure(ViewModelValuePin && OldValuePin))
		{
			ensure(Schema->TryCreateConnection(ViewModelValuePin, OldValuePin));
		}
		bResult = bResult && ViewModelValuePin && OldValuePin;
	}
	// NewValue -> SetProperty(NewValue)
	{
		UEdGraphPin* EntryValuePin = FunctionEntry->FindPin(InInputPinName, EGPD_Output);
		UEdGraphPin* NewValuePin = CallSetPropertyValue->FindPin(TEXT("NewValue"), EGPD_Input);
		if (ensure(EntryValuePin && NewValuePin))
		{
			ensure(Schema->TryCreateConnection(EntryValuePin, NewValuePin));
		}
		bResult = bResult && EntryValuePin && NewValuePin;
	}

	return bResult;
}


bool GenerateViewModelFielNotifyBroadcast(FKismetCompilerContext& InContext, UEdGraph* InFunctionGraph, FProperty* InProperty)
{
	check(InFunctionGraph);

	UK2Node_FunctionEntry* FunctionEntry = Private::FindFunctionEntry(InFunctionGraph);
	if (FunctionEntry == nullptr)
	{
		return false;
	}

	const UEdGraphSchema* Schema = InFunctionGraph->GetSchema();
	if (Schema == nullptr)
	{
		return false;
	}

	UK2Node_CallFunction* CallBroadcast = InContext.SpawnIntermediateNode<UK2Node_CallFunction>(FunctionEntry, InFunctionGraph);
	{
		CallBroadcast->FunctionReference.SetExternalMember("K2_BroadcastFieldValueChanged", UMVVMViewModelBase::StaticClass());
		CallBroadcast->AllocateDefaultPins();

		UEdGraphPin* NamePin = CallBroadcast->FindPin(TEXT("FieldId"), EGPD_Input);
		ensure(NamePin);
		if (!NamePin)
		{
			return false;
		}

		FFieldNotificationId NewPropertyNotifyId;
		NewPropertyNotifyId.FieldName = InProperty->GetFName();

		FString ValueString;
		FFieldNotificationId::StaticStruct()->ExportText(ValueString, &NewPropertyNotifyId, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);

		bool bMarkAsModified = false;
		Schema->TrySetDefaultValue(*NamePin, ValueString, bMarkAsModified);
	}

	bool bResult = true;

	// Entry -> Broadcast
	{
		UEdGraphPin* ThenPin = FunctionEntry->FindPin(UEdGraphSchema_K2::PN_Then, EGPD_Output);
		UEdGraphPin* ExecPin = CallBroadcast->FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input);
		if (ensure(ThenPin && ExecPin))
		{
			ensure(Schema->TryCreateConnection(ThenPin, ExecPin));
		}
		bResult = bResult && ThenPin && ExecPin;
	}

	return bResult;
}

namespace Private
{

UK2Node_CallFunction* CreateFunctionNode(UEdGraph* InFunctionGraph, const UFunction* InFunction, UBlueprint* InBlueprint, FKismetCompilerContext* InContext, UK2Node_FunctionEntry* InFunctionEntry)
{
	if (InContext)
	{
		check(InFunctionEntry);
		UK2Node_CallFunction* FunctionNode = InContext->SpawnIntermediateNode<UK2Node_CallFunction>(InFunctionEntry, InFunctionGraph);
		FunctionNode->SetFromFunction(InFunction);
		FunctionNode->AllocateDefaultPins();
		return FunctionNode;
	}
	else
	{
		FGraphNodeCreator<UK2Node_CallFunction> RootCreator(*InFunctionGraph);
		UK2Node_CallFunction* FunctionNode = RootCreator.CreateNode();
		FunctionNode->NodePosX = 0;
		FunctionNode->NodePosY = 0;
		FunctionNode->SetFromFunction(InFunction);
		RootCreator.Finalize();
		return FunctionNode;
	}
}

template<typename NodeType>
NodeType* CreateVariableNode(UEdGraph* InFunctionGraph, const FProperty* InProperty, UBlueprint* InBlueprint, FKismetCompilerContext* InContext, UK2Node_FunctionEntry* InFunctionEntry)
{
	if (InContext)
	{
		check(InFunctionEntry);
		NodeType* VariableNode = InContext->SpawnIntermediateNode<NodeType>(InFunctionEntry, InFunctionGraph);
		SetVariableNodeMember(VariableNode, InProperty, InBlueprint);
		VariableNode->AllocateDefaultPins();
		return VariableNode;
	}
	else
	{
		FGraphNodeCreator<NodeType> RootCreator(*InFunctionGraph);
		NodeType* VariableNode = RootCreator.CreateNode();
		VariableNode->NodePosX = 0;
		VariableNode->NodePosY = 0;
		SetVariableNodeMember(VariableNode, InProperty, InBlueprint);
		RootCreator.Finalize();
		return VariableNode;
	}
}

bool GenerateSetter(UEdGraph* InFunctionGraph, TArrayView<UE::MVVM::FMVVMConstFieldVariant> InSetterPath, UBlueprint* InBlueprint, FKismetCompilerContext* InContext)
{
	check(InFunctionGraph);

	UK2Node_FunctionEntry* FunctionEntry = Private::FindFunctionEntry(InFunctionGraph);
	if (FunctionEntry == nullptr)
	{
		return false;
	}

	UEdGraphPin* FunctionInputPin = FunctionEntry->FindPinByPredicate([](UEdGraphPin* OtherPin)
		{
			return OtherPin->Direction == EGPD_Output
				&& OtherPin->PinName != UEdGraphSchema_K2::PN_Then;
		});

	const UEdGraphSchema* Schema = InFunctionGraph->GetSchema();
	if (Schema == nullptr)
	{
		return false;
	}

	UEdGraphPin* ThenPin = FunctionEntry->FindPin(UEdGraphSchema_K2::PN_Then, EGPD_Output);
	UEdGraphPin* OutContextPin = nullptr;

	bool bResult = true;
	for (int32 Index = 0; Index < InSetterPath.Num(); ++Index)
	{
		UE::MVVM::FMVVMConstFieldVariant& Field = InSetterPath[Index];
		const bool bLastPath = Index == InSetterPath.Num() - 1;

		UEdGraphPin* NextThenPin = nullptr;
		UEdGraphPin* NextOutContextPin = nullptr;

		UEdGraphPin* SetterPin = nullptr;
		UEdGraphPin* CurrentExecPin = nullptr;
		UEdGraphPin* CurrentInContextPin = nullptr;
		if (Field.IsProperty())
		{
			check(Field.GetProperty());

			if (Field.GetProperty()->GetOwnerClass() == nullptr)
			{
				ensureMsgf(false, TEXT("Property point to a structure member and that is not supported right now."));
				return false;
			}

			if (bLastPath)
			{
				UK2Node_VariableSet* VariableSetNode = CreateVariableNode<UK2Node_VariableSet>(InFunctionGraph, Field.GetProperty(), InBlueprint, InContext, FunctionEntry);

				NextThenPin = VariableSetNode->FindPin(UEdGraphSchema_K2::PN_Then, EGPD_Output);
				CurrentExecPin = VariableSetNode->FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input);
				CurrentInContextPin = VariableSetNode->FindPin(UEdGraphSchema_K2::PN_Self, EGPD_Input);
				SetterPin = VariableSetNode->FindPin(Field.GetProperty()->GetFName(), EGPD_Input);
			}
			else
			{
				UK2Node_VariableGet* VariableGetNode = CreateVariableNode<UK2Node_VariableGet>(InFunctionGraph, Field.GetProperty(), InBlueprint, InContext, FunctionEntry);

				CurrentInContextPin = VariableGetNode->FindPin(UEdGraphSchema_K2::PN_Self, EGPD_Input);
				NextOutContextPin = VariableGetNode->FindPin(Field.GetProperty()->GetFName(), EGPD_Output);
			}
		}
		else
		{
			ensure(Field.IsFunction());
			check(Field.GetFunction());

			UK2Node_CallFunction* CallFunctionNode = CreateFunctionNode(InFunctionGraph, Field.GetFunction(), InBlueprint, InContext, FunctionEntry);

			NextThenPin = CallFunctionNode->FindPin(UEdGraphSchema_K2::PN_Then, EGPD_Output);
			CurrentExecPin = CallFunctionNode->FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input);
			CurrentInContextPin = CallFunctionNode->FindPin(UEdGraphSchema_K2::PN_Self, EGPD_Input);

			if (bLastPath)
			{
				// find the input of the setter function
				SetterPin = CallFunctionNode->FindPinByPredicate([](UEdGraphPin* OtherPin)
					{
						return OtherPin->Direction == EGPD_Input
							&& OtherPin->PinName != UEdGraphSchema_K2::PN_Execute
							&& OtherPin->PinName != UEdGraphSchema_K2::PN_Self;
					});
			}
			else
			{
				// find the output of the getter function
				NextOutContextPin = CallFunctionNode->FindPinByPredicate([](UEdGraphPin* OtherPin)
					{
						return OtherPin->Direction == EGPD_Output
							&& OtherPin->PinName != UEdGraphSchema_K2::PN_Then;
					});
			}
		}

		if (ThenPin && CurrentExecPin)
		{
			ensure(Schema->TryCreateConnection(ThenPin, CurrentExecPin));
		}

		if (OutContextPin)
		{
			if (ensure(CurrentInContextPin))
			{
				ensure(Schema->TryCreateConnection(OutContextPin, CurrentInContextPin));
			}
		}

		if (bLastPath)
		{
			if (ensure(SetterPin && FunctionInputPin))
			{
				ensure(Schema->TryCreateConnection(FunctionInputPin, SetterPin));
			}
		}

		if (NextThenPin)
		{
			ThenPin = NextThenPin;
		}
		if (NextOutContextPin)
		{
			OutContextPin = NextOutContextPin;
		}
	}

	return true;
}
} //Private

bool GenerateIntermediateSetter(FKismetCompilerContext& InContext, UEdGraph* InFunctionGraph, TArrayView<UE::MVVM::FMVVMConstFieldVariant> InSetterPath)
{
	return Private::GenerateSetter(InFunctionGraph, InSetterPath, InContext.Blueprint, &InContext);
}

bool GenerateSetter(UBlueprint* InBlueprint, UEdGraph* InFunctionGraph, TArrayView<UE::MVVM::FMVVMConstFieldVariant> InSetterPath)
{
	return Private::GenerateSetter(InFunctionGraph, InSetterPath, InBlueprint, nullptr);
}

bool IsFunctionEntryMatchSignature(const UEdGraph* FunctionGraph, const UFunction* FunctionSignature)
{
	if (FunctionGraph == nullptr || FunctionSignature == nullptr)
	{
		return false;
	}

	const UK2Node_FunctionEntry* FunctionEntry = Private::FindFunctionEntry(FunctionGraph);
	if (FunctionEntry == nullptr)
	{
		return false;
	}

	// Generate pins list
	TArray<UEdGraphPin*> EntryPins = FunctionEntry->Pins;
	for (int32 Index = EntryPins.Num()-1; Index >= 0; --Index)
	{
		const UEdGraphPin* CurPin = EntryPins[Index];
		if (CurPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec
			|| CurPin->PinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self
			|| CurPin->Direction == EGPD_Input
			|| CurPin->ParentPin != nullptr)
		{
			EntryPins.RemoveAt(Index, 1);
		}
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Generate signature pins
	TArray<FUserPinInfo> SignaturePins;
	{
		for (TFieldIterator<FProperty> It(FunctionSignature); It; ++It)
		{
			FUserPinInfo& PinInfo = SignaturePins.AddDefaulted_GetRef();
			K2Schema->ConvertPropertyToPinType(*It, PinInfo.PinType);
			PinInfo.PinName = It->GetFName();
			PinInfo.DesiredPinDirection = EGPD_Output;
		}
	}

	// Do we have the same number of arguments
	if (SignaturePins.Num() != EntryPins.Num())
	{
		return false;
	}

	// Now check through the event's pins, and check for compatible pins, removing them if we find a match.
	for (int32 Index = 0; Index < SignaturePins.Num(); ++Index)
	{
		const UEdGraphPin* CurrentEventPin = EntryPins[Index];
		const FUserPinInfo& CurrentPinInfo = SignaturePins[Index];

		bool bMatchFound = false;
		if (CurrentEventPin->PinName != CurrentPinInfo.PinName)
		{
			return false;
		}

			// Check to make sure pins are of the same type
		if (!K2Schema->ArePinTypesCompatible(CurrentEventPin->PinType, CurrentPinInfo.PinType))
		{
			return false;
		}
	}
	return true;
}

void RenameObjectToTransientPackage(UObject* ObjectToRename)
{
	const ERenameFlags RenFlags = REN_DoNotDirty | REN_DontCreateRedirectors;

	ObjectToRename->SetFlags(RF_Transient);
	ObjectToRename->ClearFlags(RF_Public | RF_Standalone | RF_ArchetypeObject);

	// Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
	FLinkerLoad::InvalidateExport(ObjectToRename);
	ObjectToRename->Rename(nullptr, GetTransientPackage(), RenFlags);
}

} //namespace
