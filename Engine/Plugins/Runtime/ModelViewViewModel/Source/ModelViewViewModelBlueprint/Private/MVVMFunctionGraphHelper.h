// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Types/MVVMFieldVariant.h"

class FName;
class FProperty;
class UObject;
enum EFunctionFlags : uint32;
template <typename T> class TSubclassOf;

class FKismetCompilerContext;
class UBlueprint;
class UEdGraph;
class UK2Node_Variable;

namespace UE::MVVM::FunctionGraphHelper
{

	/** Generate a function graph that can be viewed in the BP editor. */
	MODELVIEWVIEWMODELBLUEPRINT_API UEdGraph* CreateFunctionGraph(UBlueprint* Blueprint, const FStringView FunctionName, EFunctionFlags ExtraFunctionFlag, const FStringView Category, bool bIsEditable);

	/** Generate a function graph that cannot be viewed in the BP editor. */
	UEdGraph* CreateIntermediateFunctionGraph(FKismetCompilerContext& Context, const FStringView FunctionName, EFunctionFlags ExtraFunctionFlag, const FStringView Category, bool bIsEditable);

	/** Set the UK2Node_Variable::VariableReference correctly base on the variable's owner. */
	MODELVIEWVIEWMODELBLUEPRINT_API void SetVariableNodeMember(UK2Node_Variable* InVarNode, const FProperty* InProperty, UBlueprint* InTargetBlueprint);

	/** Add an input argument to an existing function graph. */
	bool AddFunctionArgument(UEdGraph* FunctionGraph, TSubclassOf<UObject> Argument, FName ArgumentName);

	/** Add an input argument to an existing function graph. */
	bool AddFunctionArgument(UEdGraph* FunctionGraph, const FProperty* Argument, FName ArgumentName);

	/** Generate the nodes needed for to call the MVVMView::SetViewModel function. */
	bool GenerateViewModelSetter(FKismetCompilerContext& Context, UEdGraph* FunctionGraph, FName ViewModelName);

	/** Generate the nodes needed for to set the property and to call the broadcast field notify function. */
	bool GenerateViewModelFieldNotifySetter(FKismetCompilerContext& Context, UEdGraph* FunctionGraph, FProperty* Property, FName InputPinName);

	/** Generate the nodes needed to broadcast field notify function. */
	bool GenerateViewModelFielNotifyBroadcast(FKismetCompilerContext& Context, UEdGraph* FunctionGraph, FProperty* Property);

	/** Generate the nodes needed for to set a variable/function value. */
	bool GenerateSetter(UBlueprint* Blueprint, UEdGraph* FunctionGraph, TArrayView<UE::MVVM::FMVVMConstFieldVariant> SetterPath);

	/** Generate the nodes needed to set a variable/function value. To use while while compiling the Blueprint. */
	bool GenerateIntermediateSetter(FKismetCompilerContext& Context, UEdGraph* FunctionGraph, TArrayView<UE::MVVM::FMVVMConstFieldVariant> SetterPath);

	/** @return true when the entry node in the graph matches the function signature. */
	bool IsFunctionEntryMatchSignature(const UEdGraph* FunctionGraph, const UFunction* FunctionSignature);

	/** Moves the graph object to the transient package */
	void RenameObjectToTransientPackage(UObject* ObjectToRename);

} //namespace
