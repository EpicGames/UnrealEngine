// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMPropertyPath.h"
#include "BlueprintCompilationManager.h"
#include "Blueprint/WidgetTree.h"
#include "Engine/Blueprint.h"
#include "Extensions/UIComponentContainer.h"
#include "IPlatformFilePak.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Logging/StructuredLog.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewModelContext.h"
#include "MVVMMessageLog.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "Types/MVVMFieldVariant.h"
#include "UIComponentWidgetBlueprintExtension.h"
#include "WidgetBlueprint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMPropertyPath)

#define LOCTEXT_NAMESPACE "MVVMBlueprintFieldPath"

FMVVMBlueprintFieldPath::FMVVMBlueprintFieldPath(const UBlueprint* InContext, UE::MVVM::FMVVMConstFieldVariant InField)
{
	if (!InField.IsValid())
	{
		return;
	}

	UStruct* Owner = InField.GetOwner();
	ensure(Owner);
	if (!Owner)
	{
		return;
	}

	// Find the Guid and set the BindingKind
	UClass* OwnerClass = Cast<UClass>(Owner);
	FName FieldName = InField.GetName();
	FGuid MemberGuid;
	if (InField.IsProperty())
	{
		BindingKind = EBindingKind::Property;
		if (OwnerClass)
		{
			UBlueprint::GetGuidFromClassByFieldName<FProperty>(OwnerClass, FieldName, MemberGuid);
		}
		else if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(Owner))
		{
			MemberGuid = FStructureEditorUtils::GetGuidFromPropertyName(FieldName);
		}
	}
	else if (InField.IsFunction())
	{
		BindingKind = EBindingKind::Function;
		if (OwnerClass)
		{
			UBlueprint::GetGuidFromClassByFieldName<UFunction>(OwnerClass, FieldName, MemberGuid);
		}
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("Binding to field of unknown type!"));
	}

	// Set the member reference
	bool bIsSelf = (InContext->GeneratedClass && InContext->GeneratedClass->IsChildOf(Owner))
		|| (InContext->SkeletonGeneratedClass && InContext->SkeletonGeneratedClass->IsChildOf(Owner));
	if (bIsSelf)
	{
		BindingReference.SetSelfMember(FieldName, MemberGuid);
	}
	else if (OwnerClass)
	{
		FGuid Guid;
		if (UBlueprint* VariableOwnerBP = Cast<UBlueprint>(OwnerClass->ClassGeneratedBy))
		{
			OwnerClass = VariableOwnerBP->SkeletonGeneratedClass;
		}

		BindingReference.SetExternalMember(FieldName, OwnerClass, MemberGuid);
	}
	else if (UScriptStruct* OwnerStruct = Cast<UScriptStruct>(Owner))
	{
		struct FMyMemberReference : public FMemberReference
		{
			void SetExternalStructMember(FName InMemberName, UStruct* InMemberParentStruct, FGuid InGuid)
			{
				MemberName = InMemberName;
				MemberGuid = InGuid;
				MemberParent = InMemberParentStruct;
				MemberScope.Empty();
				bSelfContext = false;
				bWasDeprecated = false;
			}
		};
		static_cast<FMyMemberReference&>(BindingReference).SetExternalStructMember(FieldName, Owner, MemberGuid);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("Local member is not supported."));
	}
}


FName FMVVMBlueprintFieldPath::GetRawFieldName() const
{
	return BindingReference.GetMemberName();
}


FName FMVVMBlueprintFieldPath::GetFieldName(const UClass* InContext) const
{
	UE::MVVM::FMVVMConstFieldVariant Result = GetField(InContext);
	if (Result.IsValid())
	{
		return Result.GetName();
	}
	return FName();
}


UE::MVVM::FMVVMConstFieldVariant FMVVMBlueprintFieldPath::GetField(const UClass* InContext) const
{
	if (!FBlueprintCompilationManager::IsGeneratedClassLayoutReady())
	{
		if (UClass* SkeletonClass = FBlueprintEditorUtils::GetSkeletonClass(BindingReference.GetMemberParentClass(const_cast<UClass*>(InContext))))
		{
			return GetFieldInternal(SkeletonClass);
		}
	}
	return GetFieldInternal(InContext);
}


UE::MVVM::FMVVMConstFieldVariant FMVVMBlueprintFieldPath::GetFieldInternal(const UClass* InContext) const
{
	// Resolve any redirectors
	if (!BindingReference.GetMemberName().IsNone())
	{
		if (BindingKind == EBindingKind::Property)
		{
			struct FMyMemberReference : public FMemberReference
			{
				UStruct* GetMemberParent() const
				{
					return Cast<UStruct>(MemberParent);
				}
				void SetMemberName(FName InMemberName)
				{
					MemberName = InMemberName;
				}
			};
			UStruct* MemberParent = static_cast<const FMyMemberReference&>(BindingReference).GetMemberParent();
			if (UScriptStruct* OwnerStruct = Cast<UScriptStruct>(MemberParent))
			{
				const FProperty* FoundProperty = FindUFieldOrFProperty<FProperty>(OwnerStruct, BindingReference.GetMemberName(), EFieldIterationFlags::IncludeAll);
				if (!FoundProperty)
				{
					// Refresh the name from Guid
					if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(OwnerStruct))
					{
						FoundProperty = FStructureEditorUtils::GetPropertyByGuid(UserDefinedStruct, BindingReference.GetMemberGuid());
						if (FoundProperty)
						{
							const_cast<FMyMemberReference&>(static_cast<const FMyMemberReference&>(BindingReference)).SetMemberName(FoundProperty->GetFName());
						}
					}
				}

				return UE::MVVM::FMVVMConstFieldVariant(FoundProperty);
			}

			// Previous path. IsLocalScope was previously used to save struct properties.
			if (BindingReference.IsLocalScope())
			{
				if (UPackage* Package = BindingReference.GetMemberParentPackage())
				{
					UObjectBase* FoundObject = FindObjectWithOuter(Package, UScriptStruct::StaticClass(), *BindingReference.GetMemberScopeName());
					if (UScriptStruct* Struct = Cast<UScriptStruct>(static_cast<UObject*>(FoundObject)))
					{
						if (const FProperty* FoundProperty = FindUFieldOrFProperty<FProperty>(Struct, BindingReference.GetMemberName(), EFieldIterationFlags::IncludeAll))
						{
							return UE::MVVM::FMVVMConstFieldVariant(FoundProperty);
						}
					}
				}
			}

			return UE::MVVM::FMVVMConstFieldVariant(BindingReference.ResolveMember<FProperty>(const_cast<UClass*>(InContext), false));
		}
		else if (BindingKind == EBindingKind::Function)
		{
			return UE::MVVM::FMVVMConstFieldVariant(BindingReference.ResolveMember<UFunction>(const_cast<UClass*>(InContext), false));
		}
	}

	return UE::MVVM::FMVVMConstFieldVariant();
}


UClass* FMVVMBlueprintFieldPath::GetParentClass(const UClass* InSelfContext) const
{
	return BindingReference.GetMemberParentClass(const_cast<UClass*>(InSelfContext));
}


#if WITH_EDITOR
void FMVVMBlueprintFieldPath::SetDeprecatedBindingReference(const FMemberReference& InBindingReference, EBindingKind InBindingKind)
{
	BindingReference = InBindingReference;
	BindingKind = InBindingKind;
}


void FMVVMBlueprintFieldPath::SetDeprecatedSelfReference(const UBlueprint* InContext)
{
	struct FMyMemberReference : public FMemberReference
	{
		void SetSelfReference()
		{
			MemberParent = nullptr;
			bSelfContext = true;
			MemberScope.Empty();
		}
	};

	if (UClass* ParentClass = BindingReference.GetMemberParentClass())
	{
		ParentClass->ConditionalPostLoad();
		bool bIsSelf = (InContext->GeneratedClass && InContext->GeneratedClass->IsChildOf(ParentClass))
			|| (InContext->SkeletonGeneratedClass && InContext->SkeletonGeneratedClass->IsChildOf(ParentClass));
		if (bIsSelf)
		{
			FMyMemberReference& MyFieldPath = static_cast<FMyMemberReference&>(BindingReference);
			MyFieldPath.SetSelfReference();
		}
	}
}
#endif


TArray<FName> FMVVMBlueprintPropertyPath::GetFieldNames(const UClass* InSelfContext) const
{
	TArray<FName> Result;
	Result.Reserve(Paths.Num());

	for (const FMVVMBlueprintFieldPath& Path : Paths)
	{
		Result.Add(Path.GetFieldName(InSelfContext));
	}

	return Result;
}


TArray<UE::MVVM::FMVVMConstFieldVariant> FMVVMBlueprintPropertyPath::GetFields(const UClass* InSelfContext) const
{
	const UWidgetBlueprintGeneratedClass* WidgetBPG = Cast<UWidgetBlueprintGeneratedClass>(InSelfContext);
	if (bIsComponent && WidgetBPG)
	{
		return GetComponentFields(InSelfContext);
	}
	else
	{
		TArray<UE::MVVM::FMVVMConstFieldVariant> Result;
		Result.Reserve(Paths.Num());
		for (const FMVVMBlueprintFieldPath& Path : Paths)
		{
			Result.Add(Path.GetField(InSelfContext));
		}

		return Result;
	}
}


TArray<UE::MVVM::FMVVMConstFieldVariant> FMVVMBlueprintPropertyPath::GetCompleteFields(const UBlueprint* InSelfContext, bool bIsForDisplay) const
{
	TArray<UE::MVVM::FMVVMConstFieldVariant> Result;
	Result.Reserve(Paths.Num() + 1);

	UClass* ContextClass = InSelfContext->SkeletonGeneratedClass ? InSelfContext->SkeletonGeneratedClass : InSelfContext->GeneratedClass;
	switch (GetSource(InSelfContext))
	{
	case EMVVMBlueprintFieldPathSource::ViewModel:
	{
		Result.AddDefaulted();
		for (const TObjectPtr<UBlueprintExtension>& Extension : InSelfContext->GetExtensions())
		{
			if (Extension && Extension->GetClass() == UMVVMWidgetBlueprintExtension_View::StaticClass())
			{
				if (UMVVMBlueprintView* View = CastChecked<UMVVMWidgetBlueprintExtension_View>(Extension)->GetBlueprintView())
				{
					if (const FMVVMBlueprintViewModelContext* Viewmodel = View->FindViewModel(GetViewModelId()))
					{
						Result[0] = UE::MVVM::FMVVMConstFieldVariant(ContextClass->FindPropertyByName(Viewmodel->GetViewModelName()));
					}
				}
			}
		}
		break;
	}
	case EMVVMBlueprintFieldPathSource::SelfContext:
		break;
	case EMVVMBlueprintFieldPathSource::Widget:
	{	
		if (!bIsComponent || bIsForDisplay)
		{
			FProperty* WidgetProperty = ContextClass->FindPropertyByName(GetWidgetName());
			Result.Add(UE::MVVM::FMVVMConstFieldVariant(WidgetProperty));
		}
		break;
	}
	default:
		check(false);
		break;
	}

	if (bIsForDisplay)
	{
		for (const FMVVMBlueprintFieldPath& Path : Paths)
		{
			Result.Add(Path.GetField(ContextClass));
		}
	}
	else
	{
		Result.Append(GetFields(ContextClass));
	}

	return Result;
}

TArray<UE::MVVM::FMVVMConstFieldVariant> FMVVMBlueprintPropertyPath::GetComponentFields(const UClass* InSelfContext) const
{
	TArray<UE::MVVM::FMVVMConstFieldVariant> Result;

	if (!bIsComponent)
	{
		return Result;
	}

	const UWidgetBlueprint* WidgetBP = Cast<UWidgetBlueprint>(InSelfContext->ClassGeneratedBy);
	if (!WidgetBP)
	{
		return Result;
	}

	const FObjectPropertyBase* WidgetProperty = nullptr;
	if (GetSource(WidgetBP) == EMVVMBlueprintFieldPathSource::Widget)
	{
		if (UClass* GeneratedClass = WidgetBP->SkeletonGeneratedClass ? WidgetBP->SkeletonGeneratedClass : WidgetBP->GeneratedClass)
		{
			WidgetProperty = CastField<const FObjectPropertyBase>(GeneratedClass->FindPropertyByName(GetWidgetName()));
		}
	}

	const FObjectPropertyBase* OwningWidget = WidgetProperty;

	for (const FMVVMBlueprintFieldPath& Path : Paths)
	{
		const UE::MVVM::FMVVMConstFieldVariant Field = Path.GetField(InSelfContext);
		if (!Field.IsProperty())
		{
			// Pass functions through directly
			Result.Add(Field);
			continue;
		}
		if (const FProperty* PropertyPtr = Field.GetProperty())
		{
			const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(PropertyPtr);
			if (!ObjectProperty)
			{
				Result.Add(Field);
				continue;
			}

			if (ObjectProperty->PropertyClass->IsChildOf(UWidget::StaticClass()))
			{
				OwningWidget = ObjectProperty;
			}
			else if (ObjectProperty->PropertyClass->IsChildOf(UUIComponent::StaticClass()))
			{
				if (!OwningWidget)
				{
					Result.Add(Field);
					continue;
				}

				if (UWidgetBlueprintGeneratedClass* OwnerWidgetBlueprintGeneratedClass = Cast<UWidgetBlueprintGeneratedClass>(ObjectProperty->GetOwnerClass()))
				{
					UWidgetBlueprint* OwnerWidgetBlueprint = Cast<UWidgetBlueprint>(OwnerWidgetBlueprintGeneratedClass->ClassGeneratedBy);
					if (UUIComponentWidgetBlueprintExtension* ComponentWidgetBlueprintExtension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(OwnerWidgetBlueprint))
					{
						if (UWidget* Widget = OwnerWidgetBlueprint->WidgetTree->FindWidget(OwningWidget->GetFName()))
						{
							TArray<UUIComponent*> ComponentsList = ComponentWidgetBlueprintExtension->GetComponentsFor(Widget);
							for (UUIComponent* Component : ComponentsList)
							{
								const FName ComponentPropertyName = UUIComponentContainer::GetPropertyNameForComponent(Component, Widget->GetFName());
								if (Result.Num() > 0 && ObjectProperty->PropertyClass == Component->GetClass() && ObjectProperty->GetFName() == ComponentPropertyName)
								{
									Result.Pop();
									continue;
								}
							}
						}
					}	
				}
			}
		}					
		Result.Add(Field);		
	}

	return Result;
}

FString FMVVMBlueprintPropertyPath::GetPropertyPath(const UClass* InSelfContext) const
{
	TStringBuilder<512> Result;
	for (const FMVVMBlueprintFieldPath& Path : Paths)
	{
		if (Result.Len() > 0)
		{
			Result << TEXT('.');
		}
		Result << Path.GetFieldName(InSelfContext);
	}
	return Result.ToString();
}


bool FMVVMBlueprintPropertyPath::HasFieldInLocalScope() const
{
	for (const FMVVMBlueprintFieldPath& Path : Paths)
	{
		if (Path.IsFieldLocalScope())
		{
			return true;
		}
	}

	return false;
}


namespace UE::MVVM::Private
{
UMVVMBlueprintView* GetView(const UBlueprint* Blueprint)
{
	const TObjectPtr<UBlueprintExtension>* ExtensionViewPtr = Blueprint->GetExtensions().FindByPredicate([](const UBlueprintExtension* Other)
	{
		return Other && Other->GetClass() == UMVVMWidgetBlueprintExtension_View::StaticClass();
	});
	if (ExtensionViewPtr && IsValid(*ExtensionViewPtr))
	{
		return CastChecked<UMVVMWidgetBlueprintExtension_View>(*ExtensionViewPtr)->GetBlueprintView();
	}
	return nullptr;
}


FName GetViewModelName(const UBlueprint* Blueprint, FGuid Id)
{
	UMVVMBlueprintView* BlueprintView = GetView(Blueprint);
	const FMVVMBlueprintViewModelContext* ViewModel = BlueprintView ? BlueprintView->FindViewModel(Id) : nullptr;
	return ViewModel ? ViewModel->GetViewModelName() : NAME_None;
}


FText GetWidgetDisplayName(const UWidgetBlueprint* WidgetBlueprint, FName WidgetName, bool bUseDisplayName, bool bIncludeMetaData)
{
	if (!bUseDisplayName && !bIncludeMetaData)
	{
		return FText::FromName(WidgetName);
	}

	UWidgetTree* WidgetTree = WidgetBlueprint ? WidgetBlueprint->WidgetTree.Get() : nullptr;
	UWidget* FoundWidget = WidgetTree ? WidgetTree->FindWidget(WidgetName) : nullptr;
	if (bIncludeMetaData)
	{
		return FoundWidget ? FoundWidget->GetLabelTextWithMetadata() : FText::FromName(WidgetName);
	}
	return FoundWidget ? FoundWidget->GetLabelText() : FText::FromName(WidgetName);
}


FText GetViewModelDisplayName(const UWidgetBlueprint* WidgetBlueprint, FGuid Id, bool bUseDisplayName)
{
	UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	UMVVMBlueprintView* BlueprintView = ExtensionView ? ExtensionView->GetBlueprintView() : nullptr;
	const FMVVMBlueprintViewModelContext* ViewModel = BlueprintView ? BlueprintView->FindViewModel(Id) : nullptr;
	if (bUseDisplayName)
	{
		return ViewModel ? ViewModel->GetDisplayName() : LOCTEXT("None", "<None>");
	}
	return ViewModel ? FText::FromName(ViewModel->GetViewModelName()) : LOCTEXT("None", "<None>");
}


FText GetRootName(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& PropertyPath, bool bUseDisplayName, bool bIncludeMetaData)
{
	switch (PropertyPath.GetSource(WidgetBlueprint))
	{
	case EMVVMBlueprintFieldPathSource::SelfContext:
		return bUseDisplayName ? LOCTEXT("Self", "Self") : FText::FromString(WidgetBlueprint->GetFriendlyName());

	case EMVVMBlueprintFieldPathSource::ViewModel:
		return GetViewModelDisplayName(WidgetBlueprint, PropertyPath.GetViewModelId(), bUseDisplayName);

	case EMVVMBlueprintFieldPathSource::Widget:
		return GetWidgetDisplayName(WidgetBlueprint, PropertyPath.GetWidgetName(), bUseDisplayName, bIncludeMetaData);
	}

	return FText::GetEmpty();
}
	
const UStruct* GetMostUpToDateStruct(const UStruct* Struct)
{
#if WITH_EDITOR
	if (const UClass* Class = Cast<UClass>(Struct))
	{
		return FBlueprintEditorUtils::GetMostUpToDateClass(Class);
	}
	return Struct;
#else
	return Struct;
#endif
}
}//namespace


bool FMVVMBlueprintPropertyPath::OnFieldRenamed(const UBlueprint* InContext, UClass* FieldOwnerClass, FName OldObjectName, FName NewObjectName)
{
	bool bFoundRenamedField = false;
	if (InContext != nullptr)
	{
		const EMVVMBlueprintFieldPathSource PathSource = GetSource(InContext);
		const UClass* SelfContext = InContext->GeneratedClass;
		
		if (SelfContext == FieldOwnerClass)
		{
			if (PathSource == EMVVMBlueprintFieldPathSource::Widget && GetWidgetName() == OldObjectName)
			{
				SetWidgetName(NewObjectName);
				bFoundRenamedField = true;
			}
			// Checking against NewObjectName since the viewmodel would have already updated its name
			else if (PathSource == EMVVMBlueprintFieldPathSource::ViewModel && UE::MVVM::Private::GetViewModelName(InContext, GetViewModelId()) == NewObjectName)
			{
				// Nothing to rename since we reference the VM by ID, but the path does contain the renamed field
				bFoundRenamedField = true;
			}
		}

		const UStruct* OwnerStruct = nullptr;

		if (PathSource == EMVVMBlueprintFieldPathSource::SelfContext)
		{
			OwnerStruct = SelfContext;
		}
		else if (PathSource == EMVVMBlueprintFieldPathSource::ViewModel || PathSource == EMVVMBlueprintFieldPathSource::Widget)
		{
			const FName OwnerName = (PathSource == EMVVMBlueprintFieldPathSource::ViewModel) ? UE::MVVM::Private::GetViewModelName(InContext, GetViewModelId()) : GetWidgetName();
			const FProperty* OwnerProperty = FBlueprintEditorUtils::GetMostUpToDateClass(SelfContext)->FindPropertyByName(OwnerName);
			if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(OwnerProperty))
			{
				OwnerStruct = ObjectProperty->PropertyClass;
			}
			else if (const FStructProperty* StructProperty = CastField<FStructProperty>(OwnerProperty))
			{
				OwnerStruct = StructProperty->Struct;
			}
		}

		// Check the path components for the renamed field
		for (FMVVMBlueprintFieldPath& Path : Paths)
		{
			if (bFoundRenamedField && !bIsComponent)
			{
				if (!Path.IsFieldSelfContext() && OwnerStruct)
				{
					FName FieldName = Path.GetFieldName(nullptr);
					// We need to update the stored parent in the path in case the parent changed type
					FProperty* UpdatedProperty = OwnerStruct->FindPropertyByName(FieldName);
					if (UpdatedProperty)
					{
						Path = { InContext, UE::MVVM::FMVVMConstFieldVariant(UpdatedProperty) };
					}
					else if (const UClass* OwnerClass = Cast<UClass>(OwnerStruct))
					{
						if (UFunction* UpdatedFunction = OwnerClass->FindFunctionByName(FieldName))
						{
							Path = { InContext, UE::MVVM::FMVVMConstFieldVariant(UpdatedFunction) };
						}
					}
					else
					{
						UE_LOGFMT(LogMVVM, Error, "OnFieldRenamed: Field name '{Field}' does not exist in class '{Struct}'. Binding won't compile.",
							("Field", FieldName.ToString()),
							("Struct", OwnerStruct->GetName()));
					}
				}
				
				break;
			}

			if (OwnerStruct)
			{
				const UStruct* UpToDateStruct = UE::MVVM::Private::GetMostUpToDateStruct(OwnerStruct);
				const FProperty* OwnerProperty = UpToDateStruct->FindPropertyByName(Path.GetFieldName(SelfContext));
				if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(OwnerProperty))
				{
					OwnerStruct = ObjectProperty->PropertyClass;
				}
				else if (const FStructProperty* StructProperty = CastField<FStructProperty>(OwnerProperty))
				{
					OwnerStruct = StructProperty->Struct;
				}
			}
			
			if (Path.GetParentClass(SelfContext) == FieldOwnerClass && Path.GetFieldName(SelfContext) == OldObjectName)
			{
				Path.SetMemberName(NewObjectName);
				bFoundRenamedField = true;
			}

			if (bIsComponent)
			{
				const UWidgetBlueprintGeneratedClass* WidgetBPG = Cast<const UWidgetBlueprintGeneratedClass>(OwnerStruct);
				const UWidgetBlueprint* WidgetBP = WidgetBPG ? Cast<UWidgetBlueprint>(WidgetBPG->ClassGeneratedBy) : Cast<UWidgetBlueprint>(InContext);
				if (!WidgetBPG)
				{
					WidgetBPG = Cast<const UWidgetBlueprintGeneratedClass>(SelfContext);
				}
				const FProperty* ComponentProperty = WidgetBPG->FindPropertyByName(Path.GetFieldName(SelfContext));
				if (const FObjectPropertyBase* ComponentObjectProperty = CastField<FObjectPropertyBase>(ComponentProperty))
				{
					if (ComponentObjectProperty->PropertyClass->IsChildOf(UUIComponent::StaticClass()))
					{ 
						if (UUIComponentWidgetBlueprintExtension* ComponentWidgetBlueprintExtension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBP))
						{
							if (UUIComponent* FoundComponent = ComponentWidgetBlueprintExtension->GetComponent(ComponentObjectProperty->PropertyClass, NewObjectName))
							{
								const FName ComponentPropertyName = UUIComponentContainer::GetPropertyNameForComponent(FoundComponent, NewObjectName);
								Path.SetMemberName(ComponentPropertyName);
								break;
							}
						}
					}
				}
			}
		}
	}

	return bFoundRenamedField;
}

FText FMVVMBlueprintPropertyPath::ToText(const UWidgetBlueprint* WidgetBlueprint, bool bUseDisplayName) const
{
	if (WidgetBlueprint == nullptr)
	{
		return FText::GetEmpty();
	}

	auto GetDisplayNameForField = [](const UE::MVVM::FMVVMConstFieldVariant& Field) -> FText
	{
		if (!Field.IsEmpty())
		{
			if (Field.IsProperty())
			{
				return Field.GetProperty()->GetDisplayNameText();
			}
			else if (Field.IsFunction())
			{
				return Field.GetFunction()->GetDisplayNameText();
			}
		}
		return LOCTEXT("None", "<None>");
	};

	TArray<FText> JoinArgs;
	JoinArgs.Add(UE::MVVM::Private::GetRootName(WidgetBlueprint, *this, bUseDisplayName, false));
	TArray<UE::MVVM::FMVVMConstFieldVariant> Fields = GetFields(WidgetBlueprint->SkeletonGeneratedClass);
	for (const UE::MVVM::FMVVMConstFieldVariant& Field : Fields)
	{
		JoinArgs.Add(bUseDisplayName ? GetDisplayNameForField(Field) : FText::FromName(Field.GetName()));
	}
	return FText::Join(LOCTEXT("PathDelimiter", "."), JoinArgs);
}


FString FMVVMBlueprintPropertyPath::ToString(const UWidgetBlueprint* WidgetBlueprint, bool bUseDisplayName, bool bIncludeMetaData) const
{
	if (WidgetBlueprint == nullptr)
	{
		return FString();
	}

	TStringBuilder<256> Builder;
	Builder << UE::MVVM::Private::GetRootName(WidgetBlueprint, *this, bUseDisplayName, bIncludeMetaData).ToString();

	TArray<UE::MVVM::FMVVMConstFieldVariant> Fields = GetFields(WidgetBlueprint->SkeletonGeneratedClass);
	for (const UE::MVVM::FMVVMConstFieldVariant& Field : Fields)
	{
		Builder << '.';
		if (!Field.IsEmpty())
		{
			if (Field.IsProperty())
			{
				if (bUseDisplayName)
				{
					Builder << Field.GetProperty()->GetDisplayNameText().ToString();
				}
				else
				{
					Builder << Field.GetProperty()->GetFName();
				}
			}
			else if (Field.IsFunction())
			{
				if (bUseDisplayName)
				{
					Builder << Field.GetFunction()->GetDisplayNameText().ToString();
				}
				else
				{
					Builder << Field.GetFunction()->GetFName();
				}
				if (bIncludeMetaData)
				{
					const FString& FunctionKeywords = Field.GetFunction()->GetMetaData(FBlueprintMetadata::MD_FunctionKeywords);
					if (!FunctionKeywords.IsEmpty())
					{
						Builder << TEXT(".");
						Builder << FunctionKeywords;
					}
				}
			}
		}
	}
	return Builder.ToString();
}

void FMVVMBlueprintPropertyPath::DeprecationUpdateSource(const UBlueprint* InContext)
{
	if (ContextId.IsValid())
	{
		Source = EMVVMBlueprintFieldPathSource::ViewModel;
	}
	else if (!WidgetName.IsNone())
	{
		Source = (InContext && InContext->GetFName() == WidgetName) ? EMVVMBlueprintFieldPathSource::SelfContext : EMVVMBlueprintFieldPathSource::Widget;
	}
	bDeprecatedSource = true;
}

#undef LOCTEXT_NAMESPACE
