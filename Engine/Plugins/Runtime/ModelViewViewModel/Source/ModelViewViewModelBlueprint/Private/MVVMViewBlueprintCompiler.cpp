// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMViewBlueprintCompiler.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Bindings/MVVMConversionFunctionHelper.h"
#include "Bindings/MVVMFieldPathHelper.h"
#include "BlueprintEditorSettings.h"
#include "Blueprint/WidgetTree.h"
#include "Components/Widget.h"
#include "Extensions/MVVMBlueprintViewExtension.h"
#include "Extensions/UIComponent.h"
#include "HAL/IConsoleManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_ComponentBoundEvent.h"
#include "Misc/NamePermissionList.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMMessageLog.h"
#include "PropertyPermissionList.h"
#include "MVVMFunctionGraphHelper.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "Node/MVVMK2Node_AreSourcesValidForBinding.h"
#include "Templates/ValueOrError.h"
#include "Types/MVVMBindingName.h"
#include "UObject/LinkerLoad.h"
#include "UObject/UnrealType.h"
#include "View/MVVMViewClass.h"
#include "View/MVVMViewModelContextResolver.h"
#include "Algo/AnyOf.h"
#include "WidgetBlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "MVVMViewBlueprintCompiler"

namespace UE::MVVM::Compiler
{
int32 FCompilerBindingHandle::IdGenerator = 0;
}

/**
When compiling the skeletal class
	CreateVariables()
		Add the less amount of errors here to have a more responsive editor and have the variable available in the editor.
		Create all the variables
		Create the public functions
When compiling the full class
	CreateVariables()
		same as skeletal, it may compile the skeletal then the full class. This will be called twice with the same instance.
	CreateFunctions()
		Create the private functions for bindings and events.
		Note The class is not linked
Once the full class is compiled, then add the bindings/events with
	PreCompile()
		Create the bindings and events data and send them to the library compiler.
	Compile()
		All bindings and events data are compiled. Create the view data with the compiled data.
 */

namespace UE::MVVM::Private
{
FAutoConsoleVariable CVarLogViewCompiledResult(
	TEXT("MVVM.LogViewCompiledResult"),
	false,
	TEXT("After the view is compiled log the compiled bindings and sources.")
);

FAutoConsoleVariable CVarLogInvalidBindingVerbose(
	TEXT("MVVM.LogInvalidBindingVerbose"),
	true,
	TEXT("If a binding is invalid. Log verbose amounts of various different content data.")
);

#if UE_BUILD_DEVELOPMENT || UE_BUILD_DEBUG
FAutoConsoleCommand CVarTestGenerateSetter(
	TEXT("MVVM.TestGenerateSetter"),
	TEXT("Generate a setter function base on the input string."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() < 3)
		{
			return;
		}
		FMVVMViewBlueprintCompiler::TestGenerateSetter(nullptr, Args[0], Args[1], Args[2]);
	})
);
#endif

static const FText CouldNotCreateSourceFieldPathFormat = LOCTEXT("CouldNotCreateSourceFieldPath", "Couldn't create the source field path '{0}'. {1}");
static const FText CouldNotCreateDestinationFieldPathFormat = LOCTEXT("CouldNotCreateDestinationFieldPath", "Couldn't create the destination field path '{0}'. {1}");
static const FText PropertyPathIsInvalidFormat = LOCTEXT("PropertyPathIsInvalid", "The property path '{0}' is invalid.");

FString PropertyPathToString(const UClass* InSelfContext, const UMVVMBlueprintView* BlueprintView, const FMVVMBlueprintPropertyPath& PropertyPath)
{
	if (!PropertyPath.IsValid())
	{
		return FString();
	}

	TStringBuilder<512> Result;
	switch (PropertyPath.GetSource(BlueprintView->GetOuterUMVVMWidgetBlueprintExtension_View()->GetOuterUWidgetBlueprint()))
	{
	case EMVVMBlueprintFieldPathSource::SelfContext:
		Result << InSelfContext->ClassGeneratedBy->GetName();
		break;
	case EMVVMBlueprintFieldPathSource::Widget:
		Result << PropertyPath.GetWidgetName();
		break;
	case EMVVMBlueprintFieldPathSource::ViewModel:
		if (const FMVVMBlueprintViewModelContext* SourceViewModelContext = BlueprintView->FindViewModel(PropertyPath.GetViewModelId()))
		{
			Result << SourceViewModelContext->GetViewModelName();
		}
		else
		{
			Result << TEXT("<none>");
		}
		break;
	default:
		Result << TEXT("<none>");
		break;
	}

	FString BasePropertyPath = PropertyPath.GetPropertyPath(InSelfContext);
	if (BasePropertyPath.Len())
	{
		Result << TEXT('.');
		Result << MoveTemp(BasePropertyPath);
	}
	return Result.ToString();
}

FText PropertyPathToText(const UClass* InSelfContext, const UMVVMBlueprintView* BlueprintView, const FMVVMBlueprintPropertyPath& PropertyPath)
{
	return FText::FromString(PropertyPathToString(InSelfContext, BlueprintView, PropertyPath));
}

FText GetViewModelIdText(FGuid Id)
{
	return FText::FromString(Id.ToString(EGuidFormats::DigitsWithHyphensInBraces));
}

TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> AddObjectFieldPath(FCompiledBindingLibraryCompiler& BindingLibraryCompiler, const UWidgetBlueprintGeneratedClass* Class, FStringView ObjectPath, const UClass* ExpectedType)
{
	// Generate a path to read the value at runtime
	static const FText InvalidGetterFormat = LOCTEXT("ViewModelInvalidGetterWithReason", "Viewmodel has an invalid Getter. {0}");

	TValueOrError<TArray<FMVVMConstFieldVariant>, FText> GeneratedField = FieldPathHelper::GenerateFieldPathList(Class, ObjectPath, true);
	if (GeneratedField.HasError())
	{
		return MakeError(FText::Format(InvalidGetterFormat, GeneratedField.GetError()));
	}

	TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> ReadFieldPathResult = BindingLibraryCompiler.AddObjectFieldPath(GeneratedField.GetValue(), ExpectedType, true);
	if (ReadFieldPathResult.HasError())
	{
		return MakeError(FText::Format(InvalidGetterFormat, ReadFieldPathResult.GetError()));
	}

	return MakeValue(ReadFieldPathResult.StealValue());
}


FMVVMViewBlueprintCompiler::FMVVMViewBlueprintCompiler(FWidgetBlueprintCompilerContext& InCreationContext, UMVVMBlueprintView* InBlueprintView)
	: WidgetBlueprintCompilerContext(InCreationContext)
	, BlueprintView(InBlueprintView)
	, BindingLibraryCompiler(InCreationContext.WidgetBlueprint())
{
	check(BlueprintView);
}


void FMVVMViewBlueprintCompiler::AddMessage(const FText& MessageText, Compiler::EMessageType MessageType) const
{
	switch (MessageType)
	{
	case Compiler::EMessageType::Info:
		WidgetBlueprintCompilerContext.MessageLog.Note(*MessageText.ToString());
		break;
	case Compiler::EMessageType::Warning:
		WidgetBlueprintCompilerContext.MessageLog.Warning(*MessageText.ToString());
		break;
	case Compiler::EMessageType::Error:
		WidgetBlueprintCompilerContext.MessageLog.Error(*MessageText.ToString());
		break;
	default:
		break;
	}
}


void FMVVMViewBlueprintCompiler::AddMessages(TArrayView<TWeakPtr<FCompilerBinding>> Bindings, TArrayView<TWeakPtr<FCompilerEvent>> Events, const FText& MessageText, Compiler::EMessageType MessageType) const
{
	for (TWeakPtr<FCompilerBinding>& Binding : Bindings)
	{
		AddMessageForBinding(Binding.Pin(), MessageText, MessageType, FMVVMBlueprintPinId());
	}
	for (TWeakPtr<FCompilerEvent>& Event : Events)
	{
		AddMessageForEvent(Event.Pin(), MessageText, MessageType, FMVVMBlueprintPinId());
	}

	if (Bindings.Num() == 0 && Events.Num() == 0)
	{
		AddMessage(MessageText, MessageType);
	}
}


void FMVVMViewBlueprintCompiler::AddMessageForBinding(const TSharedPtr<FCompilerBinding>& Binding, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const
{
	const FMVVMBlueprintViewBinding* BindingPtr = Binding ? BlueprintView->GetBindingAt(Binding->Key.ViewBindingIndex) : nullptr;
	if (BindingPtr)
	{
		AddMessageForBinding(*BindingPtr, MessageText, MessageType, PinId);
	}
	else
	{
		AddMessage(MessageText, MessageType);
	}
}


FText GetJoinArgumentNames(const FMVVMBlueprintPinId& PinId)
{
	TArray<FText> JoinedNames;
	JoinedNames.Reserve(PinId.GetNames().Num());
	for (FName Name : PinId.GetNames())
	{
		JoinedNames.Add(FText::FromName(Name));
	}
	return FText::Join(LOCTEXT("NameSeperator", "."), JoinedNames);
}


void FMVVMViewBlueprintCompiler::AddMessageForBinding(const FMVVMBlueprintViewBinding& Binding, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const
{
	const FText BindingName = FText::FromString(Binding.GetDisplayNameString(WidgetBlueprintCompilerContext.WidgetBlueprint()));

	FText FormattedError;
	if (PinId.IsValid())
	{
		FormattedError = FText::Format(LOCTEXT("BindingFormatWithArgument", "Binding '{0}': Argument '{1}' - {2}"), BindingName, UE::MVVM::Private::GetJoinArgumentNames(PinId), MessageText);
	}
	else
	{
		FormattedError = FText::Format(LOCTEXT("BindingFormat", "Binding '{0}': {1}"), BindingName, MessageText);
	}
	AddMessage(FormattedError, MessageType);

	static EBindingMessageType BindingMessageTypes[] = { EBindingMessageType ::Info, EBindingMessageType ::Warning, EBindingMessageType ::Error};
	FBindingMessage NewMessage = { FormattedError, BindingMessageTypes[static_cast<int32>(MessageType)] };
	BlueprintView->AddMessageToBinding(Binding.BindingId, NewMessage);
}


void FMVVMViewBlueprintCompiler::AddMessageForEvent(const TSharedPtr<FCompilerEvent>& Event, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const
{
	const UMVVMBlueprintViewEvent* EventPtr = Event ? Event->Event.Get() : nullptr;
	if (EventPtr)
	{
		AddMessageForEvent(EventPtr, MessageText, MessageType, PinId);
	}
	else
	{
		AddMessage(MessageText, MessageType);
	}
}


void FMVVMViewBlueprintCompiler::AddMessageForEvent(const UMVVMBlueprintViewEvent* Event, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const
{
	const FText EventName = Event->GetDisplayName(true);

	FText FormattedError;
	if (PinId.IsValid())
	{
		FormattedError = FText::Format(LOCTEXT("EventFormatWithArgument", "Event '{0}': Argument '{1}' - {2}"), EventName, UE::MVVM::Private::GetJoinArgumentNames(PinId), MessageText);
	}
	else
	{
		FormattedError = FText::Format(LOCTEXT("EventFormat", "Event '{0}': {1}"), EventName, MessageText);
	}
	AddMessage(FormattedError, MessageType);

	static UMVVMBlueprintViewEvent::EMessageType BindingMessageTypes[] = { UMVVMBlueprintViewEvent::EMessageType::Info, UMVVMBlueprintViewEvent::EMessageType::Warning, UMVVMBlueprintViewEvent::EMessageType::Error };
	UMVVMBlueprintViewEvent::FMessage NewMessage = { FormattedError, BindingMessageTypes[static_cast<int32>(MessageType)] };
	Event->AddCompilationToBinding(NewMessage);
}

void FMVVMViewBlueprintCompiler::AddMessageForCondition(const TSharedPtr<FCompilerCondition>& Condition, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const
{
	const UMVVMBlueprintViewCondition* ConditionPtr = Condition ? Condition->Condition.Get() : nullptr;
	if (ConditionPtr)
	{
		AddMessageForCondition(ConditionPtr, MessageText, MessageType, PinId);
	}
	else
	{
		AddMessage(MessageText, MessageType);
	}
}

void FMVVMViewBlueprintCompiler::AddMessageForCondition(const UMVVMBlueprintViewCondition* Condition, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const
{
	const FText ConditionName = Condition->GetDisplayName(true);

	FText FormattedError;
	if (PinId.IsValid())
	{
		FormattedError = FText::Format(LOCTEXT("ConditionFormatWithArgument", "Condition '{0}': Argument '{1}' - {2}"), ConditionName, UE::MVVM::Private::GetJoinArgumentNames(PinId), MessageText);
	}
	else
	{
		FormattedError = FText::Format(LOCTEXT("ConditionFormat", "Condition '{0}': {1}"), ConditionName, MessageText);
	}
	AddMessage(FormattedError, MessageType);

	static UMVVMBlueprintViewCondition::EMessageType BindingMessageTypes[] = { UMVVMBlueprintViewCondition::EMessageType::Info, UMVVMBlueprintViewCondition::EMessageType::Warning, UMVVMBlueprintViewCondition::EMessageType::Error };
	UMVVMBlueprintViewCondition::FMessage NewMessage = { FormattedError, BindingMessageTypes[static_cast<int32>(MessageType)] };
	Condition->AddCompilationToBinding(NewMessage);
}


void FMVVMViewBlueprintCompiler::AddMessageForViewModel(const FMVVMBlueprintViewModelContext& ViewModel, const FText& Message, Compiler::EMessageType MessageType) const
{
	const FText FormattedError = FText::Format(LOCTEXT("ViewModelFormat", "Viewodel '{0}': {1}"), ViewModel.GetDisplayName(), Message);
	AddMessage(FormattedError, MessageType);
}


void FMVVMViewBlueprintCompiler::AddMessageForViewModel(const FText& ViewModelDisplayName, const FText& Message, Compiler::EMessageType MessageType) const
{
	const FText FormattedError = FText::Format(LOCTEXT("ViewModelFormat", "Viewodel '{0}': {1}"), ViewModelDisplayName, Message);
	AddMessage(FormattedError, MessageType);
}


void FMVVMViewBlueprintCompiler::AddExtension(UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension)
{
	WidgetBlueprintCompilerContext.AddExtension(Class, ViewExtension);
}


void FMVVMViewBlueprintCompiler::CleanOldData(UWidgetBlueprintGeneratedClass* ClassToClean, UObject* OldCDO)
{
	// Clean old View
	if (!WidgetBlueprintCompilerContext.Blueprint->bIsRegeneratingOnLoad && WidgetBlueprintCompilerContext.bIsFullCompile)
	{
		TArray<UObject*> Children;
		ForEachObjectWithOuter(ClassToClean, [&Children](UObject* Child)
			{
				if (Cast<UMVVMViewClass>(Child))
				{
					Children.Add(Child);
				}
			}, EGetObjectsFlags::None);

		for (UObject* Child : Children)
		{
			FunctionGraphHelper::RenameObjectToTransientPackage(Child);
		}

		// Clean up the stale viewmodel instances.
		TMap<FGuid, TWeakObjectPtr<UObject>>& SavedViewModelInstances = BlueprintView->GetOuterUMVVMWidgetBlueprintExtension_View()->TemporaryViewModelInstances;
		TArray<FGuid> DeleteViewModelInstances;

		for (auto It = SavedViewModelInstances.CreateIterator(); It; ++It)
		{
			uint32 ViewModelIndex = ViewModelCreatorContexts.IndexOfByPredicate([It](const FCompilerViewModelCreatorContext& SourceCreatorContext)
				{
					return SourceCreatorContext.ViewModelContext.GetViewModelId() == It->Key;
				});
			
			TWeakObjectPtr<UObject> StaticViewModelInstance = nullptr;
			if (ViewModelIndex != INDEX_NONE)
			{
				const FMVVMBlueprintViewModelContext& ViewModelContext = ViewModelCreatorContexts[ViewModelIndex].ViewModelContext;
				if (It->Value.IsValid())
				{
					if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::CreateInstance
						&& ViewModelContext.bExposeInstanceInEditor)
					{
						StaticViewModelInstance = It->Value;
					}
					else
					{
						FunctionGraphHelper::RenameObjectToTransientPackage(It->Value.Get());
					}
				}
			}
			if (!StaticViewModelInstance.IsValid())
			{
				DeleteViewModelInstances.Add(It->Key);
				SavedViewModelInstances[It->Key] = nullptr;
			}
		}

		for (FGuid Guid : DeleteViewModelInstances)
		{
			SavedViewModelInstances.Remove(Guid);
		}
	}
}


void FMVVMViewBlueprintCompiler::GatherGeneratedVariables(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context)
{
	if (!BlueprintView)
	{
		return;
	}

	if (!AreStepsValid())
	{
		return;
	}
	
	CreateWidgetMap(Context);
	CreateBindingList(Context);
	CreateEventList(Context);
	CreateConditionList(Context);
	CreateExtensionList(Context);
	CreateRequiredProperties(Context);

	// Add Generated Variables
	{
		auto AddObjectVariable = [&Context](const FCompilerUserWidgetProperty& UserWidgetProperty)
		{
			FBPVariableDescription ObjectVariableDesc;
			ObjectVariableDesc.VarName = UserWidgetProperty.Name;
			ObjectVariableDesc.VarGuid = UserWidgetProperty.Guid;
			ObjectVariableDesc.VarType = FEdGraphPinType(UEdGraphSchema_K2::PC_Object, NAME_None, UserWidgetProperty.AuthoritativeClass, EPinContainerType::None, false, FEdGraphTerminalType());
			ObjectVariableDesc.FriendlyName = UserWidgetProperty.DisplayName.ToString();
			ObjectVariableDesc.PropertyFlags = (CPF_BlueprintVisible | CPF_RepSkip);			

			const bool bIsInstanceExposed = UserWidgetProperty.bInstanced && UserWidgetProperty.bInstanceExposed;
			if (bIsInstanceExposed)
			{
				ObjectVariableDesc.PropertyFlags |= (CPF_Edit | CPF_ExportObject | CPF_PersistentInstance | CPF_InstancedReference | CPF_NonNullable | CPF_NoClear);

				if (UserWidgetProperty.AuthoritativeClass->HasAnyClassFlags(CLASS_HasInstancedReference))
				{
					ObjectVariableDesc.PropertyFlags |= (CPF_ContainsInstancedReference);
				}
			}
			else
			{
				ObjectVariableDesc.PropertyFlags |= (CPF_Transient | CPF_DuplicateTransient);
			}

			if (UserWidgetProperty.bExposeOnSpawn)
			{
				ObjectVariableDesc.PropertyFlags |= (CPF_ExposeOnSpawn);
			}
			else if (!bIsInstanceExposed)
			{
				ObjectVariableDesc.PropertyFlags |= (CPF_DisableEditOnInstance);
			}

			if (UserWidgetProperty.bReadOnly)
			{
				ObjectVariableDesc.PropertyFlags |= (CPF_BlueprintReadOnly);
			}

#if WITH_EDITOR
			if (!UserWidgetProperty.BlueprintSetter.IsEmpty())
			{
				ObjectVariableDesc.SetMetaData(FBlueprintMetadata::MD_PropertySetFunction, *UserWidgetProperty.BlueprintSetter);
			}
			if (!UserWidgetProperty.DisplayName.IsEmpty())
			{
				ObjectVariableDesc.SetMetaData(FBlueprintMetadata::MD_DisplayName, *UserWidgetProperty.DisplayName.ToString());
			}
			if (!UserWidgetProperty.CategoryName.IsEmpty())
			{
				ObjectVariableDesc.SetMetaData(FBlueprintMetadata::MD_FunctionCategory, *UserWidgetProperty.CategoryName);
			}
			if (UserWidgetProperty.bExposeOnSpawn)
			{
				ObjectVariableDesc.SetMetaData(FBlueprintMetadata::MD_ExposeOnSpawn, TEXT("true"));
			}
			if (UserWidgetProperty.bPrivate)
			{
				ObjectVariableDesc.SetMetaData(FBlueprintMetadata::MD_Private, TEXT("true"));
			}
			if (bIsInstanceExposed)
			{
				ObjectVariableDesc.SetMetaData(FName("EditInline"), TEXT("true"));
			}
#endif

			Context.AddGeneratedVariable(MoveTemp(ObjectVariableDesc));
		};

		for (FCompilerUserWidgetProperty& UserWidgetProperty : NeededUserWidgetProperties)
		{
			check(UserWidgetProperty.AuthoritativeClass);
			check(!UserWidgetProperty.Name.IsNone());

			// Check if the property already exists in parent classes
			FMVVMConstFieldVariant UserWidgetPropertyField = BindingHelper::FindFieldByName(Context.GetWidgetBlueprint()->ParentClass, FMVVMBindingName(UserWidgetProperty.Name));
			if (UserWidgetPropertyField.IsEmpty())
			{
				for (FField* Field = Context.GetWidgetBlueprint()->ParentClass->ChildProperties; Field != nullptr; Field = Field->Next)
				{
					if (Field->GetFName() == UserWidgetProperty.Name)
					{
						if (CastField<FProperty>(Field))
						{
							UserWidgetPropertyField = FMVVMFieldVariant(CastField<FProperty>(Field));
						}
						else
						{
							WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("FieldIsNotProperty", "The field for source '{0}' exists but is not a property."), UserWidgetProperty.DisplayName).ToString());
							bIsGatherGeneratedVariablesStepValid = false;
						}
						break;
					}
				}
				for (UField* Field = Context.GetWidgetBlueprint()->ParentClass->Children; Field != nullptr; Field = Field->Next)
				{
					if (Field->GetFName() == UserWidgetProperty.Name)
					{
						WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("FieldIsNotProperty", "The field for source '{0}' exists but is not a property."), UserWidgetProperty.DisplayName).ToString());
						bIsGatherGeneratedVariablesStepValid = false;
						break;
					}
				}
			}
			
			const FBPVariableDescription* ExistingGeneratedVariablePtr = Context.GetWidgetBlueprint()->GeneratedVariables.FindByPredicate([&UserWidgetProperty](const FBPVariableDescription& VariableDesc)
			{
				return VariableDesc.VarName == UserWidgetProperty.Name;
			});

			// Will always create viewmodel properties.
			// Will never create properties for animation or other Self.Object
			// Will create properties for widget when they are not already created.
			if (UserWidgetProperty.CreationType == FCompilerUserWidgetProperty::ECreationType::CreateOnlyIfDoesntExist)
			{
				if (ExistingGeneratedVariablePtr != nullptr)
				{
					WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("VariableAlreadyExistsInBP", "There is already a variable named '{0}' in blueprint '{1}'."), UserWidgetProperty.DisplayName, FText::FromString(Context.GetWidgetBlueprint()->GetFriendlyName())).ToString());
					bIsGatherGeneratedVariablesStepValid = false;
					continue;
				}
				else if (!UserWidgetPropertyField.IsEmpty())
				{
					// Viewmodel property cannot already exist. It will create issues with initialization and with View::SetViewModel.
					const UClass* OwnerClass = Cast<UClass>(UserWidgetPropertyField.GetOwner());
					WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("PropertyAlreadyExistInParent", "There is already a property named '{0}' in scope '{1}'."), UserWidgetProperty.DisplayName, (OwnerClass ? OwnerClass->GetDisplayNameText() : FText::GetEmpty())).ToString());
					bIsGatherGeneratedVariablesStepValid = false;
					continue;
				}
			}

			if (ExistingGeneratedVariablePtr != nullptr && (ExistingGeneratedVariablePtr->VarType.PinCategory != UEdGraphSchema_K2::PC_Object || ExistingGeneratedVariablePtr->VarType.PinSubCategoryObject != UserWidgetProperty.AuthoritativeClass))
			{
				WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("VariableExistsWithWrongType", "There is already a variable named '{0}' in blueprint '{1}' that does not have the expected type."), UserWidgetProperty.DisplayName, FText::FromString(Context.GetWidgetBlueprint()->GetFriendlyName())).ToString());
				bIsGatherGeneratedVariablesStepValid = false;
				continue;
			}

			const bool bCreateVariable = UserWidgetProperty.CreationType == FCompilerUserWidgetProperty::ECreationType::CreateOnlyIfDoesntExist
				|| (UserWidgetProperty.CreationType == FCompilerUserWidgetProperty::ECreationType::CreateIfDoesntExist && UserWidgetPropertyField.IsEmpty());
			if (bCreateVariable && ExistingGeneratedVariablePtr == nullptr)
			{
				AddObjectVariable(UserWidgetProperty);
			}
		}
	}
}


void FMVVMViewBlueprintCompiler::CreateVariables(const FWidgetBlueprintCompilerContext::FCreateVariableContext& Context)
{
	if (!BlueprintView)
	{
		return;
	}

	if (!AreStepsValid())
	{
		return;
	}

	if (Context.GetCompileType() == EKismetCompileType::SkeletonOnly)
	{
		CreatePublicFunctionsDeclaration(Context);
	}

	// Create variable
	{
		for (FCompilerUserWidgetProperty& UserWidgetProperty : NeededUserWidgetProperties)
		{
			check(UserWidgetProperty.AuthoritativeClass);
			check(!UserWidgetProperty.Name.IsNone());
			UserWidgetProperty.Property = nullptr; // Skeletal set the property, Full needs the new property

			FMVVMConstFieldVariant UserWidgetPropertyField = BindingHelper::FindFieldByName(Context.GetGeneratedClass(), FMVVMBindingName(UserWidgetProperty.Name));

			// The class is not linked yet. It may not be available yet.
			if (UserWidgetPropertyField.IsEmpty())
			{
				for (FField* Field = Context.GetGeneratedClass()->ChildProperties; Field != nullptr; Field = Field->Next)
				{
					if (Field->GetFName() == UserWidgetProperty.Name)
					{
						if (CastField<FProperty>(Field))
						{
							UserWidgetPropertyField = FMVVMFieldVariant(CastField<FProperty>(Field));
						}
						else
						{
							WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("FieldIsNotProperty", "The field for source '{0}' exists but is not a property."), UserWidgetProperty.DisplayName).ToString());
							bIsCreateVariableStepValid = false;
						}
						break;
					}
				}
				for (UField* Field = Context.GetGeneratedClass()->Children; Field != nullptr; Field = Field->Next)
				{
					if (Field->GetFName() == UserWidgetProperty.Name)
					{
						WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("FieldIsNotProperty", "The field for source '{0}' exists but is not a property."), UserWidgetProperty.DisplayName).ToString());
						bIsCreateVariableStepValid = false;
						break;
					}
				}
			}

			if (UserWidgetPropertyField.IsEmpty())
			{
				UClass* ParentClass = Context.GetGeneratedClass()->GetSuperClass();
				if (const FProperty* Property = ParentClass->FindPropertyByName(UserWidgetProperty.Name))
				{
					UserWidgetPropertyField = FMVVMFieldVariant(Property);
				}
			}

			if (!UserWidgetPropertyField.IsEmpty())
			{
				if (UserWidgetPropertyField.IsFunction())
				{
					WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("FunctionCanBeSource", "Function can't be source. '{0}'."), UserWidgetProperty.DisplayName).ToString());
					bIsCreateVariableStepValid = false;
					continue;
				}

				if (!BindingHelper::IsValidForSourceBinding(UserWidgetPropertyField))
				{
					WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("FieldNotAccessibleAtRuntime", "The field for source '{0}' exists but is not accessible at runtime."), UserWidgetProperty.DisplayName).ToString());
					bIsCreateVariableStepValid = false;
					continue;
				}

				ensure(UserWidgetPropertyField.IsProperty());
				const FProperty* Property = UserWidgetPropertyField.IsProperty() ? UserWidgetPropertyField.GetProperty() : nullptr;
				const FObjectProperty* ObjectProperty = CastField<const FObjectProperty>(Property);
				const bool bIsCompatible = ObjectProperty && UserWidgetProperty.AuthoritativeClass->IsChildOf(ObjectProperty->PropertyClass);
				if (!bIsCompatible)
				{
					WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("PropertyExistsAndNotCompatible", "There is already a property named '{0}' that is not compatible with the source of the same name."), UserWidgetProperty.DisplayName).ToString());
					bIsCreateVariableStepValid = false;
					continue;
				}
			}

			if (UserWidgetPropertyField.IsProperty())
			{
				UserWidgetProperty.Property = UserWidgetPropertyField.GetProperty();
			}

			if (UserWidgetProperty.Property == nullptr)
			{
				WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("VariableCouldNotBeCreated", "The variable for '{0}' could not be created."), UserWidgetProperty.DisplayName).ToString());
				bIsCreateVariableStepValid = false;
				continue;
			}
		}
	}

	// Add multicast delegate variables for any events / ubergraph pages that need them to fire
	{
		for (const TSharedRef<FCompilerBinding>& ValidBinding : ValidBindings)
		{
			FMVVMBlueprintViewBinding& Binding = *BlueprintView->GetBindingAt(ValidBinding->Key.ViewBindingIndex);

			// Functions are not yet generated / valid, but their graphs exist & we need to create delegates events that may be added to the ubergraph
			if (ValidBinding->Type == FCompilerBinding::EType::Unknown)
			{
				if (UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(ValidBinding->Key.bIsForwardBinding))
				{
					if (ConversionFunction->IsUbergraphPage() && !ConversionFunction->GetWrapperGraphName().IsNone())
					{
						// Create a multicast delegate variable that the event node will bind to
						Context.CreateMulticastDelegateVariable(ConversionFunction->GetWrapperGraphName());

						// Functions / Graphs should only be compiled in full compilations
						if (Context.GetCompileType() == EKismetCompileType::Full)
						{
							ConversionFunction->SetDestinationPath(Binding.DestinationPath);
							UEdGraph* WrapperGraph = ConversionFunction->GetOrCreateIntermediateWrapperGraph(WidgetBlueprintCompilerContext);

							ensure(WrapperGraph);
							ensure(ConversionFunction->IsWrapperGraphTransient());
							ensure(!WidgetBlueprintCompilerContext.Blueprint->FunctionGraphs.Contains(WrapperGraph));
						}
					}
				}
			}
		}
	}

	// Public function permissions
	{
		FPathPermissionList& FunctionPermissions = GetMutableDefault<UBlueprintEditorSettings>()->GetFunctionPermissions();
		FPathPermissionList& GeneratedFunctionPermissions = GetMutableDefault<UMVVMDeveloperProjectSettings>()->GetGeneratedFunctionPermissions();

		FName ContextName;
		{
			TStringBuilder<512> ContextNameStr;
			Context.GetGeneratedClass()->GetPathName(nullptr, ContextNameStr);
			ContextName = ContextNameStr.ToString();
		}

		FunctionPermissions.UnregisterOwner(ContextName);
		GeneratedFunctionPermissions.UnregisterOwner(ContextName);

		const bool bHasFiltering = GetMutableDefault<UBlueprintEditorSettings>()->GetFunctionPermissions().HasFiltering();
		for (FName FunctionName : FunctionPermissionsToAdd)
		{
			TStringBuilder<512> FunctionPath;
			FunctionPath << ContextName;
			FunctionPath << TEXT(':');
			FunctionPath << FunctionName;

			GeneratedFunctionPermissions.AddAllowListItem(ContextName, FunctionPath);

			if (bHasFiltering)
			{
				FunctionPermissions.AddAllowListItem(ContextName, FunctionPath);
			}
		}
	}
}


void FMVVMViewBlueprintCompiler::CreateWidgetMap(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context)
{
	// The widget tree is not created yet for SKEL class.
	//Context.GetGeneratedClass()->GetWidgetTreeArchetype()
	WidgetNameToWidgetPointerMap.Reset();

	TArray<UWidget*> Widgets;
	UWidgetBlueprint* WidgetBPToScan = Context.GetWidgetBlueprint();
	while (WidgetBPToScan != nullptr)
	{
		Widgets = WidgetBPToScan->GetAllSourceWidgets();
		if (Widgets.Num() != 0)
		{
			break;
		}
		WidgetBPToScan = WidgetBPToScan->ParentClass && WidgetBPToScan->ParentClass->ClassGeneratedBy ? Cast<UWidgetBlueprint>(WidgetBPToScan->ParentClass->ClassGeneratedBy) : nullptr;
	}

	for (UWidget* Widget : Widgets)
	{
		WidgetNameToWidgetPointerMap.Add(Widget->GetFName(), Widget);
	}
}


void FMVVMViewBlueprintCompiler::CreateBindingList(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context)
{
	ValidBindings.Reset();

	// Build the list of bindings that we should compile.
	for (int32 Index = 0; Index < BlueprintView->GetNumBindings(); ++Index)
	{
		FMVVMBlueprintViewBinding* BindingPtr = BlueprintView->GetBindingAt(Index);
		if (BindingPtr == nullptr)
		{
			WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("BindingInvalidIndex", "Internal error: Tried to fetch binding for invalid index {0}."), Index).ToString());
			bIsGatherGeneratedVariablesStepValid = false;
			continue;
		}

		FMVVMBlueprintViewBinding& Binding = *BindingPtr;
		if (!Binding.bCompile)
		{
			continue;
		}

		const bool bHasConversionFunction = Binding.Conversion.GetConversionFunction(true) || Binding.Conversion.GetConversionFunction(false);
		if (bHasConversionFunction && Binding.BindingType == EMVVMBindingMode::TwoWay)
		{
			AddMessageForBinding(Binding, LOCTEXT("TwoWayBindingsWithConversion", "Two-way bindings are not allowed to use conversion functions."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
			bIsGatherGeneratedVariablesStepValid = false;
			continue;
		}

		if (IsForwardBinding(Binding.BindingType))
		{
			TSharedRef<FCompilerBinding> ValidBinding = MakeShared<FCompilerBinding>();
			ValidBinding->Key.ViewBindingIndex = Index;
			ValidBinding->Key.bIsForwardBinding = true;
			ValidBinding->bIsOneTimeBinding = IsOneTimeBinding(Binding.BindingType);

			ValidBindings.Add(ValidBinding);
		}
		if (IsBackwardBinding(Binding.BindingType))
		{
			TSharedRef<FCompilerBinding> ValidBinding = MakeShared<FCompilerBinding>();
			ValidBinding->Key.ViewBindingIndex = Index;
			ValidBinding->Key.bIsForwardBinding = false;
			ValidBinding->bIsOneTimeBinding = IsOneTimeBinding(Binding.BindingType);

			ValidBindings.Add(ValidBinding);
		}
	}
}


void FMVVMViewBlueprintCompiler::CreateEventList(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context)
{
	for (UMVVMBlueprintViewEvent* EventPtr : BlueprintView->GetEvents())
	{
		if (EventPtr == nullptr)
		{
			WidgetBlueprintCompilerContext.MessageLog.Error(*LOCTEXT("EventInvalid", "Internal error: An event is invalid.").ToString());
			bIsGatherGeneratedVariablesStepValid = false;
			continue;
		}

		if (!EventPtr->bCompile)
		{
			continue;
		}

		if (!EventPtr->GetEventPath().HasPaths())
		{
			AddMessageForEvent(EventPtr, LOCTEXT("EventInvalidEventPath", "The event path is invalid."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
			bIsGatherGeneratedVariablesStepValid = false;
			continue;
		}

		TSharedRef<FCompilerEvent> ValidEvent = MakeShared<FCompilerEvent>();
		ValidEvent->Event = EventPtr;

		ValidEvents.Add(ValidEvent);
	}
}

void FMVVMViewBlueprintCompiler::CreateConditionList(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context)
{
	bool bAllowConditionBinding = GetDefault<UMVVMDeveloperProjectSettings>()->bAllowConditionBinding;

	for (UMVVMBlueprintViewCondition* ConditionPtr : BlueprintView->GetConditions())
	{
		if (ConditionPtr == nullptr)
		{
			WidgetBlueprintCompilerContext.MessageLog.Error(*LOCTEXT("ConditionInvalid", "Internal error: A condition is invalid.").ToString());
			bIsGatherGeneratedVariablesStepValid = false;
			continue;
		}

		if (!ConditionPtr->bCompile || !bAllowConditionBinding)
		{
			continue;
		}

		if (!ConditionPtr->GetConditionPath().HasPaths())
		{
			AddMessageForCondition(ConditionPtr, LOCTEXT("ConditionInvalidConditionPath", "The condition path is invalid."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
			bIsGatherGeneratedVariablesStepValid = false;
			continue;
		}

		TSharedRef<FCompilerCondition> ValidCondition = MakeShared<FCompilerCondition>();
		ValidCondition->Condition = ConditionPtr;

		ValidConditions.Add(ValidCondition);
	}
}



void FMVVMViewBlueprintCompiler::CreateExtensionList(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context)
{
	for (int32 Index = BlueprintView->GetOuterUMVVMWidgetBlueprintExtension_View()->BlueprintExtensions.Num() - 1; Index >= 0; --Index)
	{
		FMVVMExtensionItem& Extension = BlueprintView->GetOuterUMVVMWidgetBlueprintExtension_View()->BlueprintExtensions[Index];
		if (Extension.ExtensionObj == nullptr)
		{
			BlueprintView->GetOuterUMVVMWidgetBlueprintExtension_View()->BlueprintExtensions.RemoveAtSwap(Index);
			break;
		}

		TSharedRef<FCompilerExtension> ValidExtension = MakeShared<FCompilerExtension>();
		ValidExtension->Extension = Extension.ExtensionObj;

		ValidExtensions.Add(ValidExtension);
	}
}


void FMVVMViewBlueprintCompiler::CreateRequiredProperties(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context)
{
	NeededBindingSources.Reset();
	NeededUserWidgetProperties.Reset();
	ViewModelCreatorContexts.Reset();
	ViewModelSettersToGenerate.Reset();

	TSet<FGuid> ViewModelGuids;
	for (const FMVVMBlueprintViewModelContext& ViewModelContext : BlueprintView->GetViewModels())
	{
		if (!ViewModelContext.GetViewModelId().IsValid())
		{
			AddMessageForViewModel(ViewModelContext, LOCTEXT("ViewmodelInvalidGuid", "GUID is invalid."), Compiler::EMessageType::Error);
			bIsGatherGeneratedVariablesStepValid = false;
			continue;
		}

		if (ViewModelGuids.Contains(ViewModelContext.GetViewModelId()))
		{
			AddMessageForViewModel(ViewModelContext, LOCTEXT("ViewmodelAlreadyAdded", "Identical viewmodel has already been added."), Compiler::EMessageType::Error);
			bIsGatherGeneratedVariablesStepValid = false;
			continue;
		}

		ViewModelGuids.Add(ViewModelContext.GetViewModelId());

		if (ViewModelContext.GetViewModelClass() == nullptr || !ViewModelContext.IsValid())
		{
			AddMessageForViewModel(ViewModelContext, LOCTEXT("ViewmodelInvalidClass", "Invalid class."), Compiler::EMessageType::Error);
			bIsGatherGeneratedVariablesStepValid = false;
			continue;
		}

		FName PropertyName = ViewModelContext.GetViewModelName();

		const bool bCreateSetterFunction = GetDefault<UMVVMDeveloperProjectSettings>()->bAllowGeneratedViewModelSetter
			&& (ViewModelContext.bCreateSetterFunction || ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::Manual);
		FString SetterFunctionName;
		if (bCreateSetterFunction)
		{
			SetterFunctionName = TEXT("Set") + ViewModelContext.GetViewModelName().ToString();

			FCompilerViewModelSetter& ViewModelSetter = ViewModelSettersToGenerate.AddDefaulted_GetRef();
			ViewModelSetter.Class = ViewModelContext.GetViewModelClass();
			ViewModelSetter.PropertyName = PropertyName;
			ViewModelSetter.BlueprintSetter = SetterFunctionName;
			ViewModelSetter.DisplayName = ViewModelContext.GetDisplayName();
		}

		TSharedRef<FCompilerBindingSource> SourceContext = MakeShared<FCompilerBindingSource>();
		{
			SourceContext->AuthoritativeClass = ViewModelContext.GetViewModelClass();
			SourceContext->Name = PropertyName;
			SourceContext->Type = FCompilerBindingSource::EType::ViewModel;
			SourceContext->bIsOptional = ViewModelContext.bOptional;

			if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::Manual || ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::Context)
			{
				SourceContext->bIsOptional = true;
			}
			else if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::CreateInstance)
			{
				SourceContext->bIsOptional = false;
			}

			NeededBindingSources.Add(SourceContext);
		}

		{
			bool bIsPublicReadable = ViewModelContext.InstancedViewModel == nullptr && ViewModelContext.bCreateGetterFunction;

			FCompilerUserWidgetProperty& SourceVariable = NeededUserWidgetProperties.AddDefaulted_GetRef();
			SourceVariable.AuthoritativeClass = ViewModelContext.GetViewModelClass();
			SourceVariable.Name = PropertyName;
			SourceVariable.DisplayName = ViewModelContext.GetDisplayName();
			SourceVariable.CategoryName = TEXT("Viewmodel");
			SourceVariable.Guid = ViewModelContext.GetViewModelId();
			SourceVariable.BlueprintSetter = SetterFunctionName;
			SourceVariable.CreationType = FCompilerUserWidgetProperty::ECreationType::CreateOnlyIfDoesntExist;
			SourceVariable.bExposeOnSpawn = bCreateSetterFunction;
			SourceVariable.bPrivate = !bIsPublicReadable;
			SourceVariable.bReadOnly = !bCreateSetterFunction;
			SourceVariable.bInstanced = ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::CreateInstance;
			SourceVariable.bInstanceExposed = ViewModelContext.bExposeInstanceInEditor;
		}

		{
			FCompilerViewModelCreatorContext& CreatorContext = ViewModelCreatorContexts.AddDefaulted_GetRef();
			CreatorContext.ViewModelContext = ViewModelContext;
			CreatorContext.Source = SourceContext;
		}
	}

	TSet<FName> WidgetSourcesCreated;
	TSet<FName> WidgetUserPropertyCreated;
	bool bSelfBindingSourceCreated = NeededBindingSources.ContainsByPredicate([](const TSharedRef<FCompilerBindingSource>& Other)
		{
			return Other->Type == FCompilerBindingSource::EType::Self;
		});

	auto GenerateCompilerContext = [Self = this, WidgetBP = Context.GetWidgetBlueprint(), &ViewModelGuids, &WidgetSourcesCreated, &WidgetUserPropertyCreated, &bSelfBindingSourceCreated](bool bInCreateSource, const FMVVMBlueprintPropertyPath& PropertyPath) -> TValueOrError<void, FText>
	{
		auto CreateSourceForSelf = [Self, &bSelfBindingSourceCreated]() -> TSharedRef<FCompilerBindingSource>
		{
			TSharedRef<FCompilerBindingSource> SourceContext = MakeShared<FCompilerBindingSource>();
			SourceContext->AuthoritativeClass = nullptr;
			SourceContext->Name = Self->WidgetBlueprintCompilerContext.WidgetBlueprint()->GetFName();
			SourceContext->Type = FCompilerBindingSource::EType::Self;
			SourceContext->bIsOptional = false;

			if (!bSelfBindingSourceCreated)
			{
				Self->NeededBindingSources.Add(SourceContext);
				bSelfBindingSourceCreated = true;
			}
			return SourceContext;
		};
		FName PropertyPathWidgetName = PropertyPath.GetWidgetName();
		FGuid PropertyPathViewModelId = PropertyPath.GetViewModelId();

		bool bIsComponentOnSelf = false;
		if (PropertyPath.IsComponent())
		{
			const UWidgetBlueprintGeneratedClass* WidgetBPGeneratedClass = Cast<UWidgetBlueprintGeneratedClass>(WidgetBP->GeneratedClass);
			const UWidgetBlueprintGeneratedClass* ComponentOwningWidgetBlueprint = UE::MVVM::FieldPathHelper::GetComponentPropertyPathSource(PropertyPath.GetFields(WidgetBPGeneratedClass), WidgetBPGeneratedClass);
			bIsComponentOnSelf = ComponentOwningWidgetBlueprint == WidgetBPGeneratedClass;
		}

		// If the path is "self.widget.property", "self.viewmodel.property", then remove the "self".
		EMVVMBlueprintFieldPathSource FieldPathSource = PropertyPath.GetSource(Self->WidgetBlueprintCompilerContext.WidgetBlueprint());
		if (FieldPathSource == EMVVMBlueprintFieldPathSource::SelfContext)
		{
			const TArrayView<const FMVVMBlueprintFieldPath> FieldPaths = PropertyPath.GetFieldPaths();
			if (FieldPaths.Num() >= 2 && !bIsComponentOnSelf)
			{
				if (FieldPaths[0].GetBindingKind() == EBindingKind::Property && FieldPaths[0].IsFieldSelfContext())
				{
					FName RawFieldName = FieldPaths[0].GetRawFieldName();
					if (Self->WidgetNameToWidgetPointerMap.Contains(RawFieldName))
					{
						FieldPathSource = EMVVMBlueprintFieldPathSource::Widget;
						PropertyPathWidgetName = RawFieldName;
					}
					else if (const FMVVMBlueprintViewModelContext* SourceViewModelContext = Self->BlueprintView->FindViewModel(RawFieldName))
					{
						FieldPathSource = EMVVMBlueprintFieldPathSource::ViewModel;
						PropertyPathViewModelId = SourceViewModelContext->GetViewModelId();
					}
				}
			}
		}

		switch (FieldPathSource)
		{
		case EMVVMBlueprintFieldPathSource::SelfContext:
		{
			CreateSourceForSelf();
			return MakeValue();
		}
		case EMVVMBlueprintFieldPathSource::Widget:
		{
			// Only do this once
			bool bNewCreateSource = !WidgetSourcesCreated.Contains(PropertyPathWidgetName) && bInCreateSource;
			bool bNewAddWidgetProperty = !WidgetUserPropertyCreated.Contains(PropertyPathWidgetName);
			if (bNewCreateSource || bNewAddWidgetProperty)
			{
				UWidget** WidgetPtr = Self->WidgetNameToWidgetPointerMap.Find(PropertyPathWidgetName);
				if (WidgetPtr == nullptr || *WidgetPtr == nullptr)
				{
					return MakeError(FText::Format(LOCTEXT("InvalidWidgetFormat", "Could not find the targeted widget: {0}"), FText::FromName(PropertyPathWidgetName)));
				}
				UWidget* Widget = *WidgetPtr;

				if (bNewCreateSource)
				{
					FName PropertyName = PropertyPathWidgetName;
					{
						TSharedRef<FCompilerBindingSource>* FoundCompilerSource = Self->NeededBindingSources.FindByPredicate([PropertyName](const TSharedRef<FCompilerBindingSource>& Other)
							{
								return Other->Name == PropertyName;
							});
						if (FoundCompilerSource != nullptr)
						{
							// It should be in the TSet<FName> WidgetSources
							return MakeError(FText::Format(LOCTEXT("ExistingWidgetSourceFormat", "Internal error. A widget source already exist: {0}"), FText::FromName(PropertyPathWidgetName)));
						}
					}

					TSharedRef<FCompilerBindingSource> SourceContext = MakeShared<FCompilerBindingSource>();
					SourceContext->AuthoritativeClass = Widget->GetClass();
					SourceContext->Name = PropertyName;
					SourceContext->Type = FCompilerBindingSource::EType::Widget;
					SourceContext->bIsOptional = false;
					Self->NeededBindingSources.Add(SourceContext);

					WidgetSourcesCreated.Add(Widget->GetFName());
				}

				if (bNewAddWidgetProperty)
				{
					FCompilerUserWidgetProperty& SourceVariable = Self->NeededUserWidgetProperties.AddDefaulted_GetRef();
					SourceVariable.AuthoritativeClass = Widget->GetClass();
					SourceVariable.Name = PropertyPathWidgetName;
					SourceVariable.Guid = WidgetBP->WidgetVariableNameToGuidMap.FindRef(Widget->GetFName());
					SourceVariable.DisplayName = Widget->IsGeneratedName() ? FText::FromString(Widget->GetName()) : Widget->GetLabelText();
					SourceVariable.CategoryName = TEXT("Widget");
					SourceVariable.CreationType = FCompilerUserWidgetProperty::ECreationType::CreateIfDoesntExist;
					SourceVariable.bExposeOnSpawn = false;
					SourceVariable.bPrivate = true;
					SourceVariable.bReadOnly = true;
					SourceVariable.bInstanced = false;
					SourceVariable.bInstanceExposed = false;

					WidgetUserPropertyCreated.Add(Widget->GetFName());
				}

				if (bIsComponentOnSelf)
				{
					CreateSourceForSelf();
				}
			}
			break;
		}
		case EMVVMBlueprintFieldPathSource::ViewModel:
		{
			const FMVVMBlueprintViewModelContext* SourceViewModelContext = Self->BlueprintView->FindViewModel(PropertyPathViewModelId);
			if (SourceViewModelContext == nullptr)
			{
				return MakeError(FText::Format(LOCTEXT("BindingViewModelNotFound", "Could not find viewmodel with GUID {0}."), GetViewModelIdText(PropertyPathViewModelId)));
			}

			if (!ViewModelGuids.Contains(SourceViewModelContext->GetViewModelId()))
			{
				return MakeError(FText::Format(LOCTEXT("BindingViewModelInvalid", "Viewmodel {0} {1} was invalid."), SourceViewModelContext->GetDisplayName(), GetViewModelIdText(PropertyPathViewModelId)));
			}
			break;
		}
		default:
			return MakeError(LOCTEXT("SourcePathNotSet", "A source path is required, but not set."));
		}
		return MakeValue();
	};

	// Find the start property for each path.
	//The full path will be tested later. We want to build the list of property needed before generating the graphs.
	for (const TSharedRef<FCompilerBinding>& ValidBinding : ValidBindings)
	{
		const FMVVMBlueprintViewBinding& Binding = *(BlueprintView->GetBindingAt(ValidBinding->Key.ViewBindingIndex));

		auto RunGenerateCompilerSourceContext = [Self = this, &GenerateCompilerContext, &Binding](bool bCreateSource, const FMVVMBlueprintPropertyPath& PropertyPath, const FMVVMBlueprintPinId& PinId)
			{
				TValueOrError<void, FText> SourceContextResult = GenerateCompilerContext(bCreateSource, PropertyPath);
				if (SourceContextResult.HasError())
				{
					Self->AddMessageForBinding(Binding, SourceContextResult.StealError(), Compiler::EMessageType::Error, PinId);
					Self->bIsGatherGeneratedVariablesStepValid = false;
				}
			};

		UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(ValidBinding->Key.bIsForwardBinding);
		if (ConversionFunction)
		{
			// validate the sources
			for (const FMVVMBlueprintPin& Pin : ConversionFunction->GetPins())
			{
				if (Pin.UsedPathAsValue())
				{
					if (!Pin.IsValid())
					{
						AddMessageForBinding(Binding
							, FText::Format(LOCTEXT("InvalidFunctionArgumentPathName", "The conversion function {0} has an invalid argument."), FText::FromString(Binding.GetDisplayNameString(WidgetBlueprintCompilerContext.WidgetBlueprint())))
							, Compiler::EMessageType::Error
							, FMVVMBlueprintPinId()
						);
						bIsGatherGeneratedVariablesStepValid = false;
						continue;
					}
					RunGenerateCompilerSourceContext(true, Pin.GetPath(), Pin.GetId());
				}
			}

			// validate the destination
			const FMVVMBlueprintPropertyPath& BindingDestinationPath = ValidBinding->Key.bIsForwardBinding ? Binding.DestinationPath : Binding.SourcePath;
			RunGenerateCompilerSourceContext(false, BindingDestinationPath, FMVVMBlueprintPinId());
		}
		else
		{
			// if we aren't using a conversion function, validate the source and destination paths
			RunGenerateCompilerSourceContext(ValidBinding->Key.bIsForwardBinding, Binding.SourcePath, FMVVMBlueprintPinId());
			RunGenerateCompilerSourceContext(!ValidBinding->Key.bIsForwardBinding, Binding.DestinationPath, FMVVMBlueprintPinId());
		}
	}

	// Find the event property.
	//All inputs must also exist (for the BP to compile).
	for (TSharedRef<FCompilerEvent>& ValidEvent : ValidEvents)
	{
		UMVVMBlueprintViewEvent* EventPtr = ValidEvent->Event.Get();
		check(EventPtr);
		auto RunGenerateCompilerSourceContext = [Self = this, &GenerateCompilerContext, EventPtr](bool bCreateSource, const FMVVMBlueprintPropertyPath& PropertyPath, const FMVVMBlueprintPinId& PinId)
		{
			TValueOrError<void, FText> SourceContextResult = GenerateCompilerContext(bCreateSource, PropertyPath);
			if (SourceContextResult.HasError())
			{
				Self->AddMessageForEvent(EventPtr, SourceContextResult.StealError(), Compiler::EMessageType::Error, PinId);
				Self->bIsGatherGeneratedVariablesStepValid = false;
			}
		};

		RunGenerateCompilerSourceContext(false, EventPtr->GetEventPath(), FMVVMBlueprintPinId());
		RunGenerateCompilerSourceContext(false, EventPtr->GetDestinationPath(), FMVVMBlueprintPinId());
		for (const FMVVMBlueprintPin& Pin : EventPtr->GetPins())
		{
			if (Pin.UsedPathAsValue())
			{
				if (!Pin.IsValid())
				{
					AddMessageForEvent(EventPtr
						, LOCTEXT("InvalidEventArgumentPathName", "The event has an invalid argument.")
						, Compiler::EMessageType::Error
						, FMVVMBlueprintPinId()
					);
					bIsGatherGeneratedVariablesStepValid = false;
					continue;
				}
				RunGenerateCompilerSourceContext(true, Pin.GetPath(), Pin.GetId());
			}
		}
	}

	for (TSharedRef<FCompilerCondition>& ValidCondition : ValidConditions)
	{
		UMVVMBlueprintViewCondition* ConditionPtr = ValidCondition->Condition.Get();
		check(ConditionPtr);
		auto RunGenerateCompilerSourceContext = [Self = this, &GenerateCompilerContext, ConditionPtr](bool bCreateSource, const FMVVMBlueprintPropertyPath& PropertyPath, const FMVVMBlueprintPinId& PinId)
			{
				TValueOrError<void, FText> SourceContextResult = GenerateCompilerContext(bCreateSource, PropertyPath);
				if (SourceContextResult.HasError())
				{
					Self->AddMessageForCondition(ConditionPtr, SourceContextResult.StealError(), Compiler::EMessageType::Error, PinId);
					Self->bIsGatherGeneratedVariablesStepValid = false;
				}
			};

		RunGenerateCompilerSourceContext(false, ConditionPtr->GetConditionPath(), FMVVMBlueprintPinId());
		for (const FMVVMBlueprintPin& Pin : ConditionPtr->GetPins())
		{
			if (Pin.UsedPathAsValue())
			{
				if (!Pin.IsValid())
				{
					AddMessageForCondition(ConditionPtr
						, LOCTEXT("InvalidConditionArgumentPathName", "The condition has an invalid argument.")
						, Compiler::EMessageType::Error
						, FMVVMBlueprintPinId()
					);
					bIsGatherGeneratedVariablesStepValid = false;
					continue;
				}
				RunGenerateCompilerSourceContext(true, Pin.GetPath(), Pin.GetId());
			}
		}
	}

	// Extension can define properties.
	for (TSharedRef<FCompilerExtension>& Extension : ValidExtensions)
	{
		UMVVMBlueprintViewExtension* ExtensionPtr = Extension->Extension.Get();
		if (ensure(ExtensionPtr))
		{
			TArray<Compiler::FBlueprintViewUserWidgetProperty> Properties = ExtensionPtr->AddProperties();
			for (const Compiler::FBlueprintViewUserWidgetProperty& Property : Properties)
			{
				FCompilerUserWidgetProperty& CompilerUserWidgetProperty = NeededUserWidgetProperties.AddDefaulted_GetRef();

				CompilerUserWidgetProperty.AuthoritativeClass = Property.AuthoritativeClass;
				CompilerUserWidgetProperty.Name = Property.Name;
				CompilerUserWidgetProperty.DisplayName = Property.DisplayName;
				CompilerUserWidgetProperty.bExposeOnSpawn = Property.bExposeOnSpawn;
				CompilerUserWidgetProperty.bPrivate = Property.bPrivate;
				CompilerUserWidgetProperty.bReadOnly = Property.bReadOnly;
				CompilerUserWidgetProperty.CategoryName = Property.CategoryName;
				CompilerUserWidgetProperty.Guid = Property.Guid;
				CompilerUserWidgetProperty.CreationType = FCompilerUserWidgetProperty::ECreationType::CreateIfDoesntExist;
			}

			TArray<Compiler::FBlueprintViewUserWidgetWidgetProperty> WidgetProperties = ExtensionPtr->AddWidgetProperties();
			for (const Compiler::FBlueprintViewUserWidgetWidgetProperty& WidgetProperty : WidgetProperties)
			{
				if (WidgetNameToWidgetPointerMap.Find(WidgetProperty.WidgetName))
				{				
					FMVVMBlueprintPropertyPath WidgetPropertyPath;
					WidgetPropertyPath.SetWidgetName(WidgetProperty.WidgetName);
					TValueOrError<void, FText> SourceContextResult = GenerateCompilerContext(false, WidgetPropertyPath);
					if (SourceContextResult.HasError())
					{
						AddMessage(SourceContextResult.StealError(), Compiler::EMessageType::Error);
						bIsGatherGeneratedVariablesStepValid = false;
					}
				}
				else
				{
					AddMessage(FText::Format(LOCTEXT("InvalidWidgetPropertyRequested", "Could not find the widget : {0} in the widget tree to make the property requested by viewmodel extension."), FText::FromName(WidgetProperty.WidgetName)), Compiler::EMessageType::Error);
					bIsGatherGeneratedVariablesStepValid = false;
				}
			}
		}
	}
}


void FMVVMViewBlueprintCompiler::CreatePublicFunctionsDeclaration(const FWidgetBlueprintCompilerContext::FCreateVariableContext& Context)
{
	// Clean all previous intermediate function graph. It should stay alive. The graph lives on the Blueprint not on the class and it's used to generate the UFunction.
	for (UEdGraph* OldGraph : BlueprintView->TemporaryGraph)
	{
		if (OldGraph)
		{
			FunctionGraphHelper::RenameObjectToTransientPackage(OldGraph);
		}
	}
	// During deserialization for undo/redo, the existing graphs can get renamed to leave room for the serialized objects with the same name. When this happens, we basically don't have
	// any pointers left on the graphs and nobody can clean them anymore.
	for (FName OldGraphName : BlueprintView->TemporaryGraphNames)
	{
		UEdGraph* OldGraph = static_cast<UEdGraph*>(StaticFindObjectFastSafe(UEdGraph::StaticClass(), WidgetBlueprintCompilerContext.Blueprint, OldGraphName));
		if (OldGraph)
		{
			FunctionGraphHelper::RenameObjectToTransientPackage(OldGraph);
		}
	}
	BlueprintView->TemporaryGraph.Reset();
	BlueprintView->TemporaryGraphNames.Reset();

	if (GetDefault<UMVVMDeveloperProjectSettings>()->bAllowGeneratedViewModelSetter)
	{
		for (FCompilerViewModelSetter& Setter : ViewModelSettersToGenerate)
		{
			ensure(Setter.SetterGraph == nullptr);

			Setter.SetterGraph = UE::MVVM::FunctionGraphHelper::CreateIntermediateFunctionGraph(
				WidgetBlueprintCompilerContext
				, Setter.BlueprintSetter
				, (FUNC_BlueprintCallable | FUNC_Public | FUNC_Final)
				, TEXT("Viewmodel")
				, false);

			if (Setter.SetterGraph == nullptr || Setter.SetterGraph->GetFName() != FName(*Setter.BlueprintSetter))
			{
				WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("SetterNameAlreadyExists", "The setter name {0} already exists and could not be autogenerated."),
					FText::FromString(Setter.BlueprintSetter)
				).ToString()
				);

				if (Setter.SetterGraph != nullptr)
				{
					FunctionGraphHelper::RenameObjectToTransientPackage(Setter.SetterGraph);
					Setter.SetterGraph = nullptr;
				}
				bIsCreateVariableStepValid = false;
				continue;
			}

			Setter.SetterGraph->ClearFlags(RF_Transactional);
			BlueprintView->TemporaryGraph.Add(Setter.SetterGraph);
			FName GraphName = Setter.SetterGraph->GetFName();
			BlueprintView->TemporaryGraphNames.Add(GraphName);
			FunctionPermissionsToAdd.AddUnique(GraphName);
			GeneratedFunctions.AddUnique(GraphName);

			UE::MVVM::FunctionGraphHelper::AddFunctionArgument(Setter.SetterGraph, const_cast<UClass*>(Setter.Class), "Viewmodel");
		}
	}
}

void FMVVMViewBlueprintCompiler::CreateFunctions(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context)
{
	if (!AreStepsValid())
	{
		return;
	}

	SourceViewModelDynamicCreatorContexts.Reset();

	CategorizeBindings(Context);
	CategorizeEvents(Context);
	CategorizeConditions(Context);
	CreateWriteFieldContexts(Context);
	CreateViewModelSetters(Context);
	CreateIntermediateGraphFunctions(Context);
	CategorizeAsyncFunctions(Context);
}

void FMVVMViewBlueprintCompiler::CategorizeBindings(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context)
{
	// Find the type of the bindings
	for (TSharedRef<FCompilerBinding>& ValidBinding : ValidBindings)
	{
		FMVVMBlueprintViewBinding& Binding = *BlueprintView->GetBindingAt(ValidBinding->Key.ViewBindingIndex);

		check(ValidBinding->Type == FCompilerBinding::EType::Unknown);
		ValidBinding->Type = FCompilerBinding::EType::Invalid;

		UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(ValidBinding->Key.bIsForwardBinding);
		if (ConversionFunction)
		{
			UClass* NewClass = WidgetBlueprintCompilerContext.NewClass;
			if (!ConversionFunction->IsValid(WidgetBlueprintCompilerContext.WidgetBlueprint()))
			{
				AddMessageForBinding(Binding, LOCTEXT("InvalidConversionFunction", "The conversion function is invalid."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());

				if (CVarLogInvalidBindingVerbose->GetBool())
				{
					FText ConversionFuncInfo = FText::Format(LOCTEXT("ConversionFuncInfo", "Conversion Function Info: Is Ubergraph: {0}, Conv Type: {1}, Conv Func: {2}")
						, FText::FromString(ConversionFunction->IsUbergraphPage() ? "Y" : "N")
						, FText::FromString(ConversionFunction->GetConversionFunction().GetType() == EMVVMBlueprintFunctionReferenceType::Function ? "Function" : "Node")
						, FText::FromString(ConversionFunction->GetConversionFunction().ToString())
					);
					AddMessageForBinding(Binding, ConversionFuncInfo, Compiler::EMessageType::Error, FMVVMBlueprintPinId());

					FText DeveloperSettingsInfo = FText::Format(LOCTEXT("DeveloperSettingsInfo", "Developer Settings Info: Filter Type: {0}")
						, FText::FromString(GetDefault<UMVVMDeveloperProjectSettings>()->ConversionFunctionFilter == EMVVMDeveloperConversionFunctionFilterType::AllowedList ? "AllowedList" : "BlueprintActionRegistry")
					);
					AddMessageForBinding(Binding, DeveloperSettingsInfo, Compiler::EMessageType::Error, FMVVMBlueprintPinId());

					for (const FSoftClassPath& AllowedClassForConversionFunction : GetDefault<UMVVMDeveloperProjectSettings>()->AllowedClassForConversionFunctions)
					{
						FText AllowedClass = FText::FromString("AllowedClass: " + AllowedClassForConversionFunction.ToString());
						AddMessageForBinding(Binding, AllowedClass, Compiler::EMessageType::Error, FMVVMBlueprintPinId());
					}

					for (const FSoftClassPath& DeniedClassForConversionFunction : GetDefault<UMVVMDeveloperProjectSettings>()->DeniedClassForConversionFunctions)
					{
						FText DeniedClass = FText::FromString("DeniedClass: " + DeniedClassForConversionFunction.ToString());
						AddMessageForBinding(Binding, DeniedClass, Compiler::EMessageType::Error, FMVVMBlueprintPinId());
					}

					for (const FName DeniedModuleForConversionFunction : GetDefault<UMVVMDeveloperProjectSettings>()->DeniedModuleForConversionFunctions)
					{
						FText DeniedModule = FText::FromString("DeniedModule: " + DeniedModuleForConversionFunction.ToString());
						AddMessageForBinding(Binding, DeniedModule, Compiler::EMessageType::Error, FMVVMBlueprintPinId());
					}

					if (TSubclassOf<UK2Node> InvalidNode = ConversionFunction->GetConversionFunction().GetNode())
					{
						TStringBuilder<512> FunctionClassPath;
						InvalidNode->GetPathName(nullptr, FunctionClassPath);

						FText NodeInfo = FText::Format(LOCTEXT("NodeInfo", "Conversion Node Info: FName: {0}, ClassPathName: {1}, Module: {2}, FunctionClassPath: {3}")
							, FText::FromString(InvalidNode->GetFName().ToString())
							, FText::FromString(InvalidNode->GetClassPathName().ToString())
							, FText::FromString(InvalidNode->GetClassPathName().GetPackageName().ToString())
							, FText::FromString(FunctionClassPath.ToString())
						);
						AddMessageForBinding(Binding, NodeInfo, Compiler::EMessageType::Error, FMVVMBlueprintPinId());
					}
				}

				bIsCreateFunctionsStepValid = false;
				continue;
			}

			// Make sure the graph is up to date
			UEdGraph* WrapperGraph = ConversionFunction->GetOrCreateIntermediateWrapperGraph(WidgetBlueprintCompilerContext);
			if (WrapperGraph == nullptr)
			{
				AddMessageForBinding(Binding, LOCTEXT("InvalidConversionFunctionGraph", "The conversion function graph could not be generated.")
					, Compiler::EMessageType::Error
					, FMVVMBlueprintPinId()
				);
				bIsCreateFunctionsStepValid = false;
				continue;
			}

			ConversionFunction->UpdatePinValues(WidgetBlueprintCompilerContext.WidgetBlueprint());

			if (ConversionFunction->HasOrphanedPin())
			{
				AddMessageForBinding(Binding, LOCTEXT("InvalidConversionFunctionGraphOrphaned", "The conversion function has an orphaned pin.")
					, Compiler::EMessageType::Warning
					, FMVVMBlueprintPinId()
				);
				bIsCreateFunctionsStepValid = false;
				continue;
			}

			if (ConversionFunction->GetPins().Num() == 0)
			{
				AddMessageForBinding(Binding
					, FText::Format(LOCTEXT("InvalidNumberOfFunctionPin", "The conversion function {0} has no source."), FText::FromString(Binding.GetDisplayNameString(WidgetBlueprintCompilerContext.WidgetBlueprint())))
					, Compiler::EMessageType::Error
					, FMVVMBlueprintPinId()
				);
				bIsCreateFunctionsStepValid = false;
				continue;
			}

			if (ConversionFunction->NeedsWrapperGraph(WidgetBlueprintCompilerContext.WidgetBlueprint()))
			{
				ValidBinding->Type = FCompilerBinding::EType::ComplexConversionFunction;
			}
			else
			{
				ValidBinding->Type = FCompilerBinding::EType::SimpleConversionFunction;
			}
			ValidBinding->ConversionFunction = ConversionFunction;

			// Because the editor use the destination to order the bindings, the destination path can be "valid". Pointing only to the WidgetName.
			//const FMVVMBlueprintPropertyPath& BindingSourcePath = ValidBinding->Key.bIsForwardBinding ? Binding.SourcePath : Binding.DestinationPath;
			//if (BindingSourcePath.IsValid())
			//{
			//	AddMessageForBinding(Binding, LOCTEXT("ShouldNotHaveSourceWarning", "Internal Error. The binding should not have a source."), EMessageType::Warning, FMVVMBlueprintPinId());
			//}
		}
		else
		{
			ValidBinding->Type = FCompilerBinding::EType::Assignment;
		}
	}
}

void FMVVMViewBlueprintCompiler::CategorizeEvents(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context)
{
	TArray<TSharedRef<FCompilerEvent>> TemporaryEvent = MoveTemp(ValidEvents);
	ValidEvents.Reset(TemporaryEvent.Num());

	for (TSharedRef<FCompilerEvent>& Event : TemporaryEvent)
	{
		UMVVMBlueprintViewEvent* EventPtr = Event->Event.Get();
		UEdGraph* WrapperGraph = EventPtr->GetOrCreateWrapperGraph();
		if (WrapperGraph == nullptr)
		{
			AddMessageForEvent(EventPtr, LOCTEXT("InvalidEventGraph", "The event could not be generated."), Compiler::EMessageType::Warning, FMVVMBlueprintPinId());
			bIsCreateFunctionsStepValid = false;
			continue;
		}

		EventPtr->UpdatePinValues();

		if (EventPtr->HasOrphanedPin())
		{
			AddMessageForEvent(EventPtr, LOCTEXT("InvalidEventGraphOrphaned", "The event has an orphaned pin."), Compiler::EMessageType::Warning, FMVVMBlueprintPinId());
			bIsCreateFunctionsStepValid = false;
			continue;
		}

		int32 Index = ValidEvents.Add(Event);
		EventPtr->UpdateEventKey(FMVVMViewClass_EventKey(Index));
	}
}

void FMVVMViewBlueprintCompiler::CategorizeConditions(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context)
{
	TArray<TSharedRef<FCompilerCondition>> TemporaryConditions = MoveTemp(ValidConditions);
	ValidConditions.Reset(TemporaryConditions.Num());

	for (TSharedRef<FCompilerCondition>& Condition : TemporaryConditions)
	{
		UMVVMBlueprintViewCondition* ConditionPtr = Condition->Condition.Get();
		UEdGraph* WrapperGraph = ConditionPtr->GetOrCreateWrapperGraph();
		if (WrapperGraph == nullptr)
		{
			AddMessageForCondition(ConditionPtr, LOCTEXT("InvalidConditionGraph", "The condition could not be generated."), Compiler::EMessageType::Warning, FMVVMBlueprintPinId());
			bIsCreateFunctionsStepValid = false;
			continue;
		}

		ConditionPtr->UpdatePinValues();

		if (ConditionPtr->HasOrphanedPin())
		{
			AddMessageForCondition(ConditionPtr, LOCTEXT("InvalidConditionGraphOrphaned", "The condition has an orphaned pin."), Compiler::EMessageType::Warning, FMVVMBlueprintPinId());
			bIsCreateFunctionsStepValid = false;
			continue;
		}

		int32 Index = ValidConditions.Add(Condition);
		ConditionPtr->UpdateConditionKey(FMVVMViewClass_ConditionKey(Index));
	}
}

void FMVVMViewBlueprintCompiler::CreateWriteFieldContexts(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context)
{
	// Use the Skeleton class. The class bind and not all functions are generated yet
	UWidgetBlueprintGeneratedClass* NewSkeletonClass = Cast<UWidgetBlueprintGeneratedClass>(WidgetBlueprintCompilerContext.Blueprint->SkeletonGeneratedClass);
	if (NewSkeletonClass == nullptr)
	{
		WidgetBlueprintCompilerContext.MessageLog.Error(*LOCTEXT("InvalidNewClass", "Internal error. The skeleton class is not valid.").ToString());
		bIsCreateFunctionsStepValid = false;
		return;
	}

	for (TSharedRef<FCompilerBinding>& ValidBinding : ValidBindings)
	{
		const FMVVMBlueprintViewBinding& Binding = *BlueprintView->GetBindingAt(ValidBinding->Key.ViewBindingIndex);

		if (ValidBinding->Type == FCompilerBinding::EType::Assignment
			|| ValidBinding->Type == FCompilerBinding::EType::SimpleConversionFunction
			|| ValidBinding->Type == FCompilerBinding::EType::ComplexConversionFunction)
		{
			const FMVVMBlueprintPropertyPath& DestinationPropertyPath = ValidBinding->Key.bIsForwardBinding ? Binding.DestinationPath : Binding.SourcePath;
			TValueOrError<FCreateFieldsResult, FText> FieldContextResult = CreateFieldContext(NewSkeletonClass, DestinationPropertyPath, false);
			if (FieldContextResult.HasError())
			{
				AddMessageForBinding(Binding, FieldContextResult.StealError(), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
				bIsCreateFunctionsStepValid = false;
				continue;
			}

			// Test if it already exist
			TSharedPtr<FGeneratedWriteFieldPathContext> WriteFieldPath;
			{
				const TArray<UE::MVVM::FMVVMConstFieldVariant>& SkeletalGeneratedFieldsResult = FieldContextResult.GetValue().SkeletalGeneratedFields;
				TSharedRef<FGeneratedWriteFieldPathContext>* Found = GeneratedWriteFieldPaths.FindByPredicate([&SkeletalGeneratedFieldsResult](const TSharedRef<FGeneratedWriteFieldPathContext>& Other)
					{
						return Other->SkeletalGeneratedFields == SkeletalGeneratedFieldsResult;
					});
				if (Found)
				{
					WriteFieldPath = *Found;

					// Backward binding are reactive and are not trigger on initialization.
					//It is valid for a backward binding to reuse the same write field path.
					if (ValidBinding->Key.bIsForwardBinding)
					{
						AddMessageForBinding(Binding
							, FText::Format(LOCTEXT("PropertyPathAlreadyUsed", "Note: the property path '{0}' is already used by another binding. Binding initialization order may not be deterministic."), PropertyPathToText(NewSkeletonClass, BlueprintView.Get(), DestinationPropertyPath))
							, Compiler::EMessageType::Info
							, FMVVMBlueprintPinId()
						);
					}
				}
				else
				{
					WriteFieldPath = MakeWriteFieldPath(
						FieldContextResult.GetValue().GeneratedFrom
						, MoveTemp(FieldContextResult.GetValue().GeneratedFields)
						, MoveTemp(FieldContextResult.GetValue().SkeletalGeneratedFields));
					GeneratedWriteFieldPaths.Add(WriteFieldPath.ToSharedRef());
				}
			}

			WriteFieldPath->UsedByBindings.AddUnique(ValidBinding);
			WriteFieldPath->bUseByNativeBinding = true; // the setter function can be in BP but the destination will be set in native

			// Assign the Destination to the binding
			ValidBinding->WritePath = WriteFieldPath;
		}
	}

	//The destination for event must also exist (for the BP to compile).
	for (TSharedRef<FCompilerEvent>& ValidEvent : ValidEvents)
	{
		UMVVMBlueprintViewEvent* EventPtr = ValidEvent->Event.Get();
		check(EventPtr);

		TValueOrError<FCreateFieldsResult, FText> FieldContextResult = CreateFieldContext(NewSkeletonClass, EventPtr->GetDestinationPath(), false);
		if (FieldContextResult.HasError())
		{
			AddMessageForEvent(EventPtr
				, FText::Format(Private::PropertyPathIsInvalidFormat, PropertyPathToText(NewSkeletonClass, BlueprintView.Get(), EventPtr->GetDestinationPath()))
				, Compiler::EMessageType::Error
				, FMVVMBlueprintPinId()
			);
			bIsCreateFunctionsStepValid = false;
			continue;
		}

		// Test if it already exist
		TSharedPtr<FGeneratedWriteFieldPathContext> WriteFieldPath;
		{
			const TArray<UE::MVVM::FMVVMConstFieldVariant>& SkeletalGeneratedFieldsResult = FieldContextResult.GetValue().SkeletalGeneratedFields;
			TSharedRef<FGeneratedWriteFieldPathContext>* Found = GeneratedWriteFieldPaths.FindByPredicate([&SkeletalGeneratedFieldsResult](const TSharedRef<FGeneratedWriteFieldPathContext>& Other) { return Other->SkeletalGeneratedFields == SkeletalGeneratedFieldsResult; });
			if (Found)
			{
				WriteFieldPath = (*Found);
			}
			else
			{
				WriteFieldPath = MakeWriteFieldPath(
					FieldContextResult.GetValue().GeneratedFrom
					, MoveTemp(FieldContextResult.GetValue().GeneratedFields)
					, MoveTemp(FieldContextResult.GetValue().SkeletalGeneratedFields));
				GeneratedWriteFieldPaths.Add(WriteFieldPath.ToSharedRef());
			}
		}

		WriteFieldPath->UsedByEvents.AddUnique(ValidEvent);
		ValidEvent->WritePath = WriteFieldPath;
	}
}


void FMVVMViewBlueprintCompiler::CreateViewModelSetters(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context)
{
	if (GetDefault<UMVVMDeveloperProjectSettings>()->bAllowGeneratedViewModelSetter)
	{
		for (FCompilerViewModelSetter& Setter : ViewModelSettersToGenerate)
		{
			if (ensure(Setter.SetterGraph != nullptr))
			{
				if (!UE::MVVM::FunctionGraphHelper::GenerateViewModelSetter(WidgetBlueprintCompilerContext, Setter.SetterGraph, Setter.PropertyName))
				{
					AddMessageForViewModel(Setter.DisplayName, LOCTEXT("SetterFunctionCouldNotBeGenerated", "The setter function could not be generated."), Compiler::EMessageType::Warning);
					bIsCreateFunctionsStepValid = false;
					continue;
				}
			}
		}
	}
}


void FMVVMViewBlueprintCompiler::CreateIntermediateGraphFunctions(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context)
{
	// Add function to set destination if needed
	for (TSharedRef<FGeneratedWriteFieldPathContext>& GeneratedDestination : GeneratedWriteFieldPaths)
	{
		// If the destination can't be set in cpp, we need to generate a BP function to set the value.
		if (!GeneratedDestination->bCanBeSetInNative && GeneratedDestination->bUseByNativeBinding)
		{
			auto AddErrorMessage = [Self = this, &GeneratedDestination](const FText& ErrorMsg)
			{
				Self->AddMessages(GeneratedDestination->UsedByBindings, GeneratedDestination->UsedByEvents, ErrorMsg, Compiler::EMessageType::Error);
			};

			const FProperty* SetterProperty = nullptr;
			if (ensure(GeneratedDestination->SkeletalGeneratedFields.Num() > 0 && GeneratedDestination->SkeletalGeneratedFields.Last().IsProperty()))
			{
				SetterProperty = GeneratedDestination->SkeletalGeneratedFields.Last().GetProperty();
			}

			if (SetterProperty == nullptr)
			{
				AddErrorMessage(FText::Format(LOCTEXT("CantGetSetter", "Internal Error. The setter function was not created. {0}"), ::UE::MVVM::FieldPathHelper::ToText(GeneratedDestination->GeneratedFields)));
				bIsCreateFunctionsStepValid = false;
				continue;
			}

			// create a setter function to be called from native. For now we follow the convention of Setter(Conversion(Getter))
			UEdGraph* GeneratedSetterGraph = UE::MVVM::FunctionGraphHelper::CreateIntermediateFunctionGraph(WidgetBlueprintCompilerContext, FString::Printf(TEXT("__Setter_%s"), *SetterProperty->GetName()), EFunctionFlags::FUNC_None, TEXT("AutogeneratedSetter"), false);
			if (GeneratedSetterGraph == nullptr)
			{
				AddErrorMessage(FText::Format(LOCTEXT("CantCreateSetter", "Internal Error. The setter function was not created. {0}"), ::UE::MVVM::FieldPathHelper::ToText(GeneratedDestination->GeneratedFields)));
				bIsCreateFunctionsStepValid = false;
				continue;
			}

			UE::MVVM::FunctionGraphHelper::AddFunctionArgument(GeneratedSetterGraph, SetterProperty, "NewValue");

			// set GeneratedSetterFunction. Use the SkeletalSetterPath here to use the setter will be generated when the function is generated.
			if (!UE::MVVM::FunctionGraphHelper::GenerateIntermediateSetter(WidgetBlueprintCompilerContext, GeneratedSetterGraph, GeneratedDestination->SkeletalGeneratedFields))
			{
				AddErrorMessage(FText::Format(LOCTEXT("CantGeneratedSetter", "Internal Error. The setter function was not generated. {0}"), ::UE::MVVM::FieldPathHelper::ToText(GeneratedDestination->GeneratedFields)));
				bIsCreateFunctionsStepValid = false;
				continue;
			}

			// the new path can only be set later, once the function is compiled.
			GeneratedDestination->GeneratedFunctionName = GeneratedSetterGraph->GetFName();
			GeneratedFunctions.Add(GeneratedSetterGraph->GetFName());
		}
	}

	// Add Generated Conversion functions to the blueprint
	for (const TSharedRef<FCompilerBinding>& ValidBinding : ValidBindings)
	{
		FMVVMBlueprintViewBinding& Binding = *BlueprintView->GetBindingAt(ValidBinding->Key.ViewBindingIndex);

		if (ValidBinding->Type == FCompilerBinding::EType::ComplexConversionFunction)
		{
			UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(ValidBinding->Key.bIsForwardBinding);
			check(ConversionFunction);

			UEdGraph* WrapperGraph = ConversionFunction->GetOrCreateIntermediateWrapperGraph(WidgetBlueprintCompilerContext);
			if (ensure(WrapperGraph) && ConversionFunction->IsWrapperGraphTransient())
			{
				bool bAlreadyContained = WidgetBlueprintCompilerContext.Blueprint->FunctionGraphs.Contains(WrapperGraph);
				if (ensure(!bAlreadyContained))
				{
					if (ConversionFunction->IsUbergraphPage())
					{
						// Calls to Latent Functions require a mapping to a stable, non-transient UObject for Latent Manager to use as UUID for execution
						WidgetBlueprintCompilerContext.MessageLog.NotifyIntermediateObjectCreation(ConversionFunction->GetWrapperNode(), ConversionFunction->GetLatentNodeUUID());
						Context.AddGeneratedUbergraphPage(WrapperGraph);
					}
					else
					{
						Context.AddGeneratedFunctionGraph(WrapperGraph);
					}
				}

				if (ConversionFunction->IsUbergraphPage())
				{
					// Append the delegate signature suffix, note that we can't name the graph this as the delegate itself uses this name.
					GeneratedFunctions.Add(UE::MVVM::BindingHelper::GetDelegateSignatureName(WrapperGraph->GetFName()));
				}
				else
				{
					GeneratedFunctions.Add(WrapperGraph->GetFName());
				}
			}
		}
	}

	// Add Generated event to the blueprint
	for (TSharedRef<FCompilerEvent>& Event : ValidEvents)
	{
		UMVVMBlueprintViewEvent* EventPtr = Event->Event.Get();
		UEdGraph* WrapperGraph = EventPtr->GetOrCreateWrapperGraph();
		if (ensure(WrapperGraph))
		{
			bool bAlreadyContained = WidgetBlueprintCompilerContext.Blueprint->FunctionGraphs.Contains(WrapperGraph);
			if (ensure(!bAlreadyContained))
			{
				Context.AddGeneratedFunctionGraph(WrapperGraph);
				GeneratedFunctions.Add(WrapperGraph->GetFName());
			}
		}
	}

	// Add Generated condition to the blueprint
	for (TSharedRef<FCompilerCondition>& Condition : ValidConditions)
	{
		UMVVMBlueprintViewCondition* ConditionPtr = Condition->Condition.Get();
		UEdGraph* WrapperGraph = ConditionPtr->GetOrCreateWrapperGraph();
		if (ensure(WrapperGraph))
		{
			bool bAlreadyContained = WidgetBlueprintCompilerContext.Blueprint->FunctionGraphs.Contains(WrapperGraph);
			if (ensure(!bAlreadyContained))
			{
				Context.AddGeneratedFunctionGraph(WrapperGraph);
				GeneratedFunctions.Add(WrapperGraph->GetFName());
			}
		}
	}

}

void FMVVMViewBlueprintCompiler::CategorizeAsyncFunctions(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context)
{
	// Populate source binding keys for async functions
	int32 Index = 0;
	for (const TSharedRef<FCompilerBinding>& ValidBinding : ValidBindings)
	{
		FMVVMBlueprintViewBinding& Binding = *BlueprintView->GetBindingAt(ValidBinding->Key.ViewBindingIndex);

		if (ValidBinding->Type == FCompilerBinding::EType::ComplexConversionFunction)
		{
			UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(ValidBinding->Key.bIsForwardBinding);
			if (ensure(ConversionFunction) && ConversionFunction->IsUbergraphPage())
			{
				UEdGraph* WrapperGraph = ConversionFunction->GetOrCreateIntermediateWrapperGraph(WidgetBlueprintCompilerContext);
				if (ensure(WrapperGraph))
				{
					TArray<UMVVMK2Node_AreSourcesValidForBinding*> AreSourcesValidNodes;
					WrapperGraph->GetNodesOfClass<UMVVMK2Node_AreSourcesValidForBinding>(AreSourcesValidNodes);

					// Like for events, this is a temp index to avoid log errors. The real index will get set after compilation
					for (UMVVMK2Node_AreSourcesValidForBinding* AreSourcesValidNode : AreSourcesValidNodes)
					{
						AreSourcesValidNode->BindingKey = FMVVMViewClass_BindingKey(Index);
					}
				}
			}
		}

		Index++;
	}
}


bool FMVVMViewBlueprintCompiler::PreCompile(UWidgetBlueprintGeneratedClass* Class)
{
	if (!AreStepsValid())
	{
		return false;
	}

	FixCompilerBindingSelfSource(Class);
	AddWarningForPropertyWithMVVMAndLegacyBinding(Class);

	GeneratedReadFieldPaths.Reset();
	CreateReadFieldContexts(Class);
	CreateCreatorContentFromBindingSource(Class);

	// NB. The dynamic sources are created.
	FixFieldPathContext(Class);

	if (!AreStepsValid())
	{
		return false;
	}

	PreCompileViewModelCreatorContexts(Class);
	PreCompileBindings(Class);
	PreCompileEvents(Class);
	PreCompileConditions(Class);
	PreCompileViewExtensions(Class);
	PreCompileSourceDependencies(Class);

	return AreStepsValid();
}


bool FMVVMViewBlueprintCompiler::Compile(UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension)
{
	if (!AreStepsValid())
	{
		return false;
	}

	TValueOrError<FCompiledBindingLibraryCompiler::FCompileResult, FText> CompileResult = BindingLibraryCompiler.Compile(BlueprintView->GetCompiledBindingLibraryId());
	if (CompileResult.HasError())
	{
		WidgetBlueprintCompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("BindingCompilationFailed", "The binding compilation failed. {1}"), CompileResult.GetError()).ToString());
		return false;
	}
	CompileSources(CompileResult.GetValue(), Class, ViewExtension);
	CompileBindings(CompileResult.GetValue(), Class, ViewExtension);
	CompileEvaluateSources(CompileResult.GetValue(), Class, ViewExtension);
	CompileEvents(CompileResult.GetValue(), Class, ViewExtension);
	CompileConditions(CompileResult.GetValue(), Class, ViewExtension);
	CompileViewExtensions(CompileResult.GetValue(), Class, ViewExtension);
	SortSourceFields(CompileResult.GetValue(), Class, ViewExtension);

	{
		ViewExtension->bInitializeSourcesOnConstruct = BlueprintView->GetSettings()->bInitializeSourcesOnConstruct;
		ViewExtension->bInitializeBindingsOnConstruct = ViewExtension->bInitializeSourcesOnConstruct ? BlueprintView->GetSettings()->bInitializeBindingsOnConstruct : false;
		ViewExtension->bInitializeEventsOnConstruct = BlueprintView->GetSettings()->bInitializeEventsOnConstruct;
		ViewExtension->bListenToViewModelCollectionChanged = Algo::AnyOf(ViewExtension->Sources, [](const FMVVMViewClass_Source& Source){ return Source.RequireGlobalViewModelCollectionUpdate(); });
		ViewExtension->bCreateViewWithoutBindings = BlueprintView->GetSettings()->bCreateViewWithoutBindings;
	}
	{
		ViewExtension->OptionalSources = 0;
		for (int32 Index = 0; Index < ViewExtension->Sources.Num(); ++Index)
		{
			const FMVVMViewClass_Source& Source = ViewExtension->Sources[Index];
			if (Source.IsOptional())
			{
				FMVVMViewClass_SourceKey ClassSourceKey = FMVVMViewClass_SourceKey(Index);
				ViewExtension->OptionalSources |= ClassSourceKey.GetBit();
			}
		}
	}

	bool bResult = AreStepsValid();
	if (bResult)
	{
		ViewExtension->BindingLibrary = MoveTemp(CompileResult.GetValue().Library);

#if UE_WITH_MVVM_DEBUGGING
		if (CVarLogViewCompiledResult->GetBool())
		{
			UMVVMViewClass::FToStringArgs ToStringArgs = UMVVMViewClass::FToStringArgs::All();
			ToStringArgs.Source.bUseDisplayName = false;
			ToStringArgs.Binding.bUseDisplayName = false;
			ToStringArgs.Evaluate.bUseDisplayName = false;
			ToStringArgs.Event.bUseDisplayName = false;
			UE_LOGF(LogMVVM, Log, "%ls", *ViewExtension->ToString(ToStringArgs));
		}
#endif
	}

	return AreStepsValid();
}


TArray<FName> FMVVMViewBlueprintCompiler::GetGeneratedFunctions() const
{
	return GeneratedFunctions;
}


void FMVVMViewBlueprintCompiler::FixFieldPathContext(UWidgetBlueprintGeneratedClass* Class)
{
	TSharedRef<FCompilerBindingSource>* SelfSourcePtr = NeededBindingSources.FindByPredicate([](const TSharedRef<FCompilerBindingSource>& Other)
		{
			return Other->Type == FCompilerBindingSource::EType::Self;
		});

	auto FindProperty = [](FMVVMConstFieldVariant Field) -> const FProperty*
	{
		if (Field.IsProperty())
		{
			return Field.GetProperty();
		}
		else if (Field.IsFunction() && Field.GetFunction())
		{
			return BindingHelper::GetReturnProperty(Field.GetFunction());
		}
		return nullptr;
	};

	auto FindDynamicCreatorContext = [Self = this, &FindProperty](TArray<UE::MVVM::FMVVMConstFieldVariant>& GeneratedFields, int32 Index, TSharedPtr<FCompilerBindingSource>& InOutSource) -> bool
	{
		bool bBreak = true;

		UE::MVVM::FMVVMConstFieldVariant NextField = GeneratedFields[Index];
		if (const FObjectProperty* FieldObjectProperty = CastField<FObjectProperty>(FindProperty(NextField)))
		{
			const UClass* NextFieldObjectPtrClass = FieldObjectProperty->PropertyClass;
			for (const TSharedRef<FCompilerSourceViewModelDynamicCreatorContext>& DynamicCreatorContext : Self->SourceViewModelDynamicCreatorContexts)
			{
				if (DynamicCreatorContext->ParentSource == InOutSource
					&& DynamicCreatorContext->NotificationId.GetFieldName() == NextField.GetName()
					&& DynamicCreatorContext->Source->AuthoritativeClass == NextFieldObjectPtrClass)
				{
					InOutSource = DynamicCreatorContext->Source;
					bBreak = false;
					break;
				}
			}
		}

		return bBreak;
	};

	//// Sanity check. Set Source to FGeneratedReadFieldPathContext
	//{
	//	for (TSharedRef<FGeneratedReadFieldPathContext>& GeneratedRead : GeneratedReadFieldPaths)
	//	{
	//		// ReadFieldPath should already be valid. Only confirm it here.
	//		TSharedPtr<FCompilerBindingSource> NewSource = SelfSourcePtr ? *SelfSourcePtr : TSharedPtr<FCompilerBindingSource>();
	//		if (GeneratedRead->GeneratedFields.Num() > 0)
	//		{
	//			UE::MVVM::FMVVMConstFieldVariant NextField = GeneratedRead->GeneratedFields[0];
	//			if (const FObjectProperty* FieldObjectProperty = CastField<FObjectProperty>(FindProperty(NextField)))
	//			{
	//				UClass* NextFieldObjectPtrClass = FieldObjectProperty->PropertyClass;
	//				TSharedRef<FCompilerBindingSource>* BindingSource = NeededBindingSources.FindByPredicate(
	//					[NextField, NextFieldObjectPtrClass](TSharedRef<FCompilerBindingSource>& Other)
	//					{
	//						return Other->Name == NextField.GetName()
	//							&& NextFieldObjectPtrClass == Other->AuthoritativeClass;
	//					}
	//				);

	//				if (BindingSource)
	//				{
	//					NewSource = *BindingSource;
	//					for (int32 GeneratedFieldIndex = 1; GeneratedFieldIndex < GeneratedRead->GeneratedFields.Num() - 1; ++GeneratedFieldIndex)
	//					{
	//						if (FindDynamicCreatorContext(GeneratedRead->GeneratedFields, GeneratedFieldIndex, NewSource))
	//						{
	//							break;
	//						}
	//					}
	//				}
	//			}
	//		}

	//		if (GeneratedRead->Source != NewSource)
	//		{
	//			// We only do this test as a sanity check and because it's the same algo for FGeneratedWriteFieldPathContext 
	//			AddMessage(FText::Format(LOCTEXT("InternalErrorNotSameSource", "Internal error. The read path {0} source do not matches with the previous calculated source."), ::UE::MVVM::FieldPathHelper::ToText(GeneratedRead->GeneratedFields))
	//				, EMessageType::Info
	//			);
	//		}
	//	}
	//}

	// Set OptionalSource and OptionalDependencySource to FGeneratedWriteFieldPathContext
	for (TSharedRef<FGeneratedWriteFieldPathContext>& GeneratedDestination : GeneratedWriteFieldPaths)
	{
		//ViewmodelA.ViewmodelB = ViewmodelC.Value				-> ViewmodelA needs to init before ViewmodelA_ViewmodelB, ViewmodelA before ViewmodelC, ViewmodelC before ViewmodelA_ViewmodelB
			// OptionSource = ViewmodelA
			// OptionalDependencySource = ViewmodelA_ViewmodelB. We do not have the info for ViewmodelC (only in the binding) and there could be more than one read path. Set the flag and add the dependency later.
		//ViewmodelA.ViewmodelB.Value = ViewmodelC.Value		-> ViewmodelA needs to init before ViewmodelA_ViewmodelB, ViewmodelA before ViewmodelC, ViewmodelA_ViewmodelB before ViewmodelC
			// OptionSource = ViewmodelA_ViewmodelB
			// OptionalDependencySource = null. We set a value, not a source.

		if (GeneratedDestination->GeneratedFields.Num() > 0)
		{
			UE::MVVM::FMVVMConstFieldVariant NextField = GeneratedDestination->GeneratedFields[0];
			if (const FObjectProperty* FieldObjectProperty = CastField<FObjectProperty>(FindProperty(NextField)))
			{
				const UClass* NextFieldObjectPtrClass = FieldObjectProperty->PropertyClass;
				TSharedRef<FCompilerBindingSource>* BindingSource = NeededBindingSources.FindByPredicate(
					[NextField, NextFieldObjectPtrClass](TSharedRef<FCompilerBindingSource>& Other)
					{
						return Other->Name == NextField.GetName()
							&& NextFieldObjectPtrClass == Other->AuthoritativeClass;
					}
				);

				if (BindingSource)
				{
					if (GeneratedDestination->GeneratedFields.Num() > 1)
					{
						GeneratedDestination->OptionalSource = *BindingSource;
						int32 GeneratedFieldIndex = 1;
						for (; GeneratedFieldIndex < GeneratedDestination->GeneratedFields.Num() - 1; ++GeneratedFieldIndex)
						{
							if (FindDynamicCreatorContext(GeneratedDestination->GeneratedFields, GeneratedFieldIndex, GeneratedDestination->OptionalSource))
							{
								break;
							}
						}
						if (GeneratedFieldIndex == GeneratedDestination->GeneratedFields.Num() - 1)
						{
							FindDynamicCreatorContext(GeneratedDestination->GeneratedFields, GeneratedFieldIndex, GeneratedDestination->OptionalDependencySource);
						}
					}
					else
					{
						GeneratedDestination->OptionalSource = SelfSourcePtr ? *SelfSourcePtr : TSharedPtr<FCompilerBindingSource>();
						GeneratedDestination->OptionalDependencySource = *BindingSource;
					}
				}
			}
		}
	}

	// If a graph was generated for the FGeneratedWriteFieldPathContext, then use it instead for the SkeletalGeneratedFields
	for (TSharedRef<FGeneratedWriteFieldPathContext>& GeneratedDestination : GeneratedWriteFieldPaths)
	{
		if (!GeneratedDestination->GeneratedFunctionName.IsNone())
		{
			UFunction* GeneratedFunction = Class->FindFunctionByName(GeneratedDestination->GeneratedFunctionName);
			if (GeneratedFunction == nullptr)
			{
				WidgetBlueprintCompilerContext.MessageLog.Error(*LOCTEXT("CantFindGeneratedBindingSetterFunction", "Internal Error. The setter function was not generated.").ToString());
				bIsPreCompileStepValid = false;
				continue;
			}

			GeneratedDestination->GeneratedFields.Reset();
			GeneratedDestination->GeneratedFields.Add(UE::MVVM::FMVVMConstFieldVariant(GeneratedFunction));
			// note the function is not added on the skeletal class
			GeneratedDestination->SkeletalGeneratedFields.Reset();
			GeneratedDestination->SkeletalGeneratedFields.Add(UE::MVVM::FMVVMConstFieldVariant(GeneratedFunction));
		}
	}
}


void FMVVMViewBlueprintCompiler::CreateReadFieldContexts(UWidgetBlueprintGeneratedClass* Class)
{
	auto AlreadyExist = [Self = this](const TArray<UE::MVVM::FMVVMConstFieldVariant>& SkeletalGeneratedFields) -> TSharedPtr<FGeneratedReadFieldPathContext>
		{
			// Test if it already exist
			TSharedRef<FGeneratedReadFieldPathContext>* Found = Self->GeneratedReadFieldPaths.FindByPredicate([&SkeletalGeneratedFields](const TSharedRef<FGeneratedReadFieldPathContext>& Other) { return Other->SkeletalGeneratedFields == SkeletalGeneratedFields; });
			return Found != nullptr ? *Found : TSharedPtr<FGeneratedReadFieldPathContext>();
		};

	for (TSharedRef<FCompilerBinding>& ValidBinding : ValidBindings)
	{
		const FMVVMBlueprintViewBinding& Binding = *BlueprintView->GetBindingAt(ValidBinding->Key.ViewBindingIndex);

		auto CreateBindingSourceContext = [Self = this, Class, &ValidBinding, &Binding](const FMVVMBlueprintPropertyPath& PropertyPath, const FMVVMBlueprintPinId& PinId) -> TValueOrError<FCreateFieldsResult, void>
		{
			TValueOrError<FCreateFieldsResult, FText> FieldContextResult = Self->CreateFieldContext(Class, PropertyPath, true);
			if (FieldContextResult.HasError())
			{
				Self->AddMessageForBinding(Binding
					, FText::Format(Private::PropertyPathIsInvalidFormat, PropertyPathToText(Class, Self->BlueprintView.Get(), PropertyPath))
					, Compiler::EMessageType::Error
					, PinId
				);
				Self->bIsPreCompileStepValid = false;
				return MakeError();
			}

			return MakeValue(FieldContextResult.StealValue());
		};

		auto CreateFieldId = [Self = this, Class, &Binding](const FMVVMBlueprintPropertyPath& PropertyPath, TSharedPtr<FGeneratedReadFieldPathContext>& ReadFieldContext, const FMVVMBlueprintPinId& PinId) -> TValueOrError<void, void>
		{
			if (!ReadFieldContext->NotificationField.IsValid())
			{
				TValueOrError<TSharedPtr<FCompilerNotifyFieldId>, FText> CreateFieldResult = Self->CreateNotifyFieldId(Class, ReadFieldContext);
				if (CreateFieldResult.HasError())
				{
					Self->AddMessageForBinding(Binding
						, FText::Format(LOCTEXT("CreateNotifyFieldIdFailedInvalidSelfContext", "The property path '{0}' is invalid. {1}"), PropertyPathToText(Class, Self->BlueprintView.Get(), PropertyPath), CreateFieldResult.StealError())
						, Compiler::EMessageType::Error
						, PinId
					);
					return MakeError();
				}

				if (CreateFieldResult.GetValue())
				{
					// Sanity check
					{
						// if there is a FieldId associated with the read property
						if (CreateFieldResult.GetValue()->Source)
						{
							EMVVMBlueprintFieldPathSource PathSource = ReadFieldContext->GeneratedFrom;
							bool bValidViewModel = CreateFieldResult.GetValue()->Source->Type == FCompilerBindingSource::EType::ViewModel && PathSource == EMVVMBlueprintFieldPathSource::ViewModel;
							bool bValidWidget = CreateFieldResult.GetValue()->Source->Type == FCompilerBindingSource::EType::Widget && PathSource == EMVVMBlueprintFieldPathSource::Widget;
							bool bDynamic = CreateFieldResult.GetValue()->Source->Type == FCompilerBindingSource::EType::DynamicViewmodel;
							bool bSelf = CreateFieldResult.GetValue()->Source->Type == FCompilerBindingSource::EType::Self && PathSource == EMVVMBlueprintFieldPathSource::SelfContext;
							bool bIsComponentOnSelf = false;
							if (ReadFieldContext->bIsComponent)
							{
								const UWidgetBlueprintGeneratedClass* ComponentOwningWidgetBlueprint = UE::MVVM::FieldPathHelper::GetComponentPropertyPathSource(ReadFieldContext->SkeletalGeneratedFields, Class);
								bIsComponentOnSelf = ComponentOwningWidgetBlueprint == Class;
							}
							bIsComponentOnSelf = bIsComponentOnSelf && CreateFieldResult.GetValue()->Source->Type == FCompilerBindingSource::EType::Self && PathSource == EMVVMBlueprintFieldPathSource::Widget;
							if (!(bValidViewModel || bValidWidget || bDynamic || bSelf || bIsComponentOnSelf))
							{
								Self->AddMessageForBinding(Binding
									, FText::Format(LOCTEXT("CreateNotifyFieldIdFailedInvalidInvalidContext", "Internal error. The property path '{0}' is invalid. The context is invalid."), PropertyPathToText(Class, Self->BlueprintView.Get(), PropertyPath))
									, Compiler::EMessageType::Error
									, PinId
								);
								return MakeError();
							}
						}
					}

					ReadFieldContext->NotificationField = CreateFieldResult.GetValue();
					ReadFieldContext->Source = ReadFieldContext->NotificationField->Source;
				}
			}
			return MakeValue();
		};

		if (ValidBinding->Type == FCompilerBinding::EType::Assignment)
		{
			const FMVVMBlueprintPropertyPath& BindingSourcePath = ValidBinding->Key.bIsForwardBinding ? Binding.SourcePath : Binding.DestinationPath;
			TValueOrError<FCreateFieldsResult, void> CreateSourceResult = CreateBindingSourceContext(BindingSourcePath, FMVVMBlueprintPinId());
			if (CreateSourceResult.HasError())
			{
				continue;
			}

			TSharedPtr<FGeneratedReadFieldPathContext> Found = AlreadyExist(CreateSourceResult.GetValue().SkeletalGeneratedFields);
			if (!Found)
			{
				Found = MakeShared<FGeneratedReadFieldPathContext>();
				Found->Source = MoveTemp(CreateSourceResult.GetValue().OptionalSource);
				Found->GeneratedFields = MoveTemp(CreateSourceResult.GetValue().GeneratedFields);
				Found->SkeletalGeneratedFields = MoveTemp(CreateSourceResult.GetValue().SkeletalGeneratedFields);
				Found->GeneratedFrom = MoveTemp(CreateSourceResult.GetValue().GeneratedFrom);
				Found->bIsComponent = MoveTemp(CreateSourceResult.GetValue().bIsComponent);

				if (CreateFieldId(BindingSourcePath, Found, FMVVMBlueprintPinId()).HasError())
				{
					bIsPreCompileStepValid = false;
					continue;
				}

				GeneratedReadFieldPaths.Add(Found.ToSharedRef());
			}

			Found->UsedByBindings.AddUnique(ValidBinding);
			ValidBinding->ReadPaths.Add(Found.ToSharedRef());
		}
		else if (ValidBinding->Type == FCompilerBinding::EType::SimpleConversionFunction
			|| ValidBinding->Type == FCompilerBinding::EType::ComplexConversionFunction)
		{
			UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(ValidBinding->Key.bIsForwardBinding);
			if (ensure(ConversionFunction))
			{
				for (const FMVVMBlueprintPin& Pin : ConversionFunction->GetPins())
				{
					if (Pin.UsedPathAsValue())
					{
						TValueOrError<FCreateFieldsResult, void> CreateSourceResult = CreateBindingSourceContext(Pin.GetPath(), Pin.GetId());
						if (CreateSourceResult.HasError())
						{
							continue;
						}

						TSharedPtr<FGeneratedReadFieldPathContext> Found = AlreadyExist(CreateSourceResult.GetValue().SkeletalGeneratedFields);
						if (!Found)
						{
							Found = MakeShared<FGeneratedReadFieldPathContext>();
							Found->Source = MoveTemp(CreateSourceResult.GetValue().OptionalSource);
							Found->GeneratedFields = MoveTemp(CreateSourceResult.GetValue().GeneratedFields);
							Found->SkeletalGeneratedFields = MoveTemp(CreateSourceResult.GetValue().SkeletalGeneratedFields);
							Found->GeneratedFrom = MoveTemp(CreateSourceResult.GetValue().GeneratedFrom);
							Found->bIsComponent = MoveTemp(CreateSourceResult.GetValue().bIsComponent);

							if (CreateFieldId(Pin.GetPath(), Found, Pin.GetId()).HasError())
							{
								bIsPreCompileStepValid = false;
								continue;
							}

							GeneratedReadFieldPaths.Add(Found.ToSharedRef());
						}

						Found->UsedByBindings.AddUnique(TWeakPtr<FCompilerBinding>(ValidBinding));
						ValidBinding->ReadPaths.Add(Found.ToSharedRef());
					}
				}
			}
		}
	}

	//The pins for event
	for (TSharedRef<FCompilerEvent>& ValidEvent : ValidEvents)
	{
		UMVVMBlueprintViewEvent* EventPtr = ValidEvent->Event.Get();
		check(EventPtr);
		for (const FMVVMBlueprintPin& Pin : EventPtr->GetPins())
		{
			if (Pin.UsedPathAsValue())
			{
				TValueOrError<FCreateFieldsResult, FText> CreateSourceResult = CreateFieldContext(Class, Pin.GetPath(), true);
				if (CreateSourceResult.HasError())
				{
					AddMessageForEvent(ValidEvent
						, FText::Format(Private::PropertyPathIsInvalidFormat, PropertyPathToText(Class, BlueprintView.Get(), Pin.GetPath()))
						, Compiler::EMessageType::Error
						, Pin.GetId()
					);
					bIsPreCompileStepValid = false;
					continue;
				}

				TSharedPtr<FGeneratedReadFieldPathContext> Found = AlreadyExist(CreateSourceResult.GetValue().SkeletalGeneratedFields);
				if (!Found)
				{
					Found = MakeShared<FGeneratedReadFieldPathContext>();
					Found->Source = MoveTemp(CreateSourceResult.GetValue().OptionalSource);
					Found->GeneratedFields = MoveTemp(CreateSourceResult.GetValue().GeneratedFields);
					Found->GeneratedFrom = MoveTemp(CreateSourceResult.GetValue().GeneratedFrom);
					Found->SkeletalGeneratedFields = MoveTemp(CreateSourceResult.GetValue().SkeletalGeneratedFields);
					Found->bIsComponent = MoveTemp(CreateSourceResult.GetValue().bIsComponent);
					GeneratedReadFieldPaths.Add(Found.ToSharedRef());
				}

				Found->UsedByEvents.AddUnique(ValidEvent);
				ValidEvent->ReadPaths.Add(Found.ToSharedRef());
			}
		}
	}

	//The pins for condition
	for (TSharedRef<FCompilerCondition>& ValidCondition : ValidConditions)
	{
		UMVVMBlueprintViewCondition* ConditionPtr = ValidCondition->Condition.Get();
		check(ConditionPtr);

		{
			TValueOrError<FCreateFieldsResult, FText> CreateConditionSourceResult = CreateFieldContext(Class, ConditionPtr->GetConditionPath(), true);
			if (CreateConditionSourceResult.HasError())
			{
				AddMessageForCondition(ValidCondition
					, FText::Format(Private::PropertyPathIsInvalidFormat, PropertyPathToText(Class, BlueprintView.Get(), ConditionPtr->GetConditionPath()))
					, Compiler::EMessageType::Error
					, FMVVMBlueprintPinId());
				bIsPreCompileStepValid = false;
				continue;
			}

			TSharedPtr<FGeneratedReadFieldPathContext> FoundReadFieldPath = AlreadyExist(CreateConditionSourceResult.GetValue().SkeletalGeneratedFields);
			if (!FoundReadFieldPath)
			{
				FoundReadFieldPath = MakeShared<FGeneratedReadFieldPathContext>();
				FoundReadFieldPath->Source = MoveTemp(CreateConditionSourceResult.GetValue().OptionalSource);
				FoundReadFieldPath->GeneratedFrom = MoveTemp(CreateConditionSourceResult.GetValue().GeneratedFrom);
				FoundReadFieldPath->GeneratedFields = MoveTemp(CreateConditionSourceResult.GetValue().GeneratedFields);
				FoundReadFieldPath->SkeletalGeneratedFields = MoveTemp(CreateConditionSourceResult.GetValue().SkeletalGeneratedFields);
				FoundReadFieldPath->bIsComponent = MoveTemp(CreateConditionSourceResult.GetValue().bIsComponent);

				GeneratedReadFieldPaths.Add(FoundReadFieldPath.ToSharedRef());
			}

			if (!FoundReadFieldPath->NotificationField.IsValid())
			{
				TValueOrError<TSharedPtr<FCompilerNotifyFieldId>, FText> CreateFieldResult = CreateNotifyFieldId(Class, FoundReadFieldPath);
				if (CreateFieldResult.HasError())
				{
					AddMessageForCondition(ValidCondition
						, FText::Format(LOCTEXT("CreateNotifyFieldIdFailedInvalidSelfContext", "The property path '{0}' is invalid. {1}"), PropertyPathToText(Class, BlueprintView.Get(), ConditionPtr->GetConditionPath()), CreateFieldResult.StealError())
						, Compiler::EMessageType::Error
						, FMVVMBlueprintPinId()
					);
					bIsPreCompileStepValid = false;
					continue;
				}

				if (CreateFieldResult.GetValue())
				{
					// Sanity check
					{
						// if there is a FieldId associated with the read property
						if (CreateFieldResult.GetValue()->Source)
						{
							EMVVMBlueprintFieldPathSource PathSource = FoundReadFieldPath->GeneratedFrom;
							bool bValidViewModel = CreateFieldResult.GetValue()->Source->Type == FCompilerBindingSource::EType::ViewModel && PathSource == EMVVMBlueprintFieldPathSource::ViewModel;
							bool bValidWidget = CreateFieldResult.GetValue()->Source->Type == FCompilerBindingSource::EType::Widget && PathSource == EMVVMBlueprintFieldPathSource::Widget;
							bool bDynamic = CreateFieldResult.GetValue()->Source->Type == FCompilerBindingSource::EType::DynamicViewmodel;
							bool bSelf = CreateFieldResult.GetValue()->Source->Type == FCompilerBindingSource::EType::Self && PathSource == EMVVMBlueprintFieldPathSource::SelfContext;
							if (!(bValidViewModel || bValidWidget || bDynamic || bSelf))
							{
								AddMessageForCondition(ValidCondition
									, FText::Format(LOCTEXT("CreateNotifyFieldIdFailedInvalidInvalidContext", "Internal error. The property path '{0}' is invalid. The context is invalid."), PropertyPathToText(Class, BlueprintView.Get(), ConditionPtr->GetConditionPath()))
									, Compiler::EMessageType::Error
									, FMVVMBlueprintPinId()
								);
								bIsPreCompileStepValid = false;
								continue;
							}
						}
					}

					FoundReadFieldPath->NotificationField = CreateFieldResult.GetValue();
					FoundReadFieldPath->Source = FoundReadFieldPath->NotificationField->Source;
				}
			}


			FoundReadFieldPath->UsedByConditions.AddUnique(ValidCondition);
			ValidCondition->ReadPaths.Add(FoundReadFieldPath.ToSharedRef());

			for (const FMVVMBlueprintPin& Pin : ConditionPtr->GetPins())
			{
				if (Pin.UsedPathAsValue())
				{
					TValueOrError<FCreateFieldsResult, FText> CreateSourceResult = CreateFieldContext(Class, Pin.GetPath(), true);
					if (CreateSourceResult.HasError())
					{
						AddMessageForCondition(ValidCondition
							, FText::Format(Private::PropertyPathIsInvalidFormat, PropertyPathToText(Class, BlueprintView.Get(), Pin.GetPath()))
							, Compiler::EMessageType::Error
							, Pin.GetId()
						);
						bIsPreCompileStepValid = false;
						continue;
					}
				}
			}
		}
	}
}


void FMVVMViewBlueprintCompiler::CreateCreatorContentFromBindingSource(UWidgetBlueprintGeneratedClass* Class)
{
	// Add all the needed sources that are not viewmodel (so not in the SourceCreators)
	for (const TSharedRef<FCompilerBindingSource>& Source : NeededBindingSources)
	{
		{
			const bool bSourceIsViewModel = (Source->Type == FCompilerBindingSource::EType::ViewModel || Source->Type == FCompilerBindingSource::EType::DynamicViewmodel);
			const FCompilerViewModelCreatorContext* FoundSourceCreator = ViewModelCreatorContexts.FindByPredicate([Source](const FCompilerViewModelCreatorContext& Other) { return Other.Source == Source; });
			if (bSourceIsViewModel && FoundSourceCreator == nullptr)
			{
				WidgetBlueprintCompilerContext.MessageLog.Error(*LOCTEXT("ViewmodelSourceNotAdded", "Internal error. A viewmodel was not added to the compiled list.").ToString());
				bIsPreCompileStepValid = false;
			}
		}

		const bool bSourceIsWidget = Source->Type == FCompilerBindingSource::EType::Widget || Source->Type == FCompilerBindingSource::EType::Self;
		if (bSourceIsWidget)
		{
			FCompilerWidgetCreatorContext& CompiledSourceCreator = WidgetCreatorContexts.AddDefaulted_GetRef();
			CompiledSourceCreator.Source = Source;
			CompiledSourceCreator.bSelfReference = Source->Type == FCompilerBindingSource::EType::Self;
		}
	}
}


void FMVVMViewBlueprintCompiler::FixCompilerBindingSelfSource(UWidgetBlueprintGeneratedClass* Class)
{
	int32 Counter = 0;
	for (TSharedRef<FCompilerBindingSource>& Source : NeededBindingSources)
	{
		if (Source->Type == FCompilerBindingSource::EType::Self)
		{
			ensure(Source->AuthoritativeClass == nullptr);
			Source->AuthoritativeClass = Class;
			++Counter;
		}
	}

	if (Counter > 1)
	{
		AddMessage(LOCTEXT("MoreThanSeldContext", "Internal error. There is more than self context.")
			, Compiler::EMessageType::Warning
		);
	}
}


void FMVVMViewBlueprintCompiler::AddWarningForPropertyWithMVVMAndLegacyBinding(UWidgetBlueprintGeneratedClass* Class)
{
	const TArray<FDelegateRuntimeBinding>& LegacyBindings = Class->Bindings;
	for (const TSharedRef<FGeneratedWriteFieldPathContext>& WriteFieldPath : GeneratedWriteFieldPaths)
	{
		FName MVVMObjectName;
		FName MVVMFieldName;
		if (WriteFieldPath->GeneratedFrom == EMVVMBlueprintFieldPathSource::SelfContext)
		{
			MVVMObjectName = Class->ClassGeneratedBy ? Class->ClassGeneratedBy->GetFName() : FName();
			MVVMFieldName = WriteFieldPath->SkeletalGeneratedFields.Num() > 0 ? WriteFieldPath->SkeletalGeneratedFields[0].GetName() : FName();
		}

		for (int32 Index = 0; Index < WriteFieldPath->SkeletalGeneratedFields.Num(); ++Index)
		{
			const UE::MVVM::FMVVMConstFieldVariant& Field = WriteFieldPath->SkeletalGeneratedFields[Index];
			const FObjectPropertyBase* ObjectProperty = Field.IsProperty() ? CastField<const FObjectPropertyBase>(Field.GetProperty()) : nullptr;
			if (ObjectProperty && ObjectProperty->PropertyClass->IsChildOf(UWidget::StaticClass()))
			{
				MVVMObjectName = ObjectProperty->GetFName();
				MVVMFieldName = WriteFieldPath->SkeletalGeneratedFields.IsValidIndex(Index+1) ? WriteFieldPath->SkeletalGeneratedFields[Index+1].GetName() : FName();
			}
		}

		for (const FDelegateRuntimeBinding& LegacyBinding : LegacyBindings)
		{
			if (LegacyBinding.ObjectName == MVVMObjectName && LegacyBinding.PropertyName == MVVMFieldName)
			{
				if (WriteFieldPath->UsedByBindings.Num())
				{
					AddMessages(WriteFieldPath->UsedByBindings
						, TArrayView<TWeakPtr<FCompilerEvent>>()
						, LOCTEXT("BindingConflictWithLegacy", "The binding is set on a property with legacy binding.")
						, Compiler::EMessageType::Warning
					);
				}
			}
		}
	}
}


void FMVVMViewBlueprintCompiler::PreCompileViewModelCreatorContexts(UWidgetBlueprintGeneratedClass* Class)
{
	for (FCompilerViewModelCreatorContext& SourceCreatorContext : ViewModelCreatorContexts)
	{
		const FMVVMBlueprintViewModelContext& ViewModelContext = SourceCreatorContext.ViewModelContext;
		checkf(ViewModelContext.GetViewModelClass(), TEXT("The viewmodel class is invalid. It was checked in CreateSourceList"));

		if (ViewModelContext.GetViewModelClass()->HasAllClassFlags(CLASS_Deprecated))
		{
			AddMessageForViewModel(ViewModelContext
				, FText::Format(LOCTEXT("ViewModelTypeDeprecated", "Viewmodel class '{0}' is deprecated and should not be used. Please update it in the View Models panel."), ViewModelContext.GetViewModelClass()->GetDisplayNameText())
				, Compiler::EMessageType::Warning
			);
		}

		if (!ViewModelContext.GetViewModelClass()->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
		{
			AddMessageForViewModel(ViewModelContext
				, LOCTEXT("ViewmodelInvalidInterface", "The class doesn't implement the interface NotifyFieldValueChanged.")
				, Compiler::EMessageType::Error
			);
			bIsPreCompileStepValid = false;
			continue;
		}

		// Test the creation mode. Dynamic source are always using Path.
		if (SourceCreatorContext.DynamicContext == nullptr)
		{
			if (!GetAllowedContextCreationType(ViewModelContext.GetViewModelClass()).Contains(ViewModelContext.CreationType))
			{
				AddMessageForViewModel(ViewModelContext
					, LOCTEXT("ViewModelContextCreationTypeInvalid", "It has an invalidate creation type. You can change it in the View Models panel.")
					, Compiler::EMessageType::Error
				);
				bIsPreCompileStepValid = false;
				continue;
			}
		}

		if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::Manual || ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::Context)
		{
		}
		else if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::CreateInstance)
		{
			if (ViewModelContext.GetViewModelClass()->HasAllClassFlags(CLASS_Abstract))
			{
				AddMessageForViewModel(ViewModelContext
					, FText::Format(LOCTEXT("ViewModelTypeAbstract", "Viewmodel class '{0}' is abstract and can't be created. You can change it in the View Models panel."), ViewModelContext.GetViewModelClass()->GetDisplayNameText())
					, Compiler::EMessageType::Error
				);
				bIsPreCompileStepValid = false;
				continue;
			}

			UObject* CDOInstance = Class->GetDefaultObject(true);

			// If requested, create an instance of the viewmodel to be edited through the details panel
			if (ViewModelContext.bExposeInstanceInEditor)
			{
				FObjectPropertyBase* ViewModelProperty = FindFProperty<FObjectPropertyBase>(Class, ViewModelContext.GetViewModelName());
				if (!ViewModelProperty->GetObjectPropertyValue_InContainer(CDOInstance))
				{
					UObject* StaticInstance = NewObject<UObject>(CDOInstance, ViewModelContext.GetViewModelClass(), NAME_None, RF_Transactional | RF_Public);
					BlueprintView->GetOuterUMVVMWidgetBlueprintExtension_View()->TemporaryViewModelInstances.Add(ViewModelContext.GetViewModelId(), StaticInstance);
					if (ensure(StaticInstance->GetClass()->IsChildOf(ViewModelProperty->PropertyClass)))
					{
						ViewModelProperty->SetObjectPropertyValue_InContainer(CDOInstance, StaticInstance);
					}
				}

				TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> ReadFieldPathResult = AddObjectFieldPath(BindingLibraryCompiler, Class, ViewModelContext.GetViewModelName().ToString(), ViewModelContext.GetViewModelClass());
				if (ReadFieldPathResult.HasError())
				{
					AddMessageForViewModel(ViewModelContext
						, ReadFieldPathResult.GetError()
						, Compiler::EMessageType::Error
					);
					bIsPreCompileStepValid = false;
					continue;
				}

				SourceCreatorContext.ReadPropertyPathHandle = ReadFieldPathResult.StealValue();
			}
			else
			{
				FObjectPropertyBase* ViewModelProperty = FindFProperty<FObjectPropertyBase>(Class, ViewModelContext.GetViewModelName());
				ViewModelProperty->SetObjectPropertyValue_InContainer(CDOInstance, nullptr);
			}
		}
		else if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::PropertyPath)
		{
			if (ViewModelContext.ViewModelPropertyPath.IsEmpty())
			{
				AddMessageForViewModel(ViewModelContext
					, LOCTEXT("ViewModelInvalidGetter", "Viewmodel has an invalid Getter. You can select a new one in the View Models panel.")
					, Compiler::EMessageType::Error
				);
				bIsPreCompileStepValid = true;
				continue;
			}

			TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> ReadFieldPathResult = AddObjectFieldPath(BindingLibraryCompiler, Class, ViewModelContext.ViewModelPropertyPath, ViewModelContext.GetViewModelClass());
			if (ReadFieldPathResult.HasError())
			{
				AddMessageForViewModel(ViewModelContext
					, ReadFieldPathResult.GetError()
					, Compiler::EMessageType::Error
				);
				bIsPreCompileStepValid = false;
				continue;
			}

			SourceCreatorContext.ReadPropertyPathHandle = ReadFieldPathResult.StealValue();
		}
		else if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::GlobalViewModelCollection)
		{
			if (ViewModelContext.GlobalViewModelIdentifier.IsNone())
			{
				AddMessageForViewModel(ViewModelContext
					, LOCTEXT("ViewmodelInvalidGlobalIdentifier", "Viewmodel doesn't have a valid Global identifier. You can specify a new one in the Viewmodels panel.")
					, Compiler::EMessageType::Error
				);
				bIsPreCompileStepValid = false;
				continue;
			}
		}
		else if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::Resolver)
		{
			if (!ViewModelContext.Resolver)
			{
				AddMessageForViewModel(ViewModelContext
					, LOCTEXT("ViewmodelInvalidResolver", "Viewmodel doesn't have a valid Resolver. You can specify a new one in the Viewmodels panel.")
					, Compiler::EMessageType::Error
				);
				bIsPreCompileStepValid = false;
				continue;
			}
		}
		else
		{
			AddMessageForViewModel(ViewModelContext
				, LOCTEXT("ViewmodelInvalidCreationType", "Viewmodel doesn't have a valid creation type. You can select one in the Viewmodels panel.")
				, Compiler::EMessageType::Error
			);
			bIsPreCompileStepValid = false;
			continue;
		}
	}

	for (FCompilerWidgetCreatorContext& WidgetCreator : WidgetCreatorContexts)
	{
		if (!WidgetCreator.Source->AuthoritativeClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
		{
			AddMessage(FText::Format(LOCTEXT("WidgetInvalidInterface", "The widget {0} class doesn't implement the interface NotifyFieldValueChanged."), FText::FromName(WidgetCreator.Source->Name))
				, Compiler::EMessageType::Error
			);
			bIsPreCompileStepValid = false;
			continue;
		}

		if (!WidgetCreator.bSelfReference)
		{
			TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> ReadFieldPathResult = AddObjectFieldPath(BindingLibraryCompiler, Class, WidgetCreator.Source->Name.ToString(), WidgetCreator.Source->AuthoritativeClass);
			if (ReadFieldPathResult.HasError())
			{
				AddMessage(FText::Format(LOCTEXT("WidgetAddObjectFieldPathFailFormat", "The widget {0} creator failed. {1}")
					, FText::FromName(WidgetCreator.Source->Name)
					, ReadFieldPathResult.GetError()
				)
					, Compiler::EMessageType::Error
				);
				bIsPreCompileStepValid = false;
				continue;
			}
			WidgetCreator.ReadPropertyPathHandle = ReadFieldPathResult.StealValue();
		}
	}
}


void FMVVMViewBlueprintCompiler::CompileSources(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension)
{
	struct FSortData
	{
		FSortData() = default;
		TSharedPtr<FCompilerBindingSource> Source;
		TArray<FName, TInlineAllocator<8>> Errors;
		int32 SortIndex = -1;
		bool bCalculating = false;

		int32 CalculateSortIndex(FName RequestedBy, TMap<FName, FSortData>& Map)
		{
			if (SortIndex < 0)
			{
				if (bCalculating)
				{
					SortIndex = 0;
					Errors.Add(RequestedBy);
				}
				else
				{
					bCalculating = true;
					if (Source->Dependencies.Num() == 0)
					{
						SortIndex = 0;
					}
					else
					{
						for (TWeakPtr<FCompilerBindingSource> WeakDependency : Source->Dependencies)
						{
							TSharedPtr<FCompilerBindingSource> Dependency = WeakDependency.Pin();
							if (ensure(Dependency))
							{
								SortIndex = FMath::Max(Map[Dependency->Name].CalculateSortIndex(Source->Name, Map) + 1, SortIndex);
							}
						}
					}
					bCalculating = false;
				}
			}
			return SortIndex;
		}
	};

	TMap<FName, FSortData> SortDatas;
	SortDatas.Reserve(ViewModelCreatorContexts.Num() + WidgetCreatorContexts.Num());
	TArray<FMVVMViewClass_Source> UnsortedSourceCreators;
	UnsortedSourceCreators.Reserve(ViewModelCreatorContexts.Num() + WidgetCreatorContexts.Num());

	for (FCompilerViewModelCreatorContext& SourceCreatorContext : ViewModelCreatorContexts)
	{
		const FMVVMBlueprintViewModelContext& ViewModelContext = SourceCreatorContext.ViewModelContext;
		FMVVMViewClass_Source CompiledSourceCreator;

		ensure(ViewModelContext.GetViewModelClass() && ViewModelContext.GetViewModelClass()->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()));
		CompiledSourceCreator.ExpectedSourceType = ViewModelContext.GetViewModelClass();
		CompiledSourceCreator.PropertyName = ViewModelContext.GetViewModelName();

		bool bCanBeSet = ViewModelContext.bCreateSetterFunction;
		bool bCanBeEvaluated = SourceCreatorContext.DynamicContext.IsValid();
		bool bIsOptional = SourceCreatorContext.Source->bIsOptional;
		bool bCreateInstance = false;
		bool bIsUserWidgetProperty = !SourceCreatorContext.DynamicContext.IsValid();
		bool bExposeInstanceInEditor = false;
		bool bGlobalViewModelCollectionUpdate = false;
		bool bAlwaysExecuteBindingsOnSetSource =  GetDefault<UMVVMDeveloperProjectSettings>()->bForceExecuteBindingsOnSetSource;
		bool bParentContextReference = false;

		if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::Manual)
		{
			bCanBeSet = true;
			ensure(bIsOptional == true);
		}
		else if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::CreateInstance)
		{
			bCreateInstance = true;
			bExposeInstanceInEditor = ViewModelContext.bExposeInstanceInEditor;

			if (bExposeInstanceInEditor)
			{
				const FMVVMVCompiledFieldPath* CompiledFieldPath = CompileResult.FieldPaths.Find(SourceCreatorContext.ReadPropertyPathHandle);
				if (CompiledFieldPath == nullptr)
				{
					AddMessageForViewModel(ViewModelContext
						, LOCTEXT("ViewModelInvalidInstanceNotFound", "The path for the created viewmodel instance was not found.")
						, Compiler::EMessageType::Error
					);
					bIsCompileStepValid = false;
					continue;
				}

				CompiledSourceCreator.FieldPath = *CompiledFieldPath;
			}
			ensure(bIsOptional == false);
		}
		else if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::PropertyPath)
		{
			const FMVVMVCompiledFieldPath* CompiledFieldPath = CompileResult.FieldPaths.Find(SourceCreatorContext.ReadPropertyPathHandle);
			if (CompiledFieldPath == nullptr)
			{
				AddMessageForViewModel(ViewModelContext
					, LOCTEXT("ViewModelInvalidInitializationBindingNotGenerated", "The viewmodel initialization binding was not generated.")
					, Compiler::EMessageType::Error
				);
				bIsCompileStepValid = false;
				continue;
			}

			CompiledSourceCreator.FieldPath = *CompiledFieldPath;
			ensure(bIsOptional == ViewModelContext.bOptional);
		}
		else if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::GlobalViewModelCollection)
		{
			ensure(!ViewModelContext.GlobalViewModelIdentifier.IsNone());

			FMVVMViewModelContext GlobalViewModelInstance;
			GlobalViewModelInstance.ContextClass = ViewModelContext.GetViewModelClass();
			GlobalViewModelInstance.ContextName = ViewModelContext.GlobalViewModelIdentifier;
			if (!GlobalViewModelInstance.IsValid())
			{
				AddMessageForViewModel(ViewModelContext
					, LOCTEXT("ViewmodelGlobalContextIdentifier", "The context for viewmodel could not be created. Change the identifier.")
					, Compiler::EMessageType::Warning
				);
			}

			CompiledSourceCreator.GlobalViewModelInstance = MoveTemp(GlobalViewModelInstance);
			bGlobalViewModelCollectionUpdate = ViewModelContext.bGlobalViewModelCollectionUpdate;
			ensure(bIsOptional == ViewModelContext.bOptional);
		}
		else if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::Resolver)
		{
			UMVVMViewModelContextResolver* Resolver = DuplicateObject(ViewModelContext.Resolver.Get(), ViewExtension);
			if (!Resolver)
			{
				AddMessageForViewModel(ViewModelContext
					, LOCTEXT("ViewmodelFailedResolverDuplicate", "Internal error. The resolver could not be dupliated.")
					, Compiler::EMessageType::Error
				);
				bIsCompileStepValid = false;
				continue;
			}

			CompiledSourceCreator.Resolver = Resolver;
			ensure(bIsOptional == ViewModelContext.bOptional);
		}
		else if (ViewModelContext.CreationType == EMVVMBlueprintViewModelContextCreationType::Context)
		{
			ensure(bCanBeSet == false);
			ensure(bIsOptional == true);
			bParentContextReference = true;
		}
		else
		{
			AddMessageForViewModel(ViewModelContext
				, LOCTEXT("ViewModelWithoutValidCreationType", "The viewmodel doesn't have a valid creation type.")
				, Compiler::EMessageType::Error
			);
			bIsCompileStepValid = false;
			continue;
		}

		if (SourceCreatorContext.DynamicContext)
		{
			if (!SourceCreatorContext.DynamicContext->ParentSource)
			{
				AddMessageForViewModel(ViewModelContext
					, LOCTEXT("ViewModelWithoutValidDyanmicParentSource", "The viewmodel doesn't have a valid parent source.")
					, Compiler::EMessageType::Error
				);
				bIsCompileStepValid = false;
				continue;
			}
		}

		if (ViewModelContext.bOverrideForceExecuteBindingsOnSetSource)
		{
			bAlwaysExecuteBindingsOnSetSource = ViewModelContext.bForceExecuteBindingsOnSetSource;
		}
		bAlwaysExecuteBindingsOnSetSource = bCanBeSet && bAlwaysExecuteBindingsOnSetSource;

		CompiledSourceCreator.Flags = 0;
		CompiledSourceCreator.Flags |= bCreateInstance ? (uint16)FMVVMViewClass_Source::EFlags::TypeCreateInstance : 0;
		CompiledSourceCreator.Flags |= bIsUserWidgetProperty ? (uint16)FMVVMViewClass_Source::EFlags::IsUserWidgetProperty : 0;
		CompiledSourceCreator.Flags |= bIsUserWidgetProperty ? (uint16)FMVVMViewClass_Source::EFlags::SetUserWidgetProperty : 0;
		CompiledSourceCreator.Flags |= bIsOptional ? (uint16)FMVVMViewClass_Source::EFlags::IsOptional : 0;
		CompiledSourceCreator.Flags |= bCanBeSet ? (uint16)FMVVMViewClass_Source::EFlags::CanBeSet : 0;
		CompiledSourceCreator.Flags |= bCanBeEvaluated || bGlobalViewModelCollectionUpdate ? (uint16)FMVVMViewClass_Source::EFlags::CanBeEvaluated : 0;
		CompiledSourceCreator.Flags |= (uint16)FMVVMViewClass_Source::EFlags::IsViewModel;
		CompiledSourceCreator.Flags |= bExposeInstanceInEditor ? (uint16)FMVVMViewClass_Source::EFlags::IsViewModelInstanceExposed : 0;
		CompiledSourceCreator.Flags |= bGlobalViewModelCollectionUpdate ? (uint16)FMVVMViewClass_Source::EFlags::GlobalViewModelCollectionUpdate : 0;
		CompiledSourceCreator.Flags |= bAlwaysExecuteBindingsOnSetSource ? (uint16)FMVVMViewClass_Source::EFlags::AlwaysExecuteBindingsOnSetSource : 0;
		CompiledSourceCreator.Flags |= bParentContextReference ? (uint16)FMVVMViewClass_Source::EFlags::ParentContextReference : 0;

		{
			FSortData SortData;
			SortData.Source = SourceCreatorContext.Source.ToSharedRef();
			SortDatas.Add(CompiledSourceCreator.GetName(), SortData);

			ensure(SortData.Source->Name == CompiledSourceCreator.GetName());
		}

		UnsortedSourceCreators.Add(MoveTemp(CompiledSourceCreator));
	}

	// The other sources needed by the view
	for (FCompilerWidgetCreatorContext& WidgetCreator : WidgetCreatorContexts)
	{
		FMVVMViewClass_Source CompiledSourceCreator;
		ensure(WidgetCreator.Source->AuthoritativeClass && WidgetCreator.Source->AuthoritativeClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()));
		CompiledSourceCreator.ExpectedSourceType = const_cast<UClass*>(WidgetCreator.Source->AuthoritativeClass);
		CompiledSourceCreator.PropertyName = WidgetCreator.Source->Name;
		CompiledSourceCreator.Flags = 0;
		CompiledSourceCreator.Flags |= WidgetCreator.bSelfReference ? (uint16)FMVVMViewClass_Source::EFlags::SelfReference : 0;

		if (!WidgetCreator.bSelfReference)
		{
			const FMVVMVCompiledFieldPath* CompiledFieldPath = CompileResult.FieldPaths.Find(WidgetCreator.ReadPropertyPathHandle);
			if (CompiledFieldPath == nullptr)
			{
				AddMessage(FText::Format(LOCTEXT("WidgetInvalidInitializationBindingNotGenerated", "The widget {0} initialization binding was not generated."), FText::FromName(WidgetCreator.Source->Name))
					, Compiler::EMessageType::Error
				);
				bIsCompileStepValid = false;
				continue;
			}
			CompiledSourceCreator.FieldPath = *CompiledFieldPath;
		}

		{
			FSortData SortData;
			SortData.Source = WidgetCreator.Source;
			SortDatas.Add(CompiledSourceCreator.GetName(), SortData);
		}

		UnsortedSourceCreators.Add(MoveTemp(CompiledSourceCreator));
	}

	// Sanity check
	{
		// Test if all NeededBindingSources in inside the UnsortedSourceCreators
		for (const TSharedRef<FCompilerBindingSource>& BindingSource : NeededBindingSources)
		{
			const FMVVMViewClass_Source* FoundClassSource = UnsortedSourceCreators.FindByPredicate([ToFindName = BindingSource->Name](const FMVVMViewClass_Source& Other){ return Other.GetName() == ToFindName; });
			if (FoundClassSource == nullptr)
			{
				AddMessage(FText::Format(LOCTEXT("CompileSources_MissingSources", "Internal error. The source {0} was not compiled."), FText::FromName(BindingSource->Name))
					, Compiler::EMessageType::Warning
				);
			}
		}

		ensure(SortDatas.Num() == UnsortedSourceCreators.Num());
	}

	// Sort the source creators by priority and then by name
	if (UnsortedSourceCreators.Num() > 1)
	{
		// Calculate dependencies
		for (auto& SortDataPair : SortDatas)
		{
			SortDataPair.Value.CalculateSortIndex(SortDataPair.Key, SortDatas);
		}

		// Report errors
		for (const auto& SortDataPair : SortDatas)
		{
			for (FName OtherSource : SortDataPair.Value.Errors)
			{
				AddMessage(
					FText::Format(LOCTEXT("CompileSources_Dependencies", "The source {0} circularly depends on the source {1}"), FText::FromName(SortDataPair.Key), FText::FromName(OtherSource))
					, Compiler::EMessageType::Info
				);
			}
		}

		// Sort the sources
		UnsortedSourceCreators.Sort([&SortDatas](const FMVVMViewClass_Source& A, const FMVVMViewClass_Source& B)
			{
				int32 ASortIndex = SortDatas[A.GetName()].SortIndex;
				int32 BSortIndex = SortDatas[B.GetName()].SortIndex;
				if (ASortIndex == BSortIndex)
				{
					return A.GetName().LexicalLess(B.GetName());
				}
				return ASortIndex < BSortIndex;
			});
	}

	// Add the sorted viewmodel array to the ViewExtension
	ViewExtension->Sources = MoveTemp(UnsortedSourceCreators);

	if (ViewExtension->Sources.Num() > 64)
	{
		// The view use a uint64 bitfield to filter which viewmodel/source is valid/initialized.
		//You can use the MVVM.LogViewCompiledResult command to display the compile result and see the name of the sources.
		AddMessage(LOCTEXT("TooManySources", "There is too many sources for the view. Try spliting your widget into other widgets."), Compiler::EMessageType::Error);
	}
}


void FMVVMViewBlueprintCompiler::PreCompileBindings(UWidgetBlueprintGeneratedClass* Class)
{
	auto TestExecutionMode = [Self = this](const FMVVMBlueprintViewBinding& Binding) -> bool
	{
		if (Binding.bOverrideExecutionMode)
		{
			if (!GetDefault<UMVVMDeveloperProjectSettings>()->IsExecutionModeAllowed(Binding.OverrideExecutionMode))
			{
				Self->AddMessageForBinding(Binding
					, LOCTEXT("NotAllowedExecutionMode", "The binding has a restricted execution mode.")
					, Compiler::EMessageType::Error
					, FMVVMBlueprintPinId()
				);
				return false;
			}
		}
		return true;
	};

	auto AddFieldIds = [Self = this, Class](FCompilerBinding& ValidBinding, const FMVVMBlueprintViewBinding& Binding) -> bool
	{
		// Add FieldId
		bool bHasFieldId = false;
		if (!ValidBinding.bIsOneTimeBinding)
		{
			for (const TSharedPtr<FGeneratedReadFieldPathContext>& ReadPath : ValidBinding.ReadPaths)
			{
				if (ReadPath->NotificationField)
				{
					if (ReadPath->NotificationField->LibraryCompilerHandle.IsValid())
					{
						bHasFieldId = true;
					}
					else if (ReadPath->NotificationField->Source) // it is maybe OneTIme
					{
						const UClass* SourceContextClass = ReadPath->NotificationField->Source->AuthoritativeClass;
						TValueOrError<FCompiledBindingLibraryCompiler::FFieldIdHandle, FText> FieldIdResult = Self->BindingLibraryCompiler.AddFieldId(SourceContextClass, ReadPath->NotificationField->NotificationId.GetFieldName());
						if (FieldIdResult.HasError() || !FieldIdResult.GetValue().IsValid())
						{
							Self->AddMessageForBinding(Binding
								, FText::Format(LOCTEXT("CouldNotCreateFieldId", "Could not create Field. {0}"), FieldIdResult.GetError())
								, Compiler::EMessageType::Error
								, FMVVMBlueprintPinId()
							);
							return false;
						}
						
						ReadPath->NotificationField->LibraryCompilerHandle = FieldIdResult.StealValue();
						bHasFieldId = ReadPath->NotificationField->LibraryCompilerHandle.IsValid();
					}
				}
			}
		}

		// Test correct numbers of FieldIds
		bool bRequiresValidFieldId = !ValidBinding.bIsOneTimeBinding;
		if (bRequiresValidFieldId && !bHasFieldId)
		{
			Self->AddMessageForBinding(Binding
				, LOCTEXT("CouldNotCreateSourceFields", "This binding will only trigger ONCE whenever the value updates (typically when the value is first initialized).\n To ensure it triggers EVERY time the value updates, turn on Field Notify on this variable.")
				, Compiler::EMessageType::Info
				, FMVVMBlueprintPinId()
			);
		}

		return true;
	};

	auto AddReadPaths = [Self = this, Class](FCompilerBinding& ValidBinding, const FMVVMBlueprintViewBinding& Binding) -> bool
	{
		bool bHasValidReadHandle = false;
		if (ValidBinding.Type != FCompilerBinding::EType::ComplexConversionFunction)
		{
			for (const TSharedPtr<FGeneratedReadFieldPathContext>& ReadPath : ValidBinding.ReadPaths)
			{
				if (ReadPath->LibraryCompilerHandle.IsValid())
				{
					bHasValidReadHandle = true;
				}
				else
				{
					TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> FieldPathResult = Self->BindingLibraryCompiler.AddFieldPath(ReadPath->SkeletalGeneratedFields, true);
					if (FieldPathResult.HasError() || !FieldPathResult.GetValue().IsValid())
					{
						Self->AddMessageForBinding(Binding
							, FText::Format(Private::CouldNotCreateSourceFieldPathFormat, ::UE::MVVM::FieldPathHelper::ToText(ReadPath->GeneratedFields), FieldPathResult.GetError())
							, Compiler::EMessageType::Error
							, FMVVMBlueprintPinId()
						);
						return false;
					}

					ReadPath->LibraryCompilerHandle = FieldPathResult.StealValue();
					bHasValidReadHandle = ReadPath->LibraryCompilerHandle.IsValid();
				}
			}
		}

		// Test read
		bool bRequiresReadHandle = ValidBinding.Type != FCompilerBinding::EType::ComplexConversionFunction;
		if (bRequiresReadHandle && !bHasValidReadHandle)
		{
			Self->AddMessageForBinding(Binding
				, LOCTEXT("CouldNotCreateSourceReadIsPresent", "Internal error. There should be no property to read from.")
				, Compiler::EMessageType::Error
				, FMVVMBlueprintPinId()
			);
			return false;
		}

		return true;
	};

	auto AddWritePaths = [Self = this, Class](FCompilerBinding& ValidBinding, const FMVVMBlueprintViewBinding& Binding) -> bool
	{
		if (ValidBinding.WritePath->LibraryCompilerHandle.IsValid())
		{
			return true;
		}

		if (!ValidBinding.WritePath)
		{
			Self->AddMessageForBinding(Binding
				, LOCTEXT("InvalidWritePath", "Internal Error. The write path is invalid.")
				, Compiler::EMessageType::Error
				, FMVVMBlueprintPinId()
			);
			return false;
		}

		TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> FieldPathResult = Self->BindingLibraryCompiler.AddFieldPath(ValidBinding.WritePath->SkeletalGeneratedFields, false);
		if (FieldPathResult.HasError())
		{
			Self->AddMessageForBinding(Binding
				, FText::Format(Private::CouldNotCreateDestinationFieldPathFormat, ::UE::MVVM::FieldPathHelper::ToText(ValidBinding.WritePath->GeneratedFields), FieldPathResult.GetError())
				, Compiler::EMessageType::Error
				, FMVVMBlueprintPinId()
			);
			return false;
		}
		ValidBinding.WritePath->LibraryCompilerHandle = FieldPathResult.StealValue();

		// test write

		return true;
	};

	auto AddConversionFunction = [Self = this, Class](FCompilerBinding& ValidBinding, const FMVVMBlueprintViewBinding& Binding) -> bool
	{
		const UFunction* ConversionFunction = nullptr;
		UMVVMBlueprintViewConversionFunction* ViewConversionFunction = ValidBinding.ConversionFunction.Get();
		{
			if (ViewConversionFunction)
			{
				ConversionFunction = ViewConversionFunction->GetCompiledFunction(Class);
				if (ConversionFunction == nullptr)
				{
					Self->AddMessageForBinding(Binding
						, FText::Format(LOCTEXT("ConversionFunctionNotFound", "The conversion function '{0}' could not be found."), FText::FromName(ViewConversionFunction->GetCompiledFunctionName(Class)))
						, Compiler::EMessageType::Error
						, FMVVMBlueprintPinId()
					);
					return false;
				}

				FMVVMBlueprintFunctionReference FunctionOrWrapperFunction = ViewConversionFunction->GetConversionFunction();
				if (FunctionOrWrapperFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
				{
					const UFunction* WrapperFunction = FunctionOrWrapperFunction.GetFunction(Self->WidgetBlueprintCompilerContext.WidgetBlueprint());
					if (!GetDefault<UMVVMDeveloperProjectSettings>()->IsConversionFunctionAllowed(Self->WidgetBlueprintCompilerContext.WidgetBlueprint(), WrapperFunction))
					{
						Self->AddMessageForBinding(Binding
							, FText::Format(LOCTEXT("ConversionFunctionNotAllow", "The conversion function {0} is not allowed."), FText::FromName(WrapperFunction ? WrapperFunction->GetFName() : FName()))
							, Compiler::EMessageType::Error
							, FMVVMBlueprintPinId()
						);
						return false;
					}
				}
				else if (FunctionOrWrapperFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Node)
				{
					if (!GetDefault<UMVVMDeveloperProjectSettings>()->IsConversionFunctionAllowed(Self->WidgetBlueprintCompilerContext.WidgetBlueprint(), FunctionOrWrapperFunction.GetNode()))
					{
						Self->AddMessageForBinding(Binding
							, FText::Format(LOCTEXT("ConversionFunctionNotAllow", "The conversion function {0} is not allowed."), FText::FromName(FunctionOrWrapperFunction.GetNode().Get() ? FunctionOrWrapperFunction.GetNode()->GetFName() : FName()))
							, Compiler::EMessageType::Error
							, FMVVMBlueprintPinId()
						);
						return false;
					}
				}
				else
				{
					Self->AddMessageForBinding(Binding
						, LOCTEXT("ConversionFunctionNodeNotAllow", "The conversion function node is not allowed.")
						, Compiler::EMessageType::Error
						, FMVVMBlueprintPinId()
					);
					return false;
				}
			}
		}

		bool bIsUbergraphPage = ViewConversionFunction && ViewConversionFunction->IsUbergraphPage();
		if (ConversionFunction != nullptr)
		{
			TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> FieldPathResult = bIsUbergraphPage 
				? Self->BindingLibraryCompiler.AddDelegateSignatureFieldPath(Class, ConversionFunction)
				: Self->BindingLibraryCompiler.AddConversionFunctionFieldPath(Class, ConversionFunction);
			if (FieldPathResult.HasError())
			{
				Self->AddMessageForBinding(Binding
					, FText::Format(LOCTEXT("CouldNotCreateConversionFunctionFieldPath", "Couldn't create the conversion function field path '{0}'. {1}")
						, FText::FromString(ConversionFunction->GetPathName())
						, FieldPathResult.GetError())
					, Compiler::EMessageType::Error
					, FMVVMBlueprintPinId()
				);
				return false;
			}

			ValidBinding.ConversionFunctionHandle = FieldPathResult.StealValue();
		}

		// Sanity check
		if (!bIsUbergraphPage)
		{
			const bool bShouldHaveConversionFunction = ValidBinding.Type == FCompilerBinding::EType::ComplexConversionFunction || ValidBinding.Type == FCompilerBinding::EType::SimpleConversionFunction;
			if ((ConversionFunction != nullptr) != bShouldHaveConversionFunction)
			{
				Self->AddMessageForBinding(Binding, LOCTEXT("ConversionFunctionShouldExist", "Internal error. The conversion function should exist."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
				return false;
			}

			const bool bShouldHaveComplexConversionFunction = ValidBinding.Type == FCompilerBinding::EType::ComplexConversionFunction;
			if (bShouldHaveComplexConversionFunction != BindingHelper::IsValidForComplexRuntimeConversion(ConversionFunction))
			{
				Self->AddMessageForBinding(Binding, LOCTEXT("ConversionFunctionIsNotComplex", "Internal Error. The complex conversion function does not respect the prerequisite."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
				return false;
			}
		}

		return true;
	};

	for (TSharedRef<FCompilerBinding>& ValidBinding : ValidBindings)
	{
		const FMVVMBlueprintViewBinding& Binding = *(BlueprintView->GetBindingAt(ValidBinding->Key.ViewBindingIndex));
		if (ValidBinding->Type != FCompilerBinding::EType::Assignment
			&& ValidBinding->Type != FCompilerBinding::EType::ComplexConversionFunction
			&& ValidBinding->Type != FCompilerBinding::EType::SimpleConversionFunction)
		{
			AddMessageForBinding(Binding, LOCTEXT("UnsupportedBindingType", "The binding is invalid."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
			bIsPreCompileStepValid = false;
			continue;
		}

		if (!TestExecutionMode(Binding))
		{
			bIsPreCompileStepValid = false;
			continue;
		}

		if (!AddFieldIds(ValidBinding.Get(), Binding)
			|| !AddReadPaths(ValidBinding.Get(), Binding)
			|| !AddWritePaths(ValidBinding.Get(), Binding)
			|| !AddConversionFunction(ValidBinding.Get(), Binding))
		{
			bIsPreCompileStepValid = false;
			continue;
		}

		// Generate the binding
		TValueOrError<FCompiledBindingLibraryCompiler::FBindingHandle, FText> BindingResult = ValidBinding->Type == FCompilerBinding::EType::ComplexConversionFunction
			? BindingLibraryCompiler.AddComplexBinding(ValidBinding->WritePath->LibraryCompilerHandle, ValidBinding->ConversionFunctionHandle)
			: BindingLibraryCompiler.AddBinding(ValidBinding->ReadPaths[0]->LibraryCompilerHandle, ValidBinding->WritePath->LibraryCompilerHandle, ValidBinding->ConversionFunctionHandle);

		if (BindingResult.HasError())
		{
			AddMessageForBinding(Binding
				, FText::Format(LOCTEXT("CouldNotCreateBinding", "Could not create binding. {0}"), BindingResult.StealError())
				, Compiler::EMessageType::Error
				, FMVVMBlueprintPinId()
			);
			bIsPreCompileStepValid = false;
			continue;
		}
		ValidBinding->BindingHandle = BindingResult.StealValue();
		ValidBinding->CompilerBindingHandle = Compiler::FCompilerBindingHandle::MakeHandle();
	}

	// Add bindings for the dynamic viewmodels
	for (TSharedRef<FCompilerSourceViewModelDynamicCreatorContext>& ViewModelDynamic : SourceViewModelDynamicCreatorContexts)
	{
		if (ensure(ViewModelDynamic->Source.IsValid() && ViewModelDynamic->ParentSource.IsValid()))
		{
			if (!ViewModelDynamic->NotificationId.IsValid())
			{
				AddMessage(FText::Format(LOCTEXT("InvalidNotificationFieldId", "{0} doesn't have a field Id."), FText::FromName(ViewModelDynamic->Source->Name)), Compiler::EMessageType::Error);
				bIsPreCompileStepValid = false;
				continue;
			}

			const UClass* SourceContextClass = ViewModelDynamic->ParentSource->AuthoritativeClass;
			TValueOrError<FCompiledBindingLibraryCompiler::FFieldIdHandle, FText> FieldIdResult = BindingLibraryCompiler.AddFieldId(SourceContextClass, ViewModelDynamic->NotificationId.GetFieldName());
			if (FieldIdResult.HasError() || !FieldIdResult.GetValue().IsValid())
			{
				AddMessage(FText::Format(LOCTEXT("CouldNotCreateFieldIdForDynamic", "Could not create Field for {0}. {1}"), FText::FromName(ViewModelDynamic->Source->Name), FieldIdResult.GetError()), Compiler::EMessageType::Error);
				bIsPreCompileStepValid = false;
				continue;
			}

			ViewModelDynamic->NotificationIdLibraryCompilerHandle = FieldIdResult.GetValue();
		}
	}
}


void FMVVMViewBlueprintCompiler::CompileBindings(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension)
{
	static IConsoleVariable* CVarDefaultExecutionMode = IConsoleManager::Get().FindConsoleVariable(TEXT("MVVM.DefaultExecutionMode"));
	ensure(CVarDefaultExecutionMode);
	if (!CVarDefaultExecutionMode)
	{
		WidgetBlueprintCompilerContext.MessageLog.Error(*LOCTEXT("CantFindDefaultExecutioMode", "The default execution mode cannot be found.").ToString());
		return;
	}

	// Sort the array to have a predictable list
	{
		UMVVMBlueprintView* LocalBlueprintView = BlueprintView.Get();
		ValidBindings.StableSort([LocalBlueprintView](const TSharedRef<FCompilerBinding>& A, const TSharedRef<FCompilerBinding>& B)
			{
				const FMVVMBlueprintViewBinding& BindingA = *(LocalBlueprintView->GetBindingAt(A->Key.ViewBindingIndex));
				const FMVVMBlueprintViewBinding& BindingB = *(LocalBlueprintView->GetBindingAt(B->Key.ViewBindingIndex));
				return BindingA.BindingId < BindingB.BindingId;
			});
	}

	ensure(ViewExtension->Bindings.Num() == 0);
	ViewExtension->Bindings.Reset(ValidBindings.Num());
	for (int32 ValidBindingIndex = 0; ValidBindingIndex < ValidBindings.Num(); ++ValidBindingIndex)
	{
		const TSharedRef<FCompilerBinding>& ValidBinding = ValidBindings[ValidBindingIndex];
		const FMVVMBlueprintViewBinding& Binding = *(BlueprintView->GetBindingAt(ValidBinding->Key.ViewBindingIndex));

		const FMVVMVCompiledBinding* CompiledBinding = CompileResult.Bindings.Find(ValidBinding->BindingHandle);
		if (CompiledBinding == nullptr)
		{
			AddMessageForBinding(Binding, LOCTEXT("CompiledBindingNotGenerated", "Could not generate compiled binding."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
			bIsCompileStepValid = false;
			continue;
		}

		FMVVMViewClass_BindingKey BindingKey = FMVVMViewClass_BindingKey(ViewExtension->Bindings.AddDefaulted());
		FMVVMViewClass_Binding& NewBinding = ViewExtension->Bindings[BindingKey.GetIndex()];
		NewBinding.Binding = CompiledBinding ? *CompiledBinding : FMVVMVCompiledBinding();
		NewBinding.Flags = 0;
		NewBinding.ExecutionMode = Binding.bOverrideExecutionMode ? Binding.OverrideExecutionMode : (EMVVMExecutionMode)CVarDefaultExecutionMode->GetInt();
		NewBinding.SourceBitField = 0;
		NewBinding.EditorId = Binding.BindingId;

		// Add the write source.
		if (ValidBinding->WritePath && ValidBinding->WritePath->OptionalSource)
		{
			int32 ViewExtensionSourceCreatorsIndex = ViewExtension->Sources.IndexOfByPredicate([LookFor = ValidBinding->WritePath->OptionalSource->Name](const FMVVMViewClass_Source& Other)
				{
					return Other.GetName() == LookFor;
				});
			if (ViewExtension->Sources.IsValidIndex(ViewExtensionSourceCreatorsIndex))
			{
				
				FMVVMViewClass_SourceKey FieldClassSourceKey = FMVVMViewClass_SourceKey(ViewExtensionSourceCreatorsIndex);
				NewBinding.SourceBitField |= FieldClassSourceKey.GetBit();
			}
		}

		TArray<FMVVMViewClass_SourceKey, TInlineAllocator<16>>  SharedExecuteAtInitializationBindings;
		int32 SharedBindingsCount = 0;
		// Find the source needed by the binding. Also generate every bindings on that source (that need to register to the FieldNotify).
		for (TSharedPtr<FGeneratedReadFieldPathContext> ReadPath : ValidBinding->ReadPaths)
		{
			if (ReadPath->Source == nullptr)
			{
				AddMessageForBinding(Binding, LOCTEXT("InvalidSourceInternal", "Internal error. The binding has an invalid source."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
				bIsCompileStepValid = false;
				continue;
			}

			int32 ViewExtensionSourceCreatorsIndex = ViewExtension->Sources.IndexOfByPredicate([LookFor = ReadPath->Source->Name](const FMVVMViewClass_Source& Other)
				{
					return Other.GetName() == LookFor;
				});
			if (!ViewExtension->Sources.IsValidIndex(ViewExtensionSourceCreatorsIndex))
			{
				AddMessageForBinding(Binding, LOCTEXT("CompiledSourceCreatorNotGenerated", "Internal error. The source creator was not generated."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
				bIsCompileStepValid = false;
				continue;
			}

			// Add the needed source.
			FMVVMViewClass_SourceKey FieldClassSourceKey = FMVVMViewClass_SourceKey(ViewExtensionSourceCreatorsIndex);
			NewBinding.SourceBitField |= FieldClassSourceKey.GetBit();

			// FieldId
			const bool bHasField = (ReadPath->NotificationField && !ValidBinding->bIsOneTimeBinding);
			const UE::FieldNotification::FFieldId* CompiledFieldId = bHasField ? CompileResult.FieldIds.Find(ReadPath->NotificationField->LibraryCompilerHandle) : nullptr;
			if (CompiledFieldId == nullptr && bHasField)
			{
				AddMessageForBinding(Binding, LOCTEXT("CompiledFieldNotGenerated", "Internal error. The FieldId was not generated."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
				bIsCompileStepValid = false;
				continue;
			}

			bool bExecuteAtInitialization = ValidBinding->Key.bIsForwardBinding;

			FMVVMViewClass_Source& ClassSource = ViewExtension->Sources[ViewExtensionSourceCreatorsIndex];
			FMVVMViewClass_SourceBinding& NewSourceBinding = ClassSource.Bindings.AddDefaulted_GetRef();
			NewSourceBinding.BindingKey = BindingKey;

			NewSourceBinding.Flags = 0;
			// Set the ExecuteAtInitialization but remove it later if it's not the last of the group.
			NewSourceBinding.Flags |= (bExecuteAtInitialization) ? (uint8)FMVVMViewClass_SourceBinding::EFlags::ExecuteAtInitialization : 0;

			if (CompiledFieldId)
			{
				NewSourceBinding.FieldId = FFieldNotificationId(CompiledFieldId->GetName());
				ClassSource.FieldToRegisterTo.AddUnique(FMVVMViewClass_FieldId(*CompiledFieldId));

				// Count the number of shared instance of that binding. (they can come from the same Source)
				//This flag is used at runtime to know if we should delay the binding execution.
				//We only delay when the field changes. It should only be shared if it has field.
				++SharedBindingsCount;
			}

			if (bExecuteAtInitialization)
			{
				SharedExecuteAtInitializationBindings.Add(FieldClassSourceKey);
			}
		}

		// Set binding flags.
		NewBinding.Flags |= (!ValidBinding->bIsOneTimeBinding) ? (uint8)FMVVMViewClass_Binding::EFlags::OneWay : 0;
		NewBinding.Flags |= (SharedBindingsCount > 1) ? (uint8)FMVVMViewClass_Binding::EFlags::Shared : 0;
		NewBinding.Flags |= (Binding.bOverrideExecutionMode) ? (uint8)FMVVMViewClass_Binding::EFlags::OverrideExecuteMode : 0;
		NewBinding.Flags |= (Binding.bEnabled) ? (uint8)FMVVMViewClass_Binding::EFlags::EnabledByDefault : 0;

		// Only the last binding in the complex conversion has ExecuteAtInitialization.
		if (SharedExecuteAtInitializationBindings.Num() > 1)
		{
			SharedExecuteAtInitializationBindings.Sort([](const FMVVMViewClass_SourceKey& A, const FMVVMViewClass_SourceKey&B)
				{
					return A.GetIndex() < B.GetIndex();
				});

			//Remove the flag on all the binding on all the sources ecept the last one.
			for (int32 Index = 0; Index < SharedExecuteAtInitializationBindings.Num() - 1; ++Index)
			{
				if (SharedExecuteAtInitializationBindings[Index] != SharedExecuteAtInitializationBindings.Last())
				{
					FMVVMViewClass_Source& ClassSource = ViewExtension->Sources[SharedExecuteAtInitializationBindings[Index].GetIndex()];
					for (FMVVMViewClass_SourceBinding& SourceBinding : ClassSource.Bindings)
					{
						if (SourceBinding.GetBindingKey() == BindingKey)
						{
							SourceBinding.Flags &= ~(uint8)FMVVMViewClass_SourceBinding::EFlags::ExecuteAtInitialization;
						}
					}
				}
			}
			// Keep only one ExecuteAtInitialization on the last source.
			{
				FMVVMViewClass_Source& ClassSource = ViewExtension->Sources[SharedExecuteAtInitializationBindings.Last().GetIndex()];
				bool bFound = false;
				for (int32 Index = ClassSource.Bindings.Num() - 1; Index >= 0; --Index)
				{
					FMVVMViewClass_SourceBinding& SourceBinding = ClassSource.Bindings[Index];
					if (SourceBinding.GetBindingKey() == BindingKey)
					{
						if (bFound)
						{
							SourceBinding.Flags &= ~(uint8)FMVVMViewClass_SourceBinding::EFlags::ExecuteAtInitialization;
						}
						bFound = true;
					}
				}
			}
		}

		// Set the real binding key for the async validation nodes
		if (ValidBinding->Type == FCompilerBinding::EType::ComplexConversionFunction)
		{
			UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(ValidBinding->Key.bIsForwardBinding);
			if (ensure(ConversionFunction) && ConversionFunction->IsUbergraphPage())
			{
				UEdGraph* WrapperGraph = ConversionFunction->GetOrCreateIntermediateWrapperGraph(WidgetBlueprintCompilerContext);
				if (ensure(WrapperGraph))
				{
					TArray<UMVVMK2Node_AreSourcesValidForBinding*> AreSourcesValidNodes;
					WrapperGraph->GetNodesOfClass<UMVVMK2Node_AreSourcesValidForBinding>(AreSourcesValidNodes);

					for (UMVVMK2Node_AreSourcesValidForBinding* AreSourcesValidNode : AreSourcesValidNodes)
					{
						AreSourcesValidNode->BindingKey = BindingKey;
					}
				}
			}
		}
	}
}


void FMVVMViewBlueprintCompiler::CompileEvaluateSources(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension)
{
	for (const TSharedRef<FCompilerSourceViewModelDynamicCreatorContext>& ViewModelDynamic : SourceViewModelDynamicCreatorContexts)
	{
		int32 ViewExtensionSourceCreatorsIndex = ViewExtension->Sources.IndexOfByPredicate([LookFor = ViewModelDynamic->Source->Name](const FMVVMViewClass_Source& Other)
			{
				return Other.GetName() == LookFor;
			});
		if (!ViewExtension->Sources.IsValidIndex(ViewExtensionSourceCreatorsIndex))
		{
			AddMessage(LOCTEXT("CompiledSourceCreatorNotGenerated", "Internal error. The source creator was not generated."), Compiler::EMessageType::Error);
			bIsCompileStepValid = false;
			continue;
		}
		int32 ViewExtensionParentCreatorsIndex = ViewExtension->Sources.IndexOfByPredicate([LookFor = ViewModelDynamic->ParentSource->Name](const FMVVMViewClass_Source& Other)
			{
				return Other.GetName() == LookFor;
			});
		if (!ViewExtension->Sources.IsValidIndex(ViewExtensionParentCreatorsIndex))
		{
			AddMessage(LOCTEXT("CompiledParentSourceCreatorNotGenerated", "Internal error. The parent source creator was not generated."), Compiler::EMessageType::Error);
			bIsCompileStepValid = false;
			continue;
		}

		const UE::FieldNotification::FFieldId* CompiledFieldId = CompileResult.FieldIds.Find(ViewModelDynamic->NotificationIdLibraryCompilerHandle);
		if (CompiledFieldId == nullptr)
		{
			AddMessage(LOCTEXT("CompiledFieldNotifyNotGenerated", "Internal error. The field notify was not generated."), Compiler::EMessageType::Error);
			bIsCompileStepValid = false;
			continue;
		}

		FMVVMViewClass_EvaluateSource& NewBinding = ViewExtension->EvaluateSources.AddDefaulted_GetRef();
		NewBinding.ParentFieldId = FFieldNotificationId(CompiledFieldId->GetName());
		NewBinding.ParentSource = FMVVMViewClass_SourceKey(ViewExtensionParentCreatorsIndex);
		NewBinding.ToEvaluate = FMVVMViewClass_SourceKey(ViewExtensionSourceCreatorsIndex);

		FMVVMViewClass_Source& ClassSource = ViewExtension->Sources[ViewExtensionParentCreatorsIndex];
		ClassSource.FieldToRegisterTo.AddUnique(FMVVMViewClass_FieldId(*CompiledFieldId));

		ClassSource.Flags |= (uint16)FMVVMViewClass_Source::EFlags::HasEvaluatedBindings;
	}

	ViewExtension->EvaluateSources.StableSort([](const FMVVMViewClass_EvaluateSource& A, const FMVVMViewClass_EvaluateSource& B)
		{
			if (A.GetParentSource() == B.GetParentSource())
			{
				return A.GetFieldId().GetFieldName().Compare(B.GetFieldId().GetFieldName()) < 0;
			}
			return A.GetParentSource().GetIndex() < B.GetParentSource().GetIndex();
		});
}


void FMVVMViewBlueprintCompiler::PreCompileEvents(UWidgetBlueprintGeneratedClass* Class)
{
	if (!GetDefault<UMVVMDeveloperProjectSettings>()->bAllowBindingEvent && BlueprintView->GetEvents().Num() > 0)
	{
		WidgetBlueprintCompilerContext.MessageLog.Warning(*LOCTEXT("EventsAreNotAllowed", "Binding events are not allowed in your project settings.").ToString());
	}

	for (TSharedRef<FCompilerEvent>& ValidEvent : ValidEvents)
	{
		UMVVMBlueprintViewEvent* EventPtr = ValidEvent->Event.Get();
		check(EventPtr);
		UEdGraph* GeneratedGraph = EventPtr->GetOrCreateWrapperGraph();
		check(GeneratedGraph);

		// Does it resolve and are the field allowed
		TValueOrError<FCreateFieldsResult, FText> EventPathResult = CreateFieldContext(Class, EventPtr->GetEventPath(), true);
		if (EventPathResult.HasError())
		{
			AddMessageForEvent(EventPtr
				, FText::Format(Private::PropertyPathIsInvalidFormat, PropertyPathToText(Class, BlueprintView.Get(), EventPtr->GetEventPath()))
				, Compiler::EMessageType::Error
				, FMVVMBlueprintPinId());
			bIsPreCompileStepValid = false;
			continue;
		}

		//Does EventPath resolves to a MulticastDelegateProperty
		const FMulticastDelegateProperty* DelegateProperty = EventPathResult.GetValue().SkeletalGeneratedFields.Last().IsProperty() ? CastField<const FMulticastDelegateProperty>(EventPathResult.GetValue().SkeletalGeneratedFields.Last().GetProperty()) : nullptr;
		if (DelegateProperty == nullptr)
		{
			AddMessageForEvent(EventPtr
				, FText::Format(LOCTEXT("EventPathIsNotMulticastDelegate", "The event {0} is not a multicast delegate."), PropertyPathToText(Class, BlueprintView.Get(), EventPtr->GetEventPath()))
				, Compiler::EMessageType::Error
				, FMVVMBlueprintPinId());
			bIsPreCompileStepValid = false;
			continue;
		}

		// Does it still have the same signature
		if (!UE::MVVM::FunctionGraphHelper::IsFunctionEntryMatchSignature(GeneratedGraph, DelegateProperty->SignatureFunction))
		{
			AddMessageForEvent(EventPtr
				, FText::Format(LOCTEXT("EventPathFunctionSignatureError", "The event {0} doesn't match the function signature."), DelegateProperty->GetDisplayNameText())
				, Compiler::EMessageType::Error
				, FMVVMBlueprintPinId());
			bIsPreCompileStepValid = false;
			continue;
		}

		// Generate the FieldPath to get the delegate property at runtime
		TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> EventFieldPathResult = BindingLibraryCompiler.AddFieldPath(EventPathResult.GetValue().SkeletalGeneratedFields, true);
		if (EventFieldPathResult.HasError())
		{
			AddMessageForEvent(EventPtr
				, FText::Format(Private::CouldNotCreateSourceFieldPathFormat, PropertyPathToText(Class, BlueprintView.Get(), EventPtr->GetEventPath()), EventFieldPathResult.GetError())
				, Compiler::EMessageType::Error
				, FMVVMBlueprintPinId());
			bIsPreCompileStepValid = false;
			continue;
		}

		//todo ? this doesn't support long path?

		FName SourceName;
		switch (EventPtr->GetEventPath().GetSource(WidgetBlueprintCompilerContext.WidgetBlueprint()))
		{
		case EMVVMBlueprintFieldPathSource::SelfContext:
			SourceName = WidgetBlueprintCompilerContext.WidgetBlueprint()->GetFName();
			break;
		case EMVVMBlueprintFieldPathSource::Widget:
		{
			FName WidgetName = EventPtr->GetEventPath().GetWidgetName();
			checkf(!WidgetName.IsNone(), TEXT("The destination should have been checked and set bAreSourceContextsValid."));
			const bool bSourceIsUserWidget = WidgetName == Class->ClassGeneratedBy->GetFName();
			ensure(!bSourceIsUserWidget);
			
			SourceName = WidgetName;
			break;
		}
		case EMVVMBlueprintFieldPathSource::ViewModel:
		{
			const FMVVMBlueprintViewModelContext* SourceViewModelContext = BlueprintView->FindViewModel(EventPtr->GetEventPath().GetViewModelId());
			check(SourceViewModelContext);
			FName ViewModelName = SourceViewModelContext->GetViewModelName();
			SourceName = ViewModelName;
			break;
		}
		default:
			ensureAlwaysMsgf(false, TEXT("An EMVVMBlueprintFieldPathSource case was not checked."));
		}

		// No need to add the generated function to the field compiler.
		//They are in the BP generated code.

		ValidEvent->DelegateFieldPathHandle = EventFieldPathResult.StealValue();
		ValidEvent->GeneratedGraphName = EventPtr->GetWrapperGraphName();
		ValidEvent->SourceName = SourceName;
	}
}

void FMVVMViewBlueprintCompiler::CompileEvents(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension)
{
	// NB. The order is important. The index is used in the generated function to identify the event.
	for (const TSharedRef<FCompilerEvent>& ValidEvent : ValidEvents)
	{
		const FMVVMVCompiledFieldPath* CompiledFieldPath = CompileResult.FieldPaths.Find(ValidEvent->DelegateFieldPathHandle);
		if (CompiledFieldPath == nullptr)
		{
			AddMessageForEvent(ValidEvent->Event.Get(), LOCTEXT("CompiledEventFieldPathNotGenerated", "Could not generate the event path."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
			bIsCompileStepValid = false;
			continue;
		}

		if (ValidEvent->GeneratedGraphName.IsNone() || Class->FindFunctionByName(ValidEvent->GeneratedGraphName) == nullptr)
		{
			AddMessageForEvent(ValidEvent->Event.Get(), LOCTEXT("CompiledEventFieldPathNotGenerated", "Could not generate the event path."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
			bIsCompileStepValid = false;
			continue;
		}

		int32 FoundSourceIndex = INDEX_NONE;
		if (!ValidEvent->SourceName.IsNone())
		{
			FoundSourceIndex = ViewExtension->Sources.IndexOfByPredicate([ToFind = ValidEvent->SourceName](const FMVVMViewClass_Source& Other)
				{
					return Other.GetName() == ToFind;
				});
		}

		uint64 SourceBitField = 0;
		{
			if (ValidEvent->WritePath && ValidEvent->WritePath->OptionalSource)
			{
				int32 ViewExtensionSourceCreatorsIndex = ViewExtension->Sources.IndexOfByPredicate([LookFor = ValidEvent->WritePath->OptionalSource->Name](const FMVVMViewClass_Source& Other)
					{
						return Other.GetName() == LookFor;
					});
				if (ViewExtension->Sources.IsValidIndex(ViewExtensionSourceCreatorsIndex))
				{

					FMVVMViewClass_SourceKey FieldClassSourceKey = FMVVMViewClass_SourceKey(ViewExtensionSourceCreatorsIndex);
					SourceBitField |= FieldClassSourceKey.GetBit();
				}
			}
			for (TSharedPtr<FGeneratedReadFieldPathContext> ReadPath : ValidEvent->ReadPaths)
			{
				if (ReadPath->Source == nullptr)
				{
					AddMessageForEvent(ValidEvent->Event.Get(), LOCTEXT("InvalidEventSourceInternal", "Internal error. The event has an invalid source."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
					bIsCompileStepValid = false;
					continue;
				}

				int32 ViewExtensionSourceCreatorsIndex = ViewExtension->Sources.IndexOfByPredicate([LookFor = ReadPath->Source->Name](const FMVVMViewClass_Source& Other)
					{
						return Other.GetName() == LookFor;
					});
				if (!ViewExtension->Sources.IsValidIndex(ViewExtensionSourceCreatorsIndex))
				{
					AddMessageForEvent(ValidEvent->Event.Get(), LOCTEXT("CompiledEventSourceCreatorNotGenerated", "Internal error. The source creator was not generated."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
					bIsCompileStepValid = false;
					continue;
				}

				// Add the needed source.
				FMVVMViewClass_SourceKey FieldClassSourceKey = FMVVMViewClass_SourceKey(ViewExtensionSourceCreatorsIndex);
				SourceBitField |= FieldClassSourceKey.GetBit();
			}
		}

		FMVVMViewClass_Event& NewBinding = ViewExtension->Events.AddDefaulted_GetRef();
		NewBinding.FieldPath = *CompiledFieldPath;
		NewBinding.UserWidgetFunctionName = ValidEvent->GeneratedGraphName;
		NewBinding.SourceToReevaluate = FoundSourceIndex != INDEX_NONE ? FMVVMViewClass_SourceKey(FoundSourceIndex) : FMVVMViewClass_SourceKey();
		NewBinding.SourceBitField = SourceBitField;
	}
}

void FMVVMViewBlueprintCompiler::PreCompileConditions(UWidgetBlueprintGeneratedClass* Class)
{
	if (!GetDefault<UMVVMDeveloperProjectSettings>()->bAllowConditionBinding && BlueprintView->GetConditions().Num() > 0)
	{
		WidgetBlueprintCompilerContext.MessageLog.Warning(*LOCTEXT("ConditionsAreNotAllowed", "Condition bindings are not allowed in your project settings.").ToString());
	}

	for (TSharedRef<FCompilerCondition>& ValidCondition : ValidConditions)
	{
		UMVVMBlueprintViewCondition* ConditionPtr = ValidCondition->Condition.Get();
		check(ConditionPtr);
		UEdGraph* GeneratedGraph = ConditionPtr->GetOrCreateWrapperGraph();
		check(GeneratedGraph);

		// Does it resolve and are the field allowed
		TValueOrError<FCreateFieldsResult, FText> ConditionPathResult = CreateFieldContext(Class, ConditionPtr->GetConditionPath(), true);
		if (ConditionPathResult.HasError())
		{
			AddMessageForCondition(ConditionPtr
				, FText::Format(Private::PropertyPathIsInvalidFormat, PropertyPathToText(Class, BlueprintView.Get(), ConditionPtr->GetConditionPath()))
				, Compiler::EMessageType::Error
				, FMVVMBlueprintPinId());
			bIsPreCompileStepValid = false;
			continue;
		}

		FName SourceName;
		switch (ConditionPtr->GetConditionPath().GetSource(WidgetBlueprintCompilerContext.WidgetBlueprint()))
		{
		case EMVVMBlueprintFieldPathSource::SelfContext:
			SourceName = WidgetBlueprintCompilerContext.WidgetBlueprint()->GetFName();
			break;
		case EMVVMBlueprintFieldPathSource::Widget:
		{
			FName WidgetName = ConditionPtr->GetConditionPath().GetWidgetName();
			checkf(!WidgetName.IsNone(), TEXT("The destination should have been checked and set bAreSourceContextsValid."));
			const bool bSourceIsUserWidget = WidgetName == Class->ClassGeneratedBy->GetFName();
			ensure(!bSourceIsUserWidget);

			SourceName = WidgetName;
			break;
		}
		case EMVVMBlueprintFieldPathSource::ViewModel:
		{
			const FMVVMBlueprintViewModelContext* SourceViewModelContext = BlueprintView->FindViewModel(ConditionPtr->GetConditionPath().GetViewModelId());
			check(SourceViewModelContext);
			FName ViewModelName = SourceViewModelContext->GetViewModelName();
			SourceName = ViewModelName;
			break;
		}
		default:
			ensureAlwaysMsgf(false, TEXT("An EMVVMBlueprintFieldPathSource case was not checked."));
		}

		for (TSharedPtr<FGeneratedReadFieldPathContext> ReadPath : ValidCondition->ReadPaths)
		{
			if (ReadPath->NotificationField)
			{
				if (ReadPath->NotificationField->Source) 
				{
					const UClass* SourceContextClass = ReadPath->NotificationField->Source->AuthoritativeClass;
					TValueOrError<FCompiledBindingLibraryCompiler::FFieldIdHandle, FText> FieldIdResult = BindingLibraryCompiler.AddFieldId(SourceContextClass, ReadPath->NotificationField->NotificationId.GetFieldName());
					if (FieldIdResult.HasError() || !FieldIdResult.GetValue().IsValid())
					{
						AddMessageForCondition(ValidCondition
							, FText::Format(LOCTEXT("CouldNotCreateFieldId", "Could not create Field. {0}"), FieldIdResult.GetError())
							, Compiler::EMessageType::Error
							, FMVVMBlueprintPinId()
						);
						bIsPreCompileStepValid = false;
						continue;
					}

					ReadPath->NotificationField->LibraryCompilerHandle = FieldIdResult.StealValue();
				}
			}
		}

		// No need to add the generated function to the field compiler.
		//They are in the BP generated code.

		ValidCondition->GeneratedGraphName = ConditionPtr->GetWrapperGraphName();
		ValidCondition->SourceName = SourceName;
	}
}

void FMVVMViewBlueprintCompiler::CompileConditions(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension)
{
	// NB. The order is important. The index is used in the generated function to identify the event.
	for (const TSharedRef<FCompilerCondition>& ValidCondition : ValidConditions)
	{
		if (ValidCondition->GeneratedGraphName.IsNone() || Class->FindFunctionByName(ValidCondition->GeneratedGraphName) == nullptr)
		{
			AddMessageForCondition(ValidCondition->Condition.Get(), LOCTEXT("CompiledConditionFieldPathNotGenerated", "Could not generate the condition path."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
			bIsCompileStepValid = false;
			continue;
		}

		int32 FoundSourceIndex = INDEX_NONE;
		if (!ValidCondition->SourceName.IsNone())
		{
			FoundSourceIndex = ViewExtension->Sources.IndexOfByPredicate([ToFind = ValidCondition->SourceName](const FMVVMViewClass_Source& Other)
				{
					return Other.GetName() == ToFind;
				});
		}


		FMVVMViewClass_ConditionKey ConditionKey = FMVVMViewClass_ConditionKey(ViewExtension->Conditions.AddDefaulted());
		FMVVMViewClass_Condition& NewCondition = ViewExtension->Conditions[ConditionKey.GetIndex()];

		uint64 SourceBitField = 0;
		{
			for (TSharedPtr<FGeneratedReadFieldPathContext> ReadPath : ValidCondition->ReadPaths)
			{
				if (ReadPath->Source == nullptr)
				{
					AddMessageForCondition(ValidCondition->Condition.Get(), LOCTEXT("InvalidConditionSourceInternal", "Internal error. The condition has an invalid source."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
					bIsCompileStepValid = false;
					continue;
				}

				int32 ViewExtensionSourceCreatorsIndex = ViewExtension->Sources.IndexOfByPredicate([LookFor = ReadPath->Source->Name](const FMVVMViewClass_Source& Other)
					{
						return Other.GetName() == LookFor;
					});
				if (!ViewExtension->Sources.IsValidIndex(ViewExtensionSourceCreatorsIndex))
				{
					AddMessageForCondition(ValidCondition->Condition.Get(), LOCTEXT("CompiledConditionSourceCreatorNotGenerated", "Internal error. The source creator was not generated."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
					bIsCompileStepValid = false;
					continue;
				}

				// Add the needed source.
				FMVVMViewClass_SourceKey FieldClassSourceKey = FMVVMViewClass_SourceKey(ViewExtensionSourceCreatorsIndex);
				SourceBitField |= FieldClassSourceKey.GetBit();

				const bool bHasField = ReadPath->NotificationField != nullptr;
				const UE::FieldNotification::FFieldId* CompiledFieldId = ReadPath->NotificationField != nullptr ? CompileResult.FieldIds.Find(ReadPath->NotificationField->LibraryCompilerHandle) : nullptr;
				if (CompiledFieldId == nullptr && bHasField)
				{
					AddMessageForCondition(ValidCondition, LOCTEXT("CompiledFieldNotGenerated", "Internal error. The FieldId was not generated."), Compiler::EMessageType::Error, FMVVMBlueprintPinId());
					bIsCompileStepValid = false;
					continue;
				}

				FMVVMViewClass_Source& ClassSource = ViewExtension->Sources[ViewExtensionSourceCreatorsIndex];
				FMVVMViewClass_SourceCondition& NewSourceCondition = ClassSource.Conditions.AddDefaulted_GetRef();
				NewSourceCondition.ConditionKey = ConditionKey;

				if (CompiledFieldId != nullptr)
				{
					NewSourceCondition.FieldId = FFieldNotificationId(CompiledFieldId->GetName());
					ClassSource.FieldToRegisterTo.AddUnique(FMVVMViewClass_FieldId(*CompiledFieldId));
				}
			}
		}

		NewCondition.UserWidgetFunctionName = ValidCondition->GeneratedGraphName;
		NewCondition.SourceToReevaluate = FoundSourceIndex != INDEX_NONE ? FMVVMViewClass_SourceKey(FoundSourceIndex) : FMVVMViewClass_SourceKey();
		NewCondition.SourceBitField = SourceBitField;
	}
}

void FMVVMViewBlueprintCompiler::PreCompileViewExtensions(UWidgetBlueprintGeneratedClass* Class)
{
	struct FExposedCompiler : Compiler::IMVVMBlueprintViewPrecompile
	{
		FExposedCompiler(FMVVMViewBlueprintCompiler* InSelf, UWidgetBlueprintGeneratedClass* InClass)
			: Self(InSelf),
			Class(InClass)
		{}

		virtual const UMVVMBlueprintView* GetBlueprintView() const override
		{
			return Self->BlueprintView.Get();
		}

		virtual const TMap<FName, UWidget*>& GetWidgetNameToWidgetPointerMap() const override
		{
			return Self->WidgetNameToWidgetPointerMap;
		}

		virtual TArray<Compiler::FCompilerBindingHandle> GetAllBindings() const override
		{
			TArray<Compiler::FCompilerBindingHandle> BindingHandles;
			for (const TSharedRef<FCompilerBinding>& Binding : Self->ValidBindings)
			{
				BindingHandles.Add(Binding->CompilerBindingHandle);
			}

			return BindingHandles;
		}

		virtual TArray<TArray<UE::MVVM::FMVVMConstFieldVariant>> GetBindingReadFields(Compiler::FCompilerBindingHandle BindingHandle) override
		{
			TArray<TArray<UE::MVVM::FMVVMConstFieldVariant>> BindingReadFields;
			const TSharedRef<FCompilerBinding>* Binding = Self->ValidBindings.FindByPredicate([BindingHandle](const TSharedRef<FCompilerBinding>& OtherBinding) { return OtherBinding->CompilerBindingHandle == BindingHandle; });
			if (Binding)
			{
				for (const TSharedPtr<FGeneratedReadFieldPathContext>& ReadPath : Binding->Get().ReadPaths)
				{
					BindingReadFields.Add(ReadPath->SkeletalGeneratedFields);
				}
			}

			return BindingReadFields;
		}

		virtual TArray<UE::MVVM::FMVVMConstFieldVariant> GetBindingWriteFields(Compiler::FCompilerBindingHandle BindingHandle) override
		{
			const TSharedRef<FCompilerBinding>* Binding = Self->ValidBindings.FindByPredicate([BindingHandle](const TSharedRef<FCompilerBinding>& OtherBinding) { return OtherBinding->CompilerBindingHandle == BindingHandle; });
			if (Binding)
			{
				if (Binding->Get().WritePath)
				{
					return Binding->Get().WritePath->SkeletalGeneratedFields;
				}
			}

			return TArray<UE::MVVM::FMVVMConstFieldVariant>();
		}

		virtual const FProperty* GetBindingSourceProperty(Compiler::FCompilerBindingHandle BindingHandle) override
		{
			const TSharedRef<FCompilerBinding>* Binding = Self->ValidBindings.FindByPredicate([BindingHandle](const TSharedRef<FCompilerBinding>& OtherBinding) { return OtherBinding->CompilerBindingHandle == BindingHandle; });
			if (Binding)
			{
				if (Binding->Get().Type == FCompilerBinding::EType::ComplexConversionFunction
					|| Binding->Get().Type == FCompilerBinding::EType::SimpleConversionFunction)
				{
					if (const UFunction* CompiledFunction = Binding->Get().ConversionFunction->GetCompiledFunction(Class))
					{
						TValueOrError<const FProperty*, FText> ReturnProperty = UE::MVVM::BindingHelper::TryGetReturnTypeForConversionFunction(CompiledFunction);
						if (ReturnProperty.HasValue())
						{
							return ReturnProperty.GetValue();
						}
					}
				}
				else if (Binding->Get().Type == FCompilerBinding::EType::Assignment)
				{
					if (Binding->Get().ReadPaths.Num() > 0)
					{
						if (const TSharedPtr<FGeneratedReadFieldPathContext>& ReadPath = Binding->Get().ReadPaths[0])
						{
							if (ReadPath->SkeletalGeneratedFields.Num() > 0)
							{
								UE::MVVM::FMVVMConstFieldVariant LastSourceField = ReadPath->SkeletalGeneratedFields.Last();
								if (LastSourceField.IsProperty())
								{
									return LastSourceField.GetProperty();
								}
								else if (LastSourceField.IsFunction())
								{
									return BindingHelper::GetReturnProperty(LastSourceField.GetFunction());
								}
							}
						}
					}
				}
			}

			return nullptr;
		}

		virtual TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> AddObjectFieldPath(const UE::MVVM::Compiler::IMVVMBlueprintViewPrecompile::FObjectFieldPathArgs& Args) override
		{
			return UE::MVVM::Private::AddObjectFieldPath(Self->BindingLibraryCompiler, Args.Class, Args.ObjectPath, Args.ExpectedType);
		}

		virtual TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> AddFieldPath(TArrayView<const FMVVMConstFieldVariant> InFieldPath, bool bInRead) override
		{
			return Self->BindingLibraryCompiler.AddFieldPath(InFieldPath, bInRead);
		}

		virtual void AddMessageForBinding(Compiler::FCompilerBindingHandle BindingHandle, const FText& MessageText, Compiler::EMessageType MessageType) const override
		{
			const TSharedRef<FCompilerBinding>* Binding = Self->ValidBindings.FindByPredicate([BindingHandle](const TSharedRef<FCompilerBinding>& OtherBinding) { return OtherBinding->CompilerBindingHandle == BindingHandle; });
			if (Binding)
			{
				Self->AddMessageForBinding(*Binding, MessageText, MessageType, FMVVMBlueprintPinId());
			}
		}

		virtual void AddMessage(const FText& MessageText, Compiler::EMessageType MessageType) override
		{
			Self->AddMessage(MessageText, MessageType);
		}

		virtual void MarkPrecompileStepInvalid() override
		{
			Self->bIsPreCompileStepValid = false;
		}

		virtual ~FExposedCompiler() {};

	private:
		FMVVMViewBlueprintCompiler* Self = nullptr;
		UWidgetBlueprintGeneratedClass* Class = nullptr;
	};

	FExposedCompiler ExposedCompiler(this, Class);

	for (TSharedRef<FCompilerExtension>& Extension : ValidExtensions)
	{
		UMVVMBlueprintViewExtension* ExtensionPtr = Extension->Extension.Get();
		if (ensure(ExtensionPtr))
		{
			ExtensionPtr->Precompile(&ExposedCompiler, Class);
		}
	}
}

void FMVVMViewBlueprintCompiler::CompileViewExtensions(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension)
{
	struct FExposedCompiler : Compiler::IMVVMBlueprintViewCompile
	{
		FExposedCompiler(FMVVMViewBlueprintCompiler* InSelf, const FCompiledBindingLibraryCompiler::FCompileResult& InCompileResult, UMVVMViewClass* InViewExtension)
			: Self(InSelf)
			, CompileResult(InCompileResult)
			, ViewClass(InViewExtension)
		{}

		virtual const UMVVMBlueprintView* GetBlueprintView() const override
		{
			return Self->BlueprintView.Get();
		}

		virtual const TMap<FName, UWidget*>& GetWidgetNameToWidgetPointerMap() const override
		{
			return Self->WidgetNameToWidgetPointerMap;
		}

		virtual TValueOrError<FMVVMVCompiledFieldPath, void> GetFieldPath(FCompiledBindingLibraryCompiler::FFieldPathHandle FieldPath) override
		{
			if (FMVVMVCompiledFieldPath* CompiledPath = CompileResult.FieldPaths.Find(FieldPath))
			{
				return MakeValue(*CompiledPath);
			}
			else
			{
				return MakeError();
			}
		}

		virtual void AddMessageForBinding(Compiler::FCompilerBindingHandle BindingHandle, const FText& MessageText, Compiler::EMessageType MessageType) const override
		{
			const TSharedRef<FCompilerBinding>* Binding = Self->ValidBindings.FindByPredicate([BindingHandle](const TSharedRef<FCompilerBinding>& OtherBinding) { return OtherBinding->CompilerBindingHandle == BindingHandle; });
			if (Binding)
			{
				Self->AddMessageForBinding(*Binding, MessageText, MessageType, FMVVMBlueprintPinId());
			}
		}

		virtual void AddMessage(const FText& MessageText, Compiler::EMessageType MessageType) override
		{
			Self->AddMessage(MessageText, MessageType);
		}

		virtual void MarkCompileStepInvalid() override
		{
			Self->bIsCompileStepValid = false;
		}

		virtual UMVVMViewClassExtension* CreateViewClassExtension(TSubclassOf<UMVVMViewClassExtension> ExtensionClass)
		{
			return Self->CreateViewClassExtension(ExtensionClass, ViewClass);
		}

		virtual ~FExposedCompiler() {};

	private:
		FMVVMViewBlueprintCompiler* Self = nullptr;
		FCompiledBindingLibraryCompiler::FCompileResult CompileResult;
		UMVVMViewClass* ViewClass = nullptr;
	};

	FExposedCompiler ExposedCompiler(this, CompileResult, ViewExtension);

	for (TSharedRef<FCompilerExtension>& Extension : ValidExtensions)
	{
		UMVVMBlueprintViewExtension* ExtensionPtr = Extension->Extension.Get();
		if (ensure(ExtensionPtr))
		{
			ExtensionPtr->Compile(&ExposedCompiler, Class, ViewExtension);
		}
	}
}

void FMVVMViewBlueprintCompiler::PreCompileSourceDependencies(UWidgetBlueprintGeneratedClass* Class)
{
	//i.	ViewmodelA.ViewmodelB = ViewmodelC.Value				-> ViewmodelA needs to init before ViewmodelA_ViewmodelB, ViewmodelA before ViewmodelC, ViewmodelC before ViewmodelA_ViewmodelB
	//ii.	ViewmodelA.ViewmodelB.Value = ViewmodelC.Value			-> ViewmodelA needs to init before ViewmodelA_ViewmodelB, ViewmodelA before ViewmodelC, ViewmodelA_ViewmodelB before ViewmodelC

	// Dynamic sources depends on their parent. (i and ii ViewmodelA before ViewmodelA_ViewmodelB)
	for (const TSharedRef<FCompilerSourceViewModelDynamicCreatorContext>& DynamicContext : SourceViewModelDynamicCreatorContexts)
	{
		ensure(DynamicContext->Source);
		ensure(DynamicContext->ParentSource);
		DynamicContext->Source->Dependencies.AddUnique(DynamicContext->ParentSource);
	}

	for (const TSharedRef<FCompilerBinding>& ValidBinding : ValidBindings)
	{
		if (ValidBinding->WritePath)
		{
			for (const TSharedPtr<FGeneratedReadFieldPathContext>& ReadPath : ValidBinding->ReadPaths)
			{
				// All the read depends on the write (i and ii ViewmodelA before ViewmodelC)
				if (ensure(ReadPath->Source))
				{
					if (ValidBinding->WritePath->OptionalSource)
					{
						ReadPath->Source->Dependencies.AddUnique(ValidBinding->WritePath->OptionalSource);
					}

					// The write depends on the read (i ViewmodelC before ViewmodelA_ViewmodelB)
					if (ValidBinding->WritePath->OptionalDependencySource)
					{
						ValidBinding->WritePath->OptionalDependencySource->Dependencies.AddUnique(ReadPath->Source);
					}
				}

				// NB (ii ViewmodelA_ViewmodelB before ViewmodelC) is implicit
			}
		}
	}
}


void FMVVMViewBlueprintCompiler::SortSourceFields(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension)
{
	for (FMVVMViewClass_Source& Source : ViewExtension->Sources)
	{
		for (const FMVVMViewClass_FieldId& Field : Source.FieldToRegisterTo)
		{
			if (Field.GetFieldId().GetIndex() == INDEX_NONE)
			{
				AddMessage(LOCTEXT("SortSourceFieldsInvalidId", "Internal error. The id is invalid."), Compiler::EMessageType::Error);
				bIsCompileStepValid = false;
				continue;
			}
		}

		Source.FieldToRegisterTo.StableSort([](const FMVVMViewClass_FieldId& A, const FMVVMViewClass_FieldId& B)
			{
				return A.GetFieldId().GetIndex() < B.GetFieldId().GetIndex();
			});
	}
}


TValueOrError<FMVVMViewBlueprintCompiler::FGetFieldsResult, FText> FMVVMViewBlueprintCompiler::GetFields(const UWidgetBlueprintGeneratedClass* Class, const FMVVMBlueprintPropertyPath& PropertyPath) const
{
	FGetFieldsResult Result;
	if (!PropertyPath.IsValid())
	{
		ensureAlwaysMsgf(false, TEXT("Empty property path found. It should have been catch before."));
		return MakeError(FText::GetEmpty());
	}

	auto FindSource = [Self = this](const FName PropertyName, FCompilerBindingSource::EType ExpectedType) -> TValueOrError<TSharedPtr<FCompilerBindingSource>, FText>
	{
		const TSharedRef<FCompilerBindingSource>* Found = Self->NeededBindingSources.FindByPredicate([PropertyName](const TSharedRef<FCompilerBindingSource>& Other) { return Other->Name == PropertyName; });
		if (Found)
		{
			if ((*Found)->Type != ExpectedType)
			{
				return MakeError(LOCTEXT("NotExpectedSourceType", "Internal error. The source of the path is not of the expected type."));
			}
			return MakeValue(*Found);
		}
		// It can be valid to not have a BindingSource, if it's a widget in a WritePath
		return MakeValue(TSharedPtr<FCompilerBindingSource>());
	};

	// If the path is "self.widget.property", "self.viewmodel.property", then remove the "self".
	EMVVMBlueprintFieldPathSource FieldPathSource = PropertyPath.GetSource(WidgetBlueprintCompilerContext.WidgetBlueprint());
	TArray<UE::MVVM::FMVVMConstFieldVariant> PropertyPathFields = PropertyPath.GetFields(Class);
	FGuid ViewModelId = PropertyPath.GetViewModelId();
	FName DestinationWidgetName = PropertyPath.GetWidgetName();
	bool bIsComponentOnSelf = false;
	if (PropertyPath.IsComponent())
	{
		const UWidgetBlueprintGeneratedClass* ComponentOwningWidgetBlueprint = UE::MVVM::FieldPathHelper::GetComponentPropertyPathSource(PropertyPath.GetFields(Class), Class);
		bIsComponentOnSelf = ComponentOwningWidgetBlueprint == Class;
	}

	if (FieldPathSource == EMVVMBlueprintFieldPathSource::SelfContext)
	{
		if (PropertyPathFields.Num() >= 2 && PropertyPathFields[0].IsProperty() && !bIsComponentOnSelf)
		{
			FName FieldName = PropertyPathFields[0].GetName();
			if (WidgetNameToWidgetPointerMap.Contains(FieldName))
			{
				FieldPathSource = EMVVMBlueprintFieldPathSource::Widget;
				DestinationWidgetName = FieldName;
				PropertyPathFields.RemoveAt(0);
			}
			else if (const FMVVMBlueprintViewModelContext* SourceViewModelContext = BlueprintView->FindViewModel(FieldName))
			{
				FieldPathSource = EMVVMBlueprintFieldPathSource::ViewModel;
				ViewModelId = SourceViewModelContext->GetViewModelId();
				PropertyPathFields.RemoveAt(0);
			}
		}
	}

	// Build the full path.
	switch (FieldPathSource)
	{
	case EMVVMBlueprintFieldPathSource::ViewModel:
	{
		const FCompilerViewModelCreatorContext* FoundViewModelCreator = ViewModelCreatorContexts.FindByPredicate([ViewModelId](const FCompilerViewModelCreatorContext& Other){ return Other.ViewModelContext.GetViewModelId() == ViewModelId; });
		check(FoundViewModelCreator);
		const FName PropertyName = FoundViewModelCreator->ViewModelContext.GetViewModelName();
		const FCompilerUserWidgetProperty* FoundUserWidgetProperty = NeededUserWidgetProperties.FindByPredicate([PropertyName](const FCompilerUserWidgetProperty& Other){ return Other.Name == PropertyName; });
		check(FoundUserWidgetProperty);
		check(FoundViewModelCreator->ViewModelContext.GetViewModelClass());
		check(FoundUserWidgetProperty->AuthoritativeClass == FoundViewModelCreator->ViewModelContext.GetViewModelClass());

		TValueOrError<TSharedPtr<FCompilerBindingSource>, FText> FindSourceResult = FindSource(PropertyName, FCompilerBindingSource::EType::ViewModel);
		if (FindSourceResult.HasError())
		{
			return MakeError(FindSourceResult.StealError());
		}
		if (!FindSourceResult.GetValue().IsValid())
		{
			return MakeError(LOCTEXT("ViewModelShouldHaveSource", "Internal error. Viewmodel should have a source."));
		}

		Result.GeneratedFields = AppendBaseField(Class, PropertyName, PropertyPathFields);
		Result.OptionalSource = FindSourceResult.StealValue();
		break;
	}
	case EMVVMBlueprintFieldPathSource::SelfContext:
	{
		TValueOrError<TSharedPtr<FCompilerBindingSource>, FText> FindSourceResult = FindSource(WidgetBlueprintCompilerContext.WidgetBlueprint()->GetFName(), FCompilerBindingSource::EType::Self);
		if (FindSourceResult.HasError())
		{
			return MakeError(FindSourceResult.StealError());
		}
		if (!FindSourceResult.GetValue().IsValid())
		{
			return MakeError(LOCTEXT("WidgetBlueprintShouldHaveSource", "Internal error. The blueprint should have a source."));
		}

		Result.GeneratedFields = AppendBaseField(Class, FName(), PropertyPathFields);
		Result.OptionalSource = FindSourceResult.StealValue();
		break;
	}
	case EMVVMBlueprintFieldPathSource::Widget:
	{
		check(WidgetNameToWidgetPointerMap.Contains(DestinationWidgetName));
		checkf(!DestinationWidgetName.IsNone(), TEXT("The destination should have been checked and set bAreSourceContextsValid."));

		const FCompilerUserWidgetProperty* FoundUserWidgetProperty = NeededUserWidgetProperties.FindByPredicate([DestinationWidgetName](const FCompilerUserWidgetProperty& Other) { return Other.Name == DestinationWidgetName; });
		check(FoundUserWidgetProperty);

		TValueOrError<TSharedPtr<FCompilerBindingSource>, FText> FindSourceResult = bIsComponentOnSelf ? FindSource(WidgetBlueprintCompilerContext.WidgetBlueprint()->GetFName(), FCompilerBindingSource::EType::Self) : FindSource(DestinationWidgetName, FCompilerBindingSource::EType::Widget);
		if (FindSourceResult.HasError())
		{
			return MakeError(FindSourceResult.StealError());
		}

		Result.GeneratedFields = bIsComponentOnSelf ? AppendBaseField(Class, FName(), PropertyPathFields) : AppendBaseField(Class, DestinationWidgetName, PropertyPathFields);
		Result.OptionalSource = FindSourceResult.StealValue();
		break;
	}
	default:
		ensureAlwaysMsgf(false, TEXT("Not supported."));
		Result.GeneratedFields = AppendBaseField(Class, FName(), PropertyPathFields);
		break;
	}

	Result.GeneratedFrom = FieldPathSource;
	return MakeValue(MoveTemp(Result));
}


TArray<FMVVMConstFieldVariant> FMVVMViewBlueprintCompiler::AppendBaseField(const UClass* Class, FName PropertyName, TArray<FMVVMConstFieldVariant> Properties)
{
	if (PropertyName.IsNone())
	{
		return Properties;
	}

	check(Class);
	FMVVMConstFieldVariant NewProperty = BindingHelper::FindFieldByName(Class, FMVVMBindingName(PropertyName));
	Properties.Insert(NewProperty, 0);
	return Properties;
}


TValueOrError<FMVVMViewBlueprintCompiler::FCreateFieldsResult, FText> FMVVMViewBlueprintCompiler::CreateFieldContext(const UWidgetBlueprintGeneratedClass* Class, const FMVVMBlueprintPropertyPath& PropertyPath, bool bForSourceReading) const
{
	FMVVMViewBlueprintCompiler::FCreateFieldsResult Result;

	// Evaluate the getter/setter path.
	{
		TValueOrError<FGetFieldsResult, FText> GetFieldResult = GetFields(Class, PropertyPath);
		if (GetFieldResult.HasError())
		{
			return MakeError(GetFieldResult.StealError());
		}

		Result.GeneratedFrom = MoveTemp(GetFieldResult.GetValue().GeneratedFrom);
		Result.OptionalSource = MoveTemp(GetFieldResult.GetValue().OptionalSource);
		Result.GeneratedFields = MoveTemp(GetFieldResult.GetValue().GeneratedFields);
		Result.bIsComponent = PropertyPath.IsComponent();
	}

	if (!IsPropertyPathValid(WidgetBlueprintCompilerContext.WidgetBlueprint(), Result.GeneratedFields))
	{
		return MakeError(FText::Format(Private::PropertyPathIsInvalidFormat, PropertyPathToText(Class, BlueprintView.Get(), PropertyPath)));
	}

	// Generate the path with property converted to BP function
	TValueOrError<TArray<FMVVMConstFieldVariant>, FText> SkeletalGeneratedFieldsResult = FieldPathHelper::GenerateFieldPathList(Result.GeneratedFields, bForSourceReading);
	if (SkeletalGeneratedFieldsResult.HasError())
	{
		return MakeError(FText::Format(Private::CouldNotCreateSourceFieldPathFormat, PropertyPathToText(Class, BlueprintView.Get(), PropertyPath), SkeletalGeneratedFieldsResult.GetError()));
	}
	if (!IsPropertyPathValid(WidgetBlueprintCompilerContext.WidgetBlueprint(), SkeletalGeneratedFieldsResult.GetValue()))
	{
		return MakeError(FText::Format(Private::CouldNotCreateSourceFieldPathFormat, PropertyPathToText(Class, BlueprintView.Get(), PropertyPath), LOCTEXT("NotAValidPropertyPath", "The path is not valid.")));
	}
	Result.SkeletalGeneratedFields = SkeletalGeneratedFieldsResult.StealValue();

	return MakeValue(Result);
}


TValueOrError<TSharedPtr<FMVVMViewBlueprintCompiler::FCompilerNotifyFieldId>, FText> FMVVMViewBlueprintCompiler::CreateNotifyFieldId(const UWidgetBlueprintGeneratedClass* Class, const TSharedPtr<FGeneratedReadFieldPathContext>& ReadFieldContext)
{
	check(ReadFieldContext->SkeletalGeneratedFields.Num() > 0);

	// The path may contains another INotifyFieldValueChanged
	TValueOrError<FieldPathHelper::FParsedNotifyBindingInfo, FText> BindingInfoResult = FieldPathHelper::GetNotifyBindingInfoFromFieldPath(Class, ReadFieldContext->SkeletalGeneratedFields);
	if (BindingInfoResult.HasError())
	{
		return MakeError(BindingInfoResult.StealError());
	}

	const FieldPathHelper::FParsedNotifyBindingInfo& BindingInfo = BindingInfoResult.GetValue();
	if (!BindingInfo.NotifyFieldId.IsValid() || ReadFieldContext->Source == nullptr)
	{
		return MakeValue(TSharedPtr<FCompilerNotifyFieldId>());
	}

	// the path is UserWidget->Widget->Property. THe source is the UserWidget but it should not, UserWidget shuould be remove from the path.
	FCompilerNotifyFieldId Result;
	Result.NotificationId = BindingInfo.NotifyFieldId;
	Result.Source = ReadFieldContext->Source;
	Result.ViewModelDynamic.Reset();

	auto GetClassFromField = [](UE::MVVM::FMVVMConstFieldVariant Field) -> const UClass*
	{
		if (Field.IsProperty())
		{
			const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Field.GetProperty());
			if (ObjectProperty)
			{
				return ObjectProperty->PropertyClass;
			}
		}
		else if (Field.IsFunction())
		{
			const FObjectPropertyBase* ReturnValue = CastField<const FObjectPropertyBase>(BindingHelper::GetReturnProperty(Field.GetFunction()));
			if (ReturnValue)
			{
				return ReturnValue->PropertyClass;
			}
		}
		return nullptr;
	};

	// If the source type is Self, we don't include "self" as a field in SkeletalGeneratedFields, so the ViewModelIndex will not consider that. Meanwhile, we do include the widget if source type is Widget.
	// So a ViewModelIndex of 0 with source type Self is equivalent to a ViewModelIndex of 1 with source type widget.
 	bool bIsSelf = ReadFieldContext->Source->Type == FCompilerBindingSource::EType::Self && ReadFieldContext->SkeletalGeneratedFields.Num() > 1;
	bool bIsComponentOnSelf = false;
	if (ReadFieldContext->bIsComponent)
	{
		const UWidgetBlueprintGeneratedClass* ComponentOwningWidgetBlueprint = UE::MVVM::FieldPathHelper::GetComponentPropertyPathSource(ReadFieldContext->SkeletalGeneratedFields, Class);
		bIsComponentOnSelf = ComponentOwningWidgetBlueprint == Class;
	}

	int32 ViewModelIndexIncludingSource = (bIsSelf || bIsComponentOnSelf) ? BindingInfo.ViewModelIndex + 1 : BindingInfo.ViewModelIndex;
	// Sanity check
	{
		const UClass* ExpectedClass = nullptr;
		if (ViewModelIndexIncludingSource < 1 && BindingInfo.NotifyFieldClass)
		{
			ExpectedClass = ReadFieldContext->Source->AuthoritativeClass;
		}
		else if (ViewModelIndexIncludingSource >= 0 && BindingInfo.ViewModelIndex != INDEX_NONE)
		{
			ExpectedClass = GetClassFromField(ReadFieldContext->SkeletalGeneratedFields[BindingInfo.ViewModelIndex]);
		}

		if (ExpectedClass == nullptr || BindingInfo.NotifyFieldClass == nullptr || !ExpectedClass->IsChildOf(BindingInfo.NotifyFieldClass))
		{
			return MakeError(LOCTEXT("InvalidNotifyFieldClassInternal", "Internal error. The viewmodel class doesn't matches."));
		}
	}

	// The INotifyFieldValueChanged/viewmodel is not the first and only INotifyFieldValueChanged/viewmodel property path.
	//Create a new source in PropertyPath creator mode. Create a special binding to update the viewmodel when it changes.
	//This binding (calling this function) will use the new source.
	if (ViewModelIndexIncludingSource >= 1)
	{
		if (!GetDefault<UMVVMDeveloperProjectSettings>()->bAllowLongSourcePath)
		{
			return MakeError(LOCTEXT("DynamicSourceEntryNotSupport", "Long source entry is not supported. Add the viewmodel manually."));
		}

		for (int32 DynamicIndex = (bIsSelf || bIsComponentOnSelf) && BindingInfo.ViewModelIndex == 0 ? 0 : 1 ; DynamicIndex <= BindingInfo.ViewModelIndex; ++DynamicIndex)
		{
			if (!ReadFieldContext->SkeletalGeneratedFields.IsValidIndex(DynamicIndex))
			{
				return MakeError(LOCTEXT("DynamicSourceEntryInternalIndex", "Internal error. The source index is not valid."));
			}

			FName NewSourceName;
			FName NewParentSourceName;
			FString NewSourcePropertyPath;
			const UClass* NewSourceAuthoritativeClass = nullptr;
			{
				TStringBuilder<512> PropertyPathBuilder;
				TStringBuilder<512> DynamicNameBuilder;
				for (int32 Index = 0; Index <= DynamicIndex; ++Index)
				{
					if (Index > 0)
					{
						PropertyPathBuilder << TEXT('.');
						DynamicNameBuilder << TEXT('_');
					}
					PropertyPathBuilder << ReadFieldContext->SkeletalGeneratedFields[Index].GetName();
					DynamicNameBuilder << ReadFieldContext->SkeletalGeneratedFields[Index].GetName();

					if (Index == DynamicIndex - 1)
					{
						NewParentSourceName = FName(DynamicNameBuilder.ToString());
					}
				}

				NewSourceName = FName(DynamicNameBuilder.ToString());
				NewSourcePropertyPath = PropertyPathBuilder.ToString();

				const UClass* OwnerSkeletalClass = GetClassFromField(ReadFieldContext->SkeletalGeneratedFields[DynamicIndex]);
				if (OwnerSkeletalClass == nullptr)
				{
					return MakeError(LOCTEXT("DVM_GeneratedFieldInvalid", "Internal error. The GeneratedFiled is invalid."));
				}
				NewSourceAuthoritativeClass = OwnerSkeletalClass->GetAuthoritativeClass();
				if (NewSourceAuthoritativeClass == nullptr)
				{
					return MakeError(LOCTEXT("DVM_AuthoritativeClassInvalid", "Internal error. No authoritative class."));
				}
			}

			if (bIsComponentOnSelf || bIsSelf)
			{
				NewParentSourceName = NewParentSourceName.IsNone() ? Class->ClassGeneratedBy->GetFName() : NewParentSourceName;
			}

			// Does the parent exist
			TSharedPtr<FCompilerBindingSource> ParentBindingSource;
			{
				const TSharedRef<FCompilerBindingSource>* FoundParentBindingSource = NeededBindingSources.FindByPredicate([NewParentSourceName](const TSharedRef<FCompilerBindingSource>& Other)
					{
						return Other->Name == NewParentSourceName;
					});
				if (FoundParentBindingSource == nullptr)
				{
					return MakeError(LOCTEXT("DVM_InvalidParentBindingSource", "Internal error. Can't find the parent binding source."));
				}
				ParentBindingSource = *FoundParentBindingSource;
			}

			// Did we already create the new source. It could be a dynamic or one added by the user
			{
				const TSharedRef<FCompilerBindingSource>* FoundBindingSource = NeededBindingSources.FindByPredicate([NewSourceName](const TSharedRef<FCompilerBindingSource>& Other)
					{
						return Other->Name == NewSourceName;
					});
				if (FoundBindingSource)
				{
					if ((*FoundBindingSource)->Type != FCompilerBindingSource::EType::DynamicViewmodel)
					{
						return MakeError(LOCTEXT("DVM_ViewmodelOfSameName", "The dynamic viewmodel cannot be added. There is already a source with that name."));
					}
					// is the class the same
					if (!(*FoundBindingSource)->AuthoritativeClass->IsChildOf(NewSourceAuthoritativeClass))
					{
						return MakeError(LOCTEXT("DVM_ExistingNotSameClass", "Internal error. The viewmodel was already added and is not the same type."));
					}
				}

				const TSharedRef<FCompilerSourceViewModelDynamicCreatorContext>* FoundViewModelDynamicCreatorContext = SourceViewModelDynamicCreatorContexts.FindByPredicate([NewSourceName](const TSharedRef<FCompilerSourceViewModelDynamicCreatorContext>& Other)
					{
						return Other->Source->Name == NewSourceName;
					});
				if (FoundViewModelDynamicCreatorContext)
				{
					if (FoundBindingSource == nullptr)
					{
						return MakeError(LOCTEXT("DVM_BindingSourceShouldExist", "Internal error. The binding source should exist."));
					}
					if ((*FoundViewModelDynamicCreatorContext)->Source != *FoundBindingSource)
					{
						return MakeError(LOCTEXT("DVM_BindingSourceShouldBeTheSame", "Internal error. The source should be the same."));
					}
					if ((*FoundViewModelDynamicCreatorContext)->ParentSource->Name != NewParentSourceName)
					{
						return MakeError(LOCTEXT("DVM_BindingSourceParentSameName", "Internal error. The parent name should be the same."));
					}
				}

				const FCompilerViewModelCreatorContext* FoundViewModelCreatorContext = ViewModelCreatorContexts.FindByPredicate([NewSourceName](const FCompilerViewModelCreatorContext& Other)
					{
						return Other.ViewModelContext.GetViewModelName() == NewSourceName;
					});
				if (FoundViewModelCreatorContext)
				{
					if (FoundBindingSource == nullptr)
					{
						return MakeError(LOCTEXT("DVM_BindingSourceShouldExist", "Internal error. The binding source should exist."));
					}
					if (FoundViewModelCreatorContext->Source != *FoundBindingSource)
					{
						return MakeError(LOCTEXT("DVM_BindingSourceShouldBeTheSame", "Internal error. The source should be the same."));
					}
					if (FoundViewModelDynamicCreatorContext == nullptr)
					{
						return MakeError(LOCTEXT("DVM_DynamicCreatorContextShouldExit", "Internal error. The creator context should exist."));
					}
					if (FoundViewModelCreatorContext->DynamicContext != *FoundViewModelDynamicCreatorContext)
					{
						return MakeError(LOCTEXT("DVM_DynamicContextSHouldBeSame", "Internal error. The creator context should be the same."));
					}
					if (FoundViewModelCreatorContext->ViewModelContext.CreationType != EMVVMBlueprintViewModelContextCreationType::PropertyPath)
					{
						return MakeError(LOCTEXT("DVM_DynamicContextShouldBePropertyPath", "Internal error. The existing creator context should use a property path."));
					}
					if (FoundViewModelCreatorContext->ViewModelContext.ViewModelPropertyPath != NewSourcePropertyPath)
					{
						return MakeError(LOCTEXT("DVM_DynamicContextSamePropertyPath", "Internal error. The existing creator context use the same property path."));
					}
				}

				if (FoundBindingSource && FoundViewModelCreatorContext == nullptr && FoundViewModelDynamicCreatorContext == nullptr)
				{
					return MakeError(LOCTEXT("DVM_MissingDefinition", "Internal error. There are missing definition for the dynamic viewmodel."));
				}

				if (FoundBindingSource)
				{
					Result.Source = *FoundBindingSource;
					Result.ViewModelDynamic = *FoundViewModelDynamicCreatorContext;
					continue; // already exist and correct to use.
				}
			}

			if (!NewSourceAuthoritativeClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
			{
				return MakeError(LOCTEXT("DVM_NewDynamicNotViewmodel", "The dynamic viewmodel is not an actual viewmodel."));
			}

			// Create the new source
			TSharedRef<FCompilerBindingSource> NewBindingSource = MakeShared<FCompilerBindingSource>();
			NewBindingSource->AuthoritativeClass = NewSourceAuthoritativeClass;
			NewBindingSource->Name = NewSourceName;
			NewBindingSource->Type = FCompilerBindingSource::EType::DynamicViewmodel;
			NewBindingSource->bIsOptional = ParentBindingSource->bIsOptional;
			NeededBindingSources.Add(NewBindingSource);

			Result.Source = NewBindingSource;

			TSharedRef<FCompilerSourceViewModelDynamicCreatorContext> NewViewModelDynamic = MakeShared<FCompilerSourceViewModelDynamicCreatorContext>();
			NewViewModelDynamic->Source = NewBindingSource;
			NewViewModelDynamic->ParentSource = ParentBindingSource;
			NewViewModelDynamic->NotificationId = FFieldNotificationId(ReadFieldContext->SkeletalGeneratedFields[DynamicIndex].GetName());
			SourceViewModelDynamicCreatorContexts.Add(NewViewModelDynamic);

			Result.ViewModelDynamic = NewViewModelDynamic;

			FCompilerViewModelCreatorContext& NewViewModelCreatorContext = ViewModelCreatorContexts.AddDefaulted_GetRef();
			NewViewModelCreatorContext.ViewModelContext = FMVVMBlueprintViewModelContext(NewSourceAuthoritativeClass, NewSourceName);
			NewViewModelCreatorContext.ViewModelContext.bCreateSetterFunction = false;
			NewViewModelCreatorContext.ViewModelContext.bOptional = NewBindingSource->bIsOptional;
			NewViewModelCreatorContext.ViewModelContext.CreationType = EMVVMBlueprintViewModelContextCreationType::PropertyPath;
			NewViewModelCreatorContext.ViewModelContext.ViewModelPropertyPath = NewSourcePropertyPath;
			NewViewModelCreatorContext.Source = NewBindingSource;
			NewViewModelCreatorContext.DynamicContext = NewViewModelDynamic;
		}
	}

	return MakeValue(MakeShared<FCompilerNotifyFieldId>(Result));
}

UMVVMViewClassExtension* FMVVMViewBlueprintCompiler::CreateViewClassExtension(TSubclassOf<UMVVMViewClassExtension> ExtensionClass, UMVVMViewClass* ViewClass)
{
	if (ensure(ExtensionClass.Get()))
	{
		UObject* ExtensionObj = NewObject<UObject>(ViewClass, ExtensionClass.Get(), NAME_None);
		UMVVMViewClassExtension* Extension = CastChecked<UMVVMViewClassExtension>(ExtensionObj);
		ViewClass->ViewClassExtensions.Add(Extension);
		return Extension;
	}
	return nullptr;
}

bool FMVVMViewBlueprintCompiler::IsPropertyPathValid(const UBlueprint* Context, TArrayView<const FMVVMConstFieldVariant> PropertyPath)
{
	const UStruct* CurrentContainer = Context->GeneratedClass ? Context->GeneratedClass : Context->SkeletonGeneratedClass;
	int32 PathLength = PropertyPath.Num();
	for (int32 Index = 0; Index < PathLength; Index++)
	{
		const FMVVMConstFieldVariant& Field = PropertyPath[Index];

		if (CurrentContainer == nullptr)
		{
			return false;
		}
		if (Field.IsEmpty())
		{
			return false;
		}
		if (Field.IsProperty())
		{
			if (Field.GetProperty() == nullptr)
			{
				return false;
			}

			if (!GetDefault<UMVVMDeveloperProjectSettings>()->IsPropertyAllowed(Context, CurrentContainer, Field.GetProperty()))
			{
				return false;
			}
		}
		if (Field.IsFunction())
		{
			if (Field.GetFunction() == nullptr)
			{
				return false;
			}
			const UClass* CurrentContainerAsClass = Cast<const UClass>(CurrentContainer);
			if (CurrentContainerAsClass == nullptr)
			{
				return false;
			}
			if (!GetDefault<UMVVMDeveloperProjectSettings>()->IsFunctionAllowed(Context, CurrentContainerAsClass, Field.GetFunction()))
			{
				return false;
			}
		}

		TValueOrError<const UStruct*, void> FieldAsContainerResult = UE::MVVM::FieldPathHelper::GetFieldAsContainer(Field);
		CurrentContainer = FieldAsContainerResult.HasValue() ? FieldAsContainerResult.GetValue() : nullptr;
	}
	return true;
}

bool FMVVMViewBlueprintCompiler::CanBeSetInNative(TArrayView<const FMVVMConstFieldVariant> PropertyPath)
{
	for (int32 Index = PropertyPath.Num() - 1; Index >= 0; --Index)
	{
		const FMVVMConstFieldVariant& Variant = PropertyPath[Index];
		// Stop the algo if the path is already a function.
		if (Variant.IsFunction())
		{
			return true;
		}

		// If the BP is defined in BP and has Net flags or FieldNotify flag, then the VaraibleSet K2Node need to be used to generate the proper byte-code.
		if (Variant.IsProperty())
		{
			// If it's an object then the path before the object doesn't matter.
			if (const FObjectPropertyBase* PropertyBase = CastField<const FObjectPropertyBase>(Variant.GetProperty()))
			{
				bool bLastPath = Index >= PropertyPath.Num() - 1;
				if (!bLastPath)
				{
					return true;
				}
			}

			if (Cast<UBlueprintGeneratedClass>(Variant.GetProperty()->GetOwnerStruct()))
			{
				if (Variant.GetProperty()->HasMetaData(FName("FieldNotify")) || Variant.GetProperty()->HasAnyPropertyFlags(CPF_Net))
				{
					return false;
				}
			}
		}
	}
	return true;
}

TSharedRef<FMVVMViewBlueprintCompiler::FGeneratedWriteFieldPathContext> FMVVMViewBlueprintCompiler::MakeWriteFieldPath(EMVVMBlueprintFieldPathSource GeneratedFrom, TArray<UE::MVVM::FMVVMConstFieldVariant>&& GeneratedFields, TArray<UE::MVVM::FMVVMConstFieldVariant>&& SkeletalGeneratedFields)
{
	TSharedRef<FGeneratedWriteFieldPathContext> WriteFieldPath = MakeShared<FGeneratedWriteFieldPathContext>();
	//WriteFieldPath->OptionalSource;
	WriteFieldPath->GeneratedFields = MoveTemp(GeneratedFields);
	WriteFieldPath->SkeletalGeneratedFields = MoveTemp(SkeletalGeneratedFields);
	WriteFieldPath->GeneratedFrom = GeneratedFrom;
	WriteFieldPath->bCanBeSetInNative = CanBeSetInNative(WriteFieldPath->SkeletalGeneratedFields);
	return WriteFieldPath;
}

void FMVVMViewBlueprintCompiler::TestGenerateSetter(const UBlueprint* Context, FStringView ObjectName, FStringView FieldPath, FStringView FunctionName)
{
#if UE_BUILD_DEVELOPMENT || UE_BUILD_DEBUG
	UWidgetBlueprint* WidgetBlueprint = nullptr;
	{
		UObject* FoundObject = FindObject<UObject>(nullptr, ObjectName.GetData(), EFindObjectFlags::None);
		WidgetBlueprint = Cast<UWidgetBlueprint>(FoundObject);
	}

	if (WidgetBlueprint == nullptr)
	{
		return;
	}

	UWidgetBlueprintGeneratedClass* NewSkeletonClass = Cast<UWidgetBlueprintGeneratedClass>(WidgetBlueprint->SkeletonGeneratedClass);
	TValueOrError<TArray<FMVVMConstFieldVariant>, FText> SkeletalSetterPathResult = FieldPathHelper::GenerateFieldPathList(NewSkeletonClass, FieldPath, false);
	if (SkeletalSetterPathResult.HasError())
	{
		return;
	}
	if (!IsPropertyPathValid(Context, SkeletalSetterPathResult.GetValue()))
	{
		return;
	}

	UEdGraph* GeneratedSetterGraph = UE::MVVM::FunctionGraphHelper::CreateFunctionGraph(WidgetBlueprint, FunctionName, EFunctionFlags::FUNC_None, TEXT(""), false);
	if (GeneratedSetterGraph == nullptr)
	{
		return;
	}

	const FProperty* SetterProperty = nullptr;
	if (SkeletalSetterPathResult.GetValue().Num() > 0 && ensure(SkeletalSetterPathResult.GetValue().Last().IsProperty()))
	{
		SetterProperty = SkeletalSetterPathResult.GetValue().Last().GetProperty();
	}
	if (SetterProperty == nullptr)
	{
		return;
	}

	UE::MVVM::FunctionGraphHelper::AddFunctionArgument(GeneratedSetterGraph, SetterProperty, "NewValue");
	UE::MVVM::FunctionGraphHelper::GenerateSetter(WidgetBlueprint, GeneratedSetterGraph, SkeletalSetterPathResult.GetValue());
#endif
}

} //namespace

#undef LOCTEXT_NAMESPACE