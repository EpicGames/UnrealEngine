// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MVVMCompiledBindingLibraryCompiler.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewCompilerInterface.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMBlueprintViewCondition.h"
#include "MVVMBlueprintViewModelContext.h"
#include "Templates/ValueOrError.h"
#include "Types/MVVMFieldVariant.h"
#include "UObject/StrongObjectPtr.h"
#include "WidgetBlueprintCompiler.h"

struct FMVVMBlueprintPinId;
struct FMVVMBlueprintViewBinding;
class FWidgetBlueprintCompilerContext;
class UEdGraph;
class UMVVMBlueprintView;
class UMVVMBlueprintViewConversionFunction;
class UMVVMBlueprintViewExtension;
class UMVVMBlueprintViewEvent;
class UMVVMViewClass;
class UMVVMViewClassExtension;
class UWidgetBlueprintGeneratedClass;

namespace  UE::MVVM
{
	enum class EBindingMessageType : uint8;
}

namespace UE::MVVM::Private
{

struct FMVVMViewBlueprintCompiler
{
private:
	struct FCompilerBindingSource;
	struct FCompilerUserWidgetProperty;
	struct FCompilerViewModelSetter;
	struct FCompilerViewModelCreatorContext;
	struct FCompilerSourceViewModelDynamicCreatorContext;
	struct FCompilerNotifyFieldId;
	struct FGeneratedReadFieldPathContext;
	struct FGeneratedWriteFieldPathContext;
	struct FCompilerBinding;
	struct FCompilerEvent;
	struct FCompilerCondition;
	struct FCompilerExtension;

public:
	FMVVMViewBlueprintCompiler(FWidgetBlueprintCompilerContext& InCreationContext, UMVVMBlueprintView* BlueprintView);

	FWidgetBlueprintCompilerContext& GetCompilerContext()
	{
		return WidgetBlueprintCompilerContext;
	}

	void AddExtension(UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension);
	void CleanOldData(UWidgetBlueprintGeneratedClass* ClassToClean, UObject* OldCDO);

	/** Update the list of compiler generated variables to be created by the kismet compiler */
	void GatherGeneratedVariables(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context);

	/** Generate variable and public function in the Skeleton class and the generated class */
	void CreateVariables(const FWidgetBlueprintCompilerContext::FCreateVariableContext& Context);
	/** Generate function that are hidden from the user (not on the Skeleton class). */
	void CreateFunctions(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context);

	/** Add all the field path and the bindings to the library compiler. */
	bool PreCompile(UWidgetBlueprintGeneratedClass* Class);
	/** Compile the library and fill the view and viewclass */
	bool Compile(UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension);

	/** Get the list of generated function during the compilation process. */
	TArray<FName> GetGeneratedFunctions() const;

	static void TestGenerateSetter(const UBlueprint* Context, FStringView ObjectName, FStringView FieldPath, FStringView FunctionName);

private:
	bool AreStepsValid() const
	{
		return bIsGatherGeneratedVariablesStepValid && bIsCreateVariableStepValid && bIsCreateFunctionsStepValid && bIsPreCompileStepValid && bIsCompileStepValid;
	}

	// GatherGeneratedVariables
	void CreateWidgetMap(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context);
	void CreateBindingList(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context);
	void CreateEventList(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context);
	void CreateConditionList(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context);
	void CreateExtensionList(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context);
	void CreateRequiredProperties(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context);

	// CreateVariables
	void CreatePublicFunctionsDeclaration(const FWidgetBlueprintCompilerContext::FCreateVariableContext& Context);

	// CreateFuntions
	void CategorizeBindings(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context);
	void CategorizeEvents(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context);
	void CategorizeConditions(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context);
	void CreateWriteFieldContexts(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context);
	void CreateViewModelSetters(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context);
	void CreateIntermediateGraphFunctions(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context);
	void CategorizeAsyncFunctions(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context);

	// PreCompile
	void FixCompilerBindingSelfSource(UWidgetBlueprintGeneratedClass* Class);
	void AddWarningForPropertyWithMVVMAndLegacyBinding(UWidgetBlueprintGeneratedClass* Class);
	void FixFieldPathContext(UWidgetBlueprintGeneratedClass* Class);
	void CreateReadFieldContexts(UWidgetBlueprintGeneratedClass* Class);
	void CreateCreatorContentFromBindingSource(UWidgetBlueprintGeneratedClass* Class);
	void PreCompileViewModelCreatorContexts(UWidgetBlueprintGeneratedClass* Class);
	void PreCompileBindings(UWidgetBlueprintGeneratedClass* Class);
	void PreCompileEvents(UWidgetBlueprintGeneratedClass* Class);
	void PreCompileConditions(UWidgetBlueprintGeneratedClass* Class);
	void PreCompileViewExtensions(UWidgetBlueprintGeneratedClass* Class);
	void PreCompileSourceDependencies(UWidgetBlueprintGeneratedClass* Class);

	// Compile
	void CompileSources(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension);
	void CompileBindings(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension);
	void CompileEvaluateSources(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension);
	void CompileEvents(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension);
	void CompileConditions(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension);
	void CompileViewExtensions(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension);
	void SortSourceFields(const FCompiledBindingLibraryCompiler::FCompileResult& CompileResult, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension);

private:
	/**
	 * List of all the sources needed by the view to register/execute the bindings.
	 * They could be viewmodel, widget or any properties on the UserWidget.
	 * They could also be a viewmodel in a long path.
	 * It may not have an associated property (dynamicviewmodel).
	 * It may only have OneTime binding.
	 */
	struct FCompilerBindingSource
	{
		enum class EType
		{
			ViewModel = 0,
			DynamicViewmodel = 1,
			Widget = 2,
			Self = 3,
		};
		const UClass* AuthoritativeClass = nullptr;
		TArray<TWeakPtr<FCompilerBindingSource>> Dependencies;
		FName Name;
		EType Type;
		bool bIsOptional = false;
	};
	TArray<TSharedRef<FCompilerBindingSource>> NeededBindingSources;

	/**
	 * Describe a Property that need to be added (if it doesn't already exist).
	 * They can be a Widget, a viewmodel, or any object owned by the UserWidget.
	 * They can be source or destination.
	 */
	struct FCompilerUserWidgetProperty : Compiler::FBlueprintViewUserWidgetProperty
	{
		enum class ECreationType
		{
			None,
			CreateIfDoesntExist,
			CreateOnlyIfDoesntExist,
		};
		FString BlueprintSetter;
		ECreationType CreationType = ECreationType::None;
		bool bInstanced = false;
		bool bInstanceExposed = false;
	};
	TArray<FCompilerUserWidgetProperty> NeededUserWidgetProperties;

	/**
	 * Describe a ViewModel generated setter function.
	 */
	struct FCompilerViewModelSetter
	{
		const UClass* Class = nullptr;
		FName PropertyName;
		FString BlueprintSetter;
		FText DisplayName;

		UEdGraph* SetterGraph = nullptr;
	};
	TArray<FCompilerViewModelSetter> ViewModelSettersToGenerate;

	/** 
	 * Describe the data to initialize the view's properties/viewmodels.
	 */
	struct FCompilerViewModelCreatorContext
	{
		FMVVMBlueprintViewModelContext ViewModelContext;
		TSharedPtr<FCompilerBindingSource> Source;

		TSharedPtr<FCompilerSourceViewModelDynamicCreatorContext> DynamicContext;
		FCompiledBindingLibraryCompiler::FFieldPathHandle ReadPropertyPathHandle;
	};
	TArray<FCompilerViewModelCreatorContext> ViewModelCreatorContexts;

	/**
	 * Describe the data to initialize the view's properties/widget.
	 */
	struct FCompilerWidgetCreatorContext
	{
		TSharedPtr<FCompilerBindingSource> Source;
		bool bSelfReference = false;

		FCompiledBindingLibraryCompiler::FFieldPathHandle ReadPropertyPathHandle;
	};
	TArray<FCompilerWidgetCreatorContext> WidgetCreatorContexts;

	/**
	 * Describe the data to initialize a viewmodel in a long path.
	 * The viewmodel is not added in the BlueprintView but it needs to be dynamic added.
	 */
	struct FCompilerSourceViewModelDynamicCreatorContext
	{
		TSharedPtr<FCompilerBindingSource> Source;
		TSharedPtr<FCompilerBindingSource> ParentSource; // a dynamic always has a parent

		FFieldNotificationId NotificationId;
		FCompiledBindingLibraryCompiler::FFieldIdHandle NotificationIdLibraryCompilerHandle;
	};
	TArray<TSharedRef<FCompilerSourceViewModelDynamicCreatorContext>> SourceViewModelDynamicCreatorContexts;

	/**
	 * The field id for a specific ReadFieldPathContext
	 */
	struct FCompilerNotifyFieldId
	{
		TArray<FGuid> BindingEditorKeys;
		TArray<FGuid> EventKeys;

		FFieldNotificationId NotificationId;
		TSharedPtr<FCompilerBindingSource> Source;
		TSharedPtr<FCompilerSourceViewModelDynamicCreatorContext> ViewModelDynamic;

		FCompiledBindingLibraryCompiler::FFieldIdHandle LibraryCompilerHandle;
	};
	TArray<TSharedPtr<FCompilerNotifyFieldId>> NotificationFields;

	/**
	 * The source path we need to read from.
	 * Can be any EMVVMBindingMode (OneTime, OneWay, ...)
	 */
	struct FGeneratedReadFieldPathContext
	{
		TArray<TWeakPtr<FCompilerBinding>> UsedByBindings;
		TArray<TWeakPtr<FCompilerEvent>> UsedByEvents;
		TArray<TWeakPtr<FCompilerCondition>> UsedByConditions;

		TSharedPtr<FCompilerBindingSource> Source;
		TArray<UE::MVVM::FMVVMConstFieldVariant> GeneratedFields; // the string path converted into field
		TArray<UE::MVVM::FMVVMConstFieldVariant> SkeletalGeneratedFields; // the field path converted with getter and setter
		EMVVMBlueprintFieldPathSource GeneratedFrom = EMVVMBlueprintFieldPathSource::None;
		bool bIsComponent = false;

		TSharedPtr<FCompilerNotifyFieldId> NotificationField;
		FCompiledBindingLibraryCompiler::FFieldPathHandle LibraryCompilerHandle;
	};
	TArray<TSharedRef<FGeneratedReadFieldPathContext>> GeneratedReadFieldPaths;

	/**
	 * Destination path we need to write to.
	 * Only if the bindings/events have a destination.
	 * The info needs to be validate before we generate the functions list.
	 */
	struct FGeneratedWriteFieldPathContext
	{
		TArray<TWeakPtr<FCompilerBinding>> UsedByBindings;
		TArray<TWeakPtr<FCompilerEvent>> UsedByEvents;

		/**
		 * Can be invalid if it's a widget with no Read Path.
		 * It is the start of the path.
		 */
		TSharedPtr<FCompilerBindingSource> OptionalSource;
		TSharedPtr<FCompilerBindingSource> OptionalDependencySource;

		/** The string path converted into field. It always start from the UserWidget. */
		TArray<UE::MVVM::FMVVMConstFieldVariant> GeneratedFields;
		/** The field path converted with getter and setter. It always start from the UserWidget. */
		TArray<UE::MVVM::FMVVMConstFieldVariant> SkeletalGeneratedFields;
		EMVVMBlueprintFieldPathSource GeneratedFrom = EMVVMBlueprintFieldPathSource::None;
		bool bCanBeSetInNative = true;
		bool bUseByNativeBinding = false;

		FName GeneratedFunctionName;
		FCompiledBindingLibraryCompiler::FFieldPathHandle LibraryCompilerHandle;
	};
	TArray<TSharedRef<FGeneratedWriteFieldPathContext>> GeneratedWriteFieldPaths;

	/**
	 * The list of all the valid bindings to iterate on.
	 */
	struct FCompilerBinding
	{
		struct FKey
		{
			int32 ViewBindingIndex = INDEX_NONE;
			bool bIsForwardBinding = false;
			FKey() = default;
			FKey(int32 InIndex, bool bInForward)
				: ViewBindingIndex(InIndex)
				, bIsForwardBinding(bInForward)
			{}
			bool operator==(const FKey& Other) const
			{
				return ViewBindingIndex == Other.ViewBindingIndex && bIsForwardBinding == Other.bIsForwardBinding;
			}
		};
		enum class EType : int32
		{
			Unknown = 0, // was not evaluated yet
			Invalid = -1, // evaluated and not valid
			Assignment = 1, //Destination=Source
			SimpleConversionFunction = 2, //Destination=Function(Source)
			ComplexConversionFunction = 3, //Destination=Function(SourceA, SourceB)
			//Function = 4, //Function(SourceA, SourceB)
		};
		FKey Key;
		EType Type = EType::Unknown;
		bool bIsOneTimeBinding = false;

		TArray<TSharedPtr<FGeneratedReadFieldPathContext>> ReadPaths;
		TSharedPtr<FGeneratedWriteFieldPathContext> WritePath;
		TWeakObjectPtr<UMVVMBlueprintViewConversionFunction> ConversionFunction;

		FCompiledBindingLibraryCompiler::FBindingHandle BindingHandle;
		FCompiledBindingLibraryCompiler::FFieldPathHandle ConversionFunctionHandle;
		UE::MVVM::Compiler::FCompilerBindingHandle CompilerBindingHandle;
	};
	TArray<TSharedRef<FCompilerBinding>> ValidBindings;

	/**
	 * The list of all the valid events to iterate on.
	 */
	struct FCompilerEvent
	{
		TWeakObjectPtr<UMVVMBlueprintViewEvent> Event;

		TArray<TSharedPtr<FGeneratedReadFieldPathContext>> ReadPaths;
		TSharedPtr<FGeneratedWriteFieldPathContext> WritePath;

		TSharedPtr<FGeneratedWriteFieldPathContext> DelegateFieldPath;

		FName GeneratedGraphName;
		FName SourceName; // may not be in the NeededBindingSources
		FCompiledBindingLibraryCompiler::FFieldPathHandle DelegateFieldPathHandle;
	};
	TArray<TSharedRef<FCompilerEvent>> ValidEvents;

	/**
	 * The list of all the valid conditions to iterate on.
	 */
	struct FCompilerCondition
	{
		TWeakObjectPtr<UMVVMBlueprintViewCondition> Condition;
		TArray<TSharedPtr<FGeneratedReadFieldPathContext>> ReadPaths;
		FName GeneratedGraphName;
		FName SourceName; // may not be in the NeededBindingSources
		FCompiledBindingLibraryCompiler::FFieldPathHandle DelegateFieldPathHandle;
	};
	TArray<TSharedRef<FCompilerCondition>> ValidConditions;

	/**
	 * The list of all the valid extension to iterates on.
	 */
	struct FCompilerExtension
	{
		TWeakObjectPtr<UMVVMBlueprintViewExtension> Extension;
	};
	TArray<TSharedRef<FCompilerExtension>> ValidExtensions;

	/**
	 * List of public expose function
	 */
	TArray<FName> FunctionPermissionsToAdd;

	/**
	 * List of generated function
	 */
	TArray<FName> GeneratedFunctions;

private:
	TMap<FName, UWidget*> WidgetNameToWidgetPointerMap;
	FWidgetBlueprintCompilerContext& WidgetBlueprintCompilerContext;
	TObjectPtr<UMVVMBlueprintView> BlueprintView = nullptr;
	FCompiledBindingLibraryCompiler BindingLibraryCompiler;
	bool bIsGatherGeneratedVariablesStepValid = true;
	bool bIsCreateVariableStepValid = true;
	bool bIsCreateFunctionsStepValid = true;
	bool bIsPreCompileStepValid = true;
	bool bIsCompileStepValid = true;

private:
	void AddMessage(const FText& MessageText, Compiler::EMessageType MessageType) const;
	void AddMessages(TArrayView<TWeakPtr<FCompilerBinding>> Bindings, TArrayView<TWeakPtr<FCompilerEvent>> Events, const FText& MessageText, Compiler::EMessageType MessageType) const;
	void AddMessageForBinding(const TSharedPtr<FCompilerBinding>& Binding, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const;
	void AddMessageForBinding(const FMVVMBlueprintViewBinding& Binding, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const;
	void AddMessageForEvent(const TSharedPtr<FCompilerEvent>& Event, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const;
	void AddMessageForEvent(const UMVVMBlueprintViewEvent* Event, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const;
	void AddMessageForCondition(const TSharedPtr<FCompilerCondition>& Condition, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const;
	void AddMessageForCondition(const UMVVMBlueprintViewCondition* Condition, const FText& MessageText, Compiler::EMessageType MessageType, const FMVVMBlueprintPinId& PinId) const;
	void AddMessageForViewModel(const FMVVMBlueprintViewModelContext& ViewModel, const FText& Message, Compiler::EMessageType MessageType) const;
	void AddMessageForViewModel(const FText& ViewModelDisplayName, const FText& Message, Compiler::EMessageType MessageType) const;

	struct FGetFieldsResult
	{
		TSharedPtr<FCompilerBindingSource> OptionalSource;
		EMVVMBlueprintFieldPathSource GeneratedFrom = EMVVMBlueprintFieldPathSource::None;
		TArray<FMVVMConstFieldVariant> GeneratedFields;
	};
	struct FCreateFieldsResult
	{
		TSharedPtr<FCompilerBindingSource> OptionalSource;
		EMVVMBlueprintFieldPathSource GeneratedFrom = EMVVMBlueprintFieldPathSource::None;
		TArray<UE::MVVM::FMVVMConstFieldVariant> GeneratedFields;
		TArray<UE::MVVM::FMVVMConstFieldVariant> SkeletalGeneratedFields;
		bool bIsComponent;
	};
	TValueOrError<FGetFieldsResult, FText> GetFields(const UWidgetBlueprintGeneratedClass* Class, const FMVVMBlueprintPropertyPath& PropertyPath) const;
	TValueOrError<FCreateFieldsResult, FText> CreateFieldContext(const UWidgetBlueprintGeneratedClass* Class, const FMVVMBlueprintPropertyPath& PropertyPath, bool bForSourceReading) const;
	TValueOrError<TSharedPtr<FCompilerNotifyFieldId>, FText> CreateNotifyFieldId(const UWidgetBlueprintGeneratedClass* Class, const TSharedPtr<FGeneratedReadFieldPathContext>& ReadFieldContext);

	UMVVMViewClassExtension* CreateViewClassExtension(TSubclassOf<UMVVMViewClassExtension> ExtensionClass, UMVVMViewClass* ViewClass);

	static TArray<FMVVMConstFieldVariant> AppendBaseField(const UClass* Class, FName PropertyName, TArray<FMVVMConstFieldVariant> Properties);
	static bool IsPropertyPathValid(const UBlueprint* Context, TArrayView<const FMVVMConstFieldVariant> PropertyPath);
	static bool CanBeSetInNative(TArrayView<const FMVVMConstFieldVariant> PropertyPath);
	static TSharedRef<FGeneratedWriteFieldPathContext> MakeWriteFieldPath(EMVVMBlueprintFieldPathSource GeneratedFrom, TArray<UE::MVVM::FMVVMConstFieldVariant>&& GeneratedFields, TArray<UE::MVVM::FMVVMConstFieldVariant>&& SkeletalGeneratedFields);
};

} //namespace
