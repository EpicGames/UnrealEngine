// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMWidgetBlueprintDiff.h"

#if WITH_EDITOR

#include "Blueprint/WidgetTree.h"
#include "Bindings/MVVMConversionFunctionHelper.h"
#include "Extensions/MVVMBlueprintViewExtension.h"
#include "MVVMBlueprintInstancedViewModel.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewModel.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMBlueprintViewBinding.h"
#include "MVVMBlueprintViewCondition.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMPropertyPath.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "WidgetBlueprint.h"

#include "DiffResults.h"

#define LOCTEXT_NAMESPACE "MVVMWidgetBlueprintDiff"

namespace UE::MVVM::Private
{
	TSharedPtr<UE::MVVM::FMVVMDiffCustomObjectProvider> DiffCustomObjectProvider;

	struct FMVVMBlueprintViewDiffDetails
	{
		const UWidgetBlueprint* OldBlueprint;
		const UWidgetBlueprint* NewBlueprint;
		UMVVMBlueprintView* OldBlueprintView;
		UMVVMBlueprintView* NewBlueprintView;
		FDiffResults* ResultsPtr;
	};

	enum class EMVVMBindingPropertyPathType
	{
		Source,
		Destination,
		Event,
		Condition
	};

	FText GetPathSourceText(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& Path)
	{
		switch (Path.GetSource(WidgetBlueprint))
		{
		case EMVVMBlueprintFieldPathSource::SelfContext:
		{
			return FText::FromString(WidgetBlueprint->GetFriendlyName());
		}
		case EMVVMBlueprintFieldPathSource::ViewModel:
		{
			UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
			UMVVMBlueprintView* BlueprintView = ExtensionView ? ExtensionView->GetBlueprintView() : nullptr;
			const FMVVMBlueprintViewModelContext* ViewModel = BlueprintView ? BlueprintView->FindViewModel(Path.GetViewModelId()) : nullptr;
			return ViewModel ? ViewModel->GetDisplayName() : LOCTEXT("None", "<None>");
		}
		case EMVVMBlueprintFieldPathSource::Widget:
		{
			UWidgetTree* WidgetTree = WidgetBlueprint ? WidgetBlueprint->WidgetTree.Get() : nullptr;
			UWidget* FoundWidget = WidgetTree ? WidgetTree->FindWidget(Path.GetWidgetName()) : nullptr;
			return FoundWidget ? FoundWidget->GetLabelText() : FText::FromName(Path.GetWidgetName());
		}
		}
		return FText::GetEmpty();
	}

	template <class TViewProperty>
	FText GetContextText(const FMVVMBlueprintViewDiffDetails& Details, TViewProperty NewProperty, TViewProperty OldProperty)
	{
		FText NewContext;
		FText OldContext;

		if constexpr (std::is_same_v<TViewProperty, FMVVMBlueprintViewBinding*>)
		{
			NewContext = NewProperty != nullptr ? GetPathSourceText(Details.NewBlueprint, NewProperty->DestinationPath) : FText::GetEmpty();
			OldContext = OldProperty != nullptr ? GetPathSourceText(Details.OldBlueprint, OldProperty->DestinationPath) : FText::GetEmpty();
		}
		else if constexpr (std::is_same_v<TViewProperty, const UMVVMBlueprintViewCondition*>)
		{
			NewContext = NewProperty != nullptr ? GetPathSourceText(Details.NewBlueprint, NewProperty->GetConditionPath()) : FText::GetEmpty();
			OldContext = OldProperty != nullptr ? GetPathSourceText(Details.OldBlueprint, OldProperty->GetConditionPath()) : FText::GetEmpty();
		}
		else if constexpr (std::is_same_v<TViewProperty, const UMVVMBlueprintViewEvent*>)
		{
			NewContext = NewProperty != nullptr ? GetPathSourceText(Details.NewBlueprint, NewProperty->GetEventPath()) : FText::GetEmpty();
			OldContext = OldProperty != nullptr ? GetPathSourceText(Details.OldBlueprint, OldProperty->GetEventPath()) : FText::GetEmpty();
		}

		// Get the context of the diff. If both properties are valid, the context may differ i.e. destination binding changes between widgets. 
		// In this case the diff details will denote the change, and context is not necessarily relevant. 
		if (NewContext.IsEmpty())
		{
			return OldContext;
		}
		else if (OldContext.IsEmpty() || NewContext.EqualTo(OldContext))
		{
			return NewContext;

		}
		return FText::GetEmpty();
	}

	template <class TViewProperty>
	void ApplyDiff(const FMVVMBlueprintViewDiffDetails& Details, TViewProperty NewProperty, TViewProperty OldProperty, EDiffType::Category Category, const FText& DisplayString)
	{
		FDiffSingleResult Diff;
		Diff.Category = Category;
		Diff.Object1 = Details.NewBlueprintView;
		Diff.Object2 = Details.OldBlueprintView;
		Diff.DisplayString = DisplayString;

		if (!DiffCustomObjectProvider.IsValid())
		{
			Diff.Diff = EDiffType::OBJECT_PROPERTY;
		}
		else
		{
			Diff.Diff = EDiffType::CUSTOM_OBJECT;

			if constexpr (std::is_same_v<TViewProperty, FMVVMBlueprintViewBinding*>)
			{
				const FGuid NewId = NewProperty ? NewProperty->BindingId : FGuid();
				const FGuid OldId = OldProperty ? OldProperty->BindingId : FGuid();
				Diff.CustomProperty = DiffCustomObjectProvider->CreatePropertyBindingDiff(NewId, OldId);
			}
			else if constexpr (std::is_same_v<TViewProperty, FMVVMBlueprintPin>)
			{
				Diff.CustomProperty = DiffCustomObjectProvider->CreatePropertyParameterDiff(NewProperty.GetId(), OldProperty.GetId());
			}
			else if constexpr (std::is_same_v<TViewProperty, const UMVVMBlueprintViewCondition*>)
			{
				Diff.CustomProperty = DiffCustomObjectProvider->CreatePropertyConditionDiff(NewProperty, OldProperty);
			}
			else if constexpr (std::is_same_v<TViewProperty, const UMVVMBlueprintViewEvent*>)
			{
				Diff.CustomProperty = DiffCustomObjectProvider->CreatePropertyEventDiff(NewProperty, OldProperty);
			}
		}
		
		Details.ResultsPtr->Add(Diff);
	}

	template <class TViewProperty>
	bool DiffPropertyPath(FMVVMBlueprintViewDiffDetails& Details, TViewProperty NewProperty, TViewProperty OldProperty, const FMVVMBlueprintPropertyPath& OldPath, const FMVVMBlueprintPropertyPath& NewPath, EMVVMBindingPropertyPathType Type)
	{
		if (OldPath == NewPath)
		{
			return false;
		}
		FText TypeText;
		switch (Type)
		{		
		case EMVVMBindingPropertyPathType::Source:	 
			TypeText = FText::FromString(TEXT("Source"));  
			break;
		case EMVVMBindingPropertyPathType::Destination:
			TypeText = FText::FromString(TEXT("Destination"));  
			break;
		case EMVVMBindingPropertyPathType::Event:
			TypeText = FText::FromString(TEXT("Event"));  
			break;
		case EMVVMBindingPropertyPathType::Condition:
			TypeText = FText::FromString(TEXT("Condition"));  
			break;
		}

		FFormatNamedArguments Args;
		Args.Add(TEXT("Context"), GetContextText(Details, NewProperty, OldProperty));
		Args.Add(TEXT("Type"), TypeText);
		const FText DisplayString = FText::Format(LOCTEXT("MVVM_ChangedPathDisplay", "[{Context}] {Type} changed"), Args);

		ApplyDiff(Details, NewProperty, OldProperty, EDiffType::MODIFICATION, DisplayString);

		return true;
	}

	using FPinPair = TTuple<FMVVMBlueprintPin, UEdGraphPin*>;
	void DiffPins(FMVVMBlueprintViewDiffDetails& Details, const TMap<FString, FPinPair>& OldPins, const TMap<FString, FPinPair>& NewPins, const FText& ContextText)
	{
		// Check if pin values changed. Do not need to check for added/removed pins. Pins are associated with blueprint conversion functions. 
		// If the function is changed, this will be noted in the property path diff.
		for (TPair<FString, FPinPair> Pair : OldPins)
		{
			const FPinPair* NewPinPairPtr = NewPins.Find(Pair.Key);

			// Pins should not differ as source and destination paths are not changed.
			if (NewPinPairPtr == nullptr)
			{
				continue;
			}

			const FPinPair& OldPinPair = Pair.Value;
			const FPinPair& NewPinPair = *NewPinPairPtr;

			const FMVVMBlueprintPin& OldPin = OldPinPair.Key;
			const FMVVMBlueprintPin& NewPin = NewPinPair.Key;
			
			UEdGraphPin* OldGraphPin = OldPinPair.Value;
			UEdGraphPin* NewGraphPin = NewPinPair.Value;

			FString OldPinValue;
			if (OldPin.UsedPathAsValue())
			{
				OldPinValue = OldPin.GetPath().ToString(Details.OldBlueprint, true, false);
			}
			else
			{
				OldPinValue = OldPin.GetValueAsString(Details.OldBlueprint ? Details.OldBlueprint->SkeletonGeneratedClass : nullptr);
			}

			FString NewPinValue;
			if (NewPin.UsedPathAsValue())
			{
				NewPinValue = NewPin.GetPath().ToString(Details.NewBlueprint, true, false);
			}
			else
			{
				NewPinValue = NewPin.GetValueAsString(Details.NewBlueprint ? Details.NewBlueprint->SkeletonGeneratedClass : nullptr);
			}

			if(OldPinValue != NewPinValue)
			{				
				FText PinDisplayName;
				if (OldGraphPin != nullptr)
				{
					PinDisplayName = OldGraphPin->GetDisplayName();
				}
				else
				{
					PinDisplayName = FText::FromString(OldPin.GetId().ToString());
				}

				FFormatNamedArguments Args;
				Args.Add(TEXT("Context"), ContextText);
				Args.Add(TEXT("PinName"), PinDisplayName);
				const FText DisplayString = FText::Format(LOCTEXT("MVVM_ChangedPinDisplay", "[{Context}] Pin \"{PinName}\" changed"), Args);

				ApplyDiff(Details, NewPin, OldPin, EDiffType::MODIFICATION, DisplayString);
			}
		}
	}
	
	void DiffBindings(FMVVMBlueprintViewDiffDetails& Details)
	{
		TSet<FGuid> OldBindings;
		TSet<FGuid> NewBindings;
		TMap<FGuid, FMVVMBlueprintViewBinding> OldBindingMap;
		TMap<FGuid, FMVVMBlueprintViewBinding> NewBindingMap;

		if (Details.OldBlueprintView != nullptr)
		{
			for (const FMVVMBlueprintViewBinding& OldBinding : Details.OldBlueprintView->GetBindings())
			{
				OldBindings.Add(OldBinding.BindingId);
				OldBindingMap.Add(OldBinding.BindingId, OldBinding);
			}
		}

		if (Details.NewBlueprintView != nullptr)
		{
			for (const FMVVMBlueprintViewBinding& NewBinding : Details.NewBlueprintView->GetBindings())
			{
				NewBindings.Add(NewBinding.BindingId);
				NewBindingMap.Add(NewBinding.BindingId, NewBinding);
			}
		}

		const TSet<FGuid> SharedBindings = OldBindings.Intersect(NewBindings);
		const TSet<FGuid> AddedBindings = NewBindings.Difference(OldBindings);
		const TSet<FGuid> RemovedBindings = OldBindings.Difference(NewBindings);

		for (const FGuid& Id: AddedBindings)
		{
			FMVVMBlueprintViewBinding* NewBinding = NewBindingMap.Find(Id);
			FMVVMBlueprintViewBinding* OldBinding = nullptr;
			if (!ensure(NewBinding != nullptr))
			{
				continue;
			}
			
			FFormatNamedArguments Args;
			Args.Add(TEXT("Context"), GetContextText(Details, NewBinding, OldBinding));
			Args.Add(TEXT("BindingName"), Details.NewBlueprint ? FText::FromString(NewBinding->GetDisplayNameString(Details.NewBlueprint, true)) : FText::GetEmpty());
			const FText DisplayString = FText::Format(LOCTEXT("MVVM_AddedBindingDisplay", "[{Context}] Added Binding \"{BindingName}\""), Args);
			
			ApplyDiff(Details, NewBinding, OldBinding, EDiffType::ADDITION, DisplayString);
		}

		for (const FGuid& Id: RemovedBindings)
		{
			FMVVMBlueprintViewBinding* OldBinding = OldBindingMap.Find(Id);
			FMVVMBlueprintViewBinding* NewBinding = nullptr;
			if (!ensure(OldBinding != nullptr))
			{
				continue;
			}

			FFormatNamedArguments Args;
			Args.Add(TEXT("Context"), GetContextText(Details, NewBinding, OldBinding));
			Args.Add(TEXT("BindingName"), Details.OldBlueprint ? FText::FromString(OldBinding->GetDisplayNameString(Details.OldBlueprint, true)) : FText::GetEmpty());
			const FText DisplayString = FText::Format(LOCTEXT("MVVM_RemovedBindingDisplay", "[{Context}] Removed Binding \"{BindingName}\""), Args);

			ApplyDiff(Details, NewBinding, OldBinding, EDiffType::SUBTRACTION, DisplayString);
		}

		for (const FGuid& Id: SharedBindings)
		{
			FMVVMBlueprintViewBinding* OldBinding = OldBindingMap.Find(Id);
			FMVVMBlueprintViewBinding* NewBinding = NewBindingMap.Find(Id);
			if (!ensure(OldBinding != nullptr && NewBinding != nullptr))
			{
				continue;
			}

			// 1. Diff property paths
			const bool bSourcePathChanged = DiffPropertyPath(Details, NewBinding, OldBinding, OldBinding->SourcePath, NewBinding->SourcePath, EMVVMBindingPropertyPathType::Source);
			const bool bDestinationPathChanged = DiffPropertyPath(Details, NewBinding, OldBinding, OldBinding->DestinationPath, NewBinding->DestinationPath, EMVVMBindingPropertyPathType::Destination);

			// 2. Diff the binding modes
			if (OldBinding->BindingType != NewBinding->BindingType)
			{
				UEnum* BindingModeEnum = StaticEnum<EMVVMBindingMode>();

				FFormatNamedArguments Args;
				Args.Add(TEXT("Context"), GetContextText(Details, NewBinding, OldBinding));
				Args.Add(TEXT("NewMode"), BindingModeEnum->GetDisplayValueAsText(NewBinding->BindingType));

				const FText DisplayString = FText::Format(LOCTEXT("MVVM_ChangedModeDisplay", "[{Context}] Binding Mode changed to \"{NewMode}\""), Args);

				ApplyDiff(Details, NewBinding, OldBinding, EDiffType::MODIFICATION, DisplayString);
			}

			// 3. Diff the source and destination conversion function Pins
			TMap<FString, FPinPair> OldPins;
			TMap<FString, FPinPair> NewPins;

			// If the source path did not changed. Check if any pins have changed
			if (!bSourcePathChanged)
			{
				if (UMVVMBlueprintViewConversionFunction* OldSourceFunction = OldBinding->Conversion.GetConversionFunction(true))
				{
					for (const FMVVMBlueprintPin& Pin : OldSourceFunction->GetPins())
					{
						UEdGraphPin* GraphPin = UE::MVVM::ConversionFunctionHelper::FindPin(OldSourceFunction->GetWrapperGraph(), Pin.GetId().GetNames());
						OldPins.Add(Pin.GetId().ToString(), MakeTuple(Pin, GraphPin));
					}
				}

				if (UMVVMBlueprintViewConversionFunction* NewSourceFunction = NewBinding->Conversion.GetConversionFunction(true))
				{
					for (const FMVVMBlueprintPin& Pin : NewSourceFunction->GetPins())
					{
						UEdGraphPin* GraphPin = UE::MVVM::ConversionFunctionHelper::FindPin(NewSourceFunction->GetWrapperGraph(), Pin.GetId().GetNames());
						NewPins.Add(Pin.GetId().ToString(), MakeTuple(Pin, GraphPin));
					}
				}
			}

			// If the destination path did not changed. Check if any pins have changed
			if (!bDestinationPathChanged)
			{
				if (UMVVMBlueprintViewConversionFunction* OldDestinationFunction = OldBinding->Conversion.GetConversionFunction(false))
				{
					for (const FMVVMBlueprintPin& Pin : OldDestinationFunction->GetPins())
					{
						UEdGraphPin* GraphPin = UE::MVVM::ConversionFunctionHelper::FindPin(OldDestinationFunction->GetWrapperGraph(), Pin.GetId().GetNames());
						OldPins.Add(Pin.GetId().ToString(), MakeTuple(Pin, GraphPin));
					}
				}

				if (UMVVMBlueprintViewConversionFunction* NewDestinationFunction = NewBinding->Conversion.GetConversionFunction(false))
				{
					for (const FMVVMBlueprintPin& Pin : NewDestinationFunction->GetPins())
					{
						UEdGraphPin* GraphPin = UE::MVVM::ConversionFunctionHelper::FindPin(NewDestinationFunction->GetWrapperGraph(), Pin.GetId().GetNames());
						NewPins.Add(Pin.GetId().ToString(), MakeTuple(Pin, GraphPin));
					}
				}
			}

			const FText ContextText = GetContextText(Details, NewBinding, OldBinding);
			DiffPins(Details, OldPins, NewPins, ContextText);
		}
	}

	void DiffEvents(FMVVMBlueprintViewDiffDetails& Details)
	{
		TSet<FName> OldEvents;
		TSet<FName> NewEvents;
		TMap<FName, const UMVVMBlueprintViewEvent*> OldEventMap;
		TMap<FName, const UMVVMBlueprintViewEvent*> NewEventMap;

		if (Details.OldBlueprintView != nullptr )
		{
			for (const UMVVMBlueprintViewEvent* OldEvent : Details.OldBlueprintView->GetEvents())
			{
				OldEvents.Add(OldEvent->GetFName());
				OldEventMap.Add(OldEvent->GetFName(), OldEvent);
			}
		}

		if (Details.NewBlueprintView != nullptr )
		{
			for (const UMVVMBlueprintViewEvent* NewEvent : Details.NewBlueprintView->GetEvents())
			{
				NewEvents.Add(NewEvent->GetFName());
				NewEventMap.Add(NewEvent->GetFName(), NewEvent);
			}
		}

		const TSet<FName> SharedEvents = OldEvents.Intersect(NewEvents);
		const TSet<FName> AddedEvents = NewEvents.Difference(OldEvents);
		const TSet<FName> RemovedEvents = OldEvents.Difference(NewEvents);

		for (const FName& Id: AddedEvents)
		{
			const UMVVMBlueprintViewEvent** EventPtr = NewEventMap.Find(Id);
			if (!ensure(EventPtr != nullptr && *EventPtr != nullptr))
			{
				continue;
			}

			const UMVVMBlueprintViewEvent* NewEvent = *EventPtr;
			const UMVVMBlueprintViewEvent* OldEvent = nullptr;
						
			FFormatNamedArguments Args;
			Args.Add(TEXT("Context"), GetContextText(Details, NewEvent, OldEvent));
			Args.Add(TEXT("EventName"), NewEvent->GetDisplayName(true));			
			const FText DisplayString = FText::Format(LOCTEXT("MVVM_AddedEventDisplay", "[{Context}] Added Event {EventName}"), Args);

			ApplyDiff(Details, NewEvent, OldEvent, EDiffType::ADDITION, DisplayString);
		}

		for (const FName& Id: RemovedEvents)
		{
			const UMVVMBlueprintViewEvent** EventPtr = OldEventMap.Find(Id);
			if (!ensure(EventPtr != nullptr && *EventPtr != nullptr))
			{
				continue;
			}

			const UMVVMBlueprintViewEvent* NewEvent = nullptr;
			const UMVVMBlueprintViewEvent* OldEvent = *EventPtr;

			FFormatNamedArguments Args;
			Args.Add(TEXT("Context"), GetContextText(Details, NewEvent, OldEvent));
			Args.Add(TEXT("EventName"), OldEvent->GetDisplayName(true));			
			const FText DisplayString = FText::Format(LOCTEXT("MVVM_RemovedEventDisplay", "[{Context}] Removed Event {EventName}"), Args);

			ApplyDiff(Details, NewEvent, OldEvent, EDiffType::SUBTRACTION, DisplayString);
		}

		for (const FName& Id: SharedEvents)
		{
			const UMVVMBlueprintViewEvent** OldEventPtr = OldEventMap.Find(Id);
			const UMVVMBlueprintViewEvent** NewEventPtr = NewEventMap.Find(Id);
			if (!ensure(OldEventPtr != nullptr && NewEventPtr != nullptr))
			{
				continue;
			}

			const UMVVMBlueprintViewEvent* OldEvent = *OldEventPtr;
			const UMVVMBlueprintViewEvent* NewEvent = *NewEventPtr;
			if (!ensure(OldEvent != nullptr && NewEvent != nullptr))
			{
				continue;
			}

			// Diff property paths
			const bool bEventPathChanged = DiffPropertyPath(Details, NewEvent, OldEvent, OldEvent->GetEventPath(), NewEvent->GetEventPath(), EMVVMBindingPropertyPathType::Event);
			const bool bDestinationPathChanged = DiffPropertyPath(Details, NewEvent, OldEvent, OldEvent->GetDestinationPath(), NewEvent->GetDestinationPath(), EMVVMBindingPropertyPathType::Destination);

			TMap<FString, FPinPair> OldPins;
			TMap<FString, FPinPair> NewPins;

			// If the destination path did not changed. Check if any pins have changed
			if (!bDestinationPathChanged)
			{
				for (const FMVVMBlueprintPin& Pin : OldEvent->GetPins())
				{
					UEdGraphPin* GraphPin = UE::MVVM::ConversionFunctionHelper::FindPin(OldEvent->GetWrapperGraph(), Pin.GetId().GetNames());
					OldPins.Add(Pin.GetId().ToString(), MakeTuple(Pin, GraphPin));
				}

				for (const FMVVMBlueprintPin& Pin : NewEvent->GetPins())
				{
					UEdGraphPin* GraphPin = UE::MVVM::ConversionFunctionHelper::FindPin(OldEvent->GetWrapperGraph(), Pin.GetId().GetNames());
					NewPins.Add(Pin.GetId().ToString(), MakeTuple(Pin, GraphPin));
				}
			}

			const FText ContextText = GetContextText(Details, NewEvent, OldEvent);
			DiffPins(Details, OldPins, NewPins, ContextText);
		}
	}

	void DiffConditions(FMVVMBlueprintViewDiffDetails& Details)
	{
		TSet<FName> OldConditions;
		TSet<FName> NewConditions;
		TMap<FName, const UMVVMBlueprintViewCondition*> OldConditionMap;
		TMap<FName, const UMVVMBlueprintViewCondition*> NewConditionMap;

		
		if (Details.OldBlueprintView != nullptr)
		{
			for (const UMVVMBlueprintViewCondition* OldCondition : Details.OldBlueprintView->GetConditions())
			{
				OldConditions.Add(OldCondition->GetFName());
				OldConditionMap.Add(OldCondition->GetFName(), OldCondition);
			}
		}
		
		if (Details.NewBlueprintView != nullptr)
		{
			for (const UMVVMBlueprintViewCondition* NewCondition : Details.NewBlueprintView->GetConditions())
			{
				NewConditions.Add(NewCondition->GetFName());
				NewConditionMap.Add(NewCondition->GetFName(), NewCondition);
			}
		}

		const TSet<FName> SharedConditions = OldConditions.Intersect(NewConditions);
		const TSet<FName> AddedConditions = NewConditions.Difference(OldConditions);
		const TSet<FName> RemovedConditions = OldConditions.Difference(NewConditions);

		for (const FName& Id: AddedConditions)
		{
			const UMVVMBlueprintViewCondition** ConditionPtr = NewConditionMap.Find(Id);
			if (!ensure(ConditionPtr != nullptr && *ConditionPtr != nullptr))
			{
				continue;
			}

			const UMVVMBlueprintViewCondition* NewCondition = *ConditionPtr;
			const UMVVMBlueprintViewCondition* OldCondition = nullptr;
	
			FFormatNamedArguments Args;
			Args.Add(TEXT("Context"), GetContextText(Details, NewCondition, OldCondition));
			Args.Add(TEXT("ConditionName"), NewCondition->GetDisplayName(true));
			const FText DisplayString = FText::Format(LOCTEXT("MVVM_AddedConditionDisplay", "[{Context}] Added Condition {ConditionName}"), Args);

			ApplyDiff(Details, NewCondition, OldCondition, EDiffType::ADDITION, DisplayString);
		}

		for (const FName& Id: RemovedConditions)
		{
			const UMVVMBlueprintViewCondition** ConditionPtr = OldConditionMap.Find(Id);
			if (!ensure(ConditionPtr != nullptr && *ConditionPtr != nullptr))
			{
				continue;
			}
		
			const UMVVMBlueprintViewCondition* OldCondition = *ConditionPtr;
			const UMVVMBlueprintViewCondition* NewCondition = nullptr;

			FFormatNamedArguments Args;
			Args.Add(TEXT("Context"), GetContextText(Details, NewCondition, OldCondition));
			Args.Add(TEXT("ConditionName"), OldCondition->GetDisplayName(true));
			const FText DisplayString = FText::Format(LOCTEXT("MVVM_RemovedConditionDisplay", "[{Context}] Removed Condition {ConditionName}"), Args);

			ApplyDiff(Details, NewCondition, OldCondition, EDiffType::SUBTRACTION, DisplayString);
		}

		for (const FName& Id: SharedConditions)
		{
			const UMVVMBlueprintViewCondition** OldConditionPtr = OldConditionMap.Find(Id);
			const UMVVMBlueprintViewCondition** NewConditionPtr = NewConditionMap.Find(Id);
			if (!ensure(OldConditionPtr != nullptr && NewConditionPtr != nullptr))
			{
				continue;
			}

			const UMVVMBlueprintViewCondition* OldCondition = * OldConditionPtr;
			const UMVVMBlueprintViewCondition* NewCondition = *NewConditionPtr;
			if (!ensure(OldCondition != nullptr && NewCondition != nullptr))
			{
				continue;
			}

			// Diff property paths
			const bool bConditionPathChanged = DiffPropertyPath(Details, NewCondition, OldCondition, OldCondition->GetConditionPath(), NewCondition->GetConditionPath(), EMVVMBindingPropertyPathType::Condition);
			const bool bDestinationPathChanged = DiffPropertyPath(Details, NewCondition, OldCondition, OldCondition->GetDestinationPath(), NewCondition->GetDestinationPath(), EMVVMBindingPropertyPathType::Destination);

			if (OldCondition->GetOperation() != NewCondition->GetOperation())
			{
				UEnum* OperationEnum = StaticEnum<EMVVMConditionOperation>();

				FFormatNamedArguments Args;
				Args.Add(TEXT("Context"), GetContextText(Details, NewCondition, OldCondition));
				Args.Add(TEXT("NewOperation"), OperationEnum->GetDisplayValueAsText(NewCondition->GetOperation()));
				const FText DisplayString = FText::Format(LOCTEXT("MVVM_ChangedOperationDisplay", "[{Context}] Condition operation changed to \"{NewOperation}\""), Args);

				ApplyDiff(Details, NewCondition, OldCondition, EDiffType::MODIFICATION, DisplayString);
			}

			if (OldCondition->GetOperationValue() != NewCondition->GetOperationValue())
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Context"), GetContextText(Details, NewCondition, OldCondition));
				Args.Add(TEXT("NewValue"), FText::AsNumber(NewCondition->GetOperationValue()));
				const FText DisplayString = FText::Format(LOCTEXT("MVVM_ChangedOperationValueDisplay", "[{Context}] Condition value changed to \"{NewValue}\""), Args);

				ApplyDiff(Details, NewCondition, OldCondition, EDiffType::MODIFICATION, DisplayString);
			}

			if (OldCondition->GetOperationMaxValue() != NewCondition->GetOperationMaxValue())
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Context"), GetContextText(Details, NewCondition, OldCondition));
				Args.Add(TEXT("NewValue"), FText::AsNumber(NewCondition->GetOperationMaxValue()));
				const FText DisplayString = FText::Format(LOCTEXT("MVVM_ChangedOperationMaxValueDisplay", "[{Context}] Condition max value changed to \"{NewValue}\""), Args);

				ApplyDiff(Details, NewCondition, OldCondition, EDiffType::MODIFICATION, DisplayString);
			}

			TMap<FString, FPinPair> OldPins;
			TMap<FString, FPinPair> NewPins;

			// If the destination path did not changed. Check if any pins have changed
			if (!bDestinationPathChanged)
			{
				for (const FMVVMBlueprintPin& Pin : OldCondition->GetPins())
				{
					UEdGraphPin* GraphPin = UE::MVVM::ConversionFunctionHelper::FindPin(OldCondition->GetWrapperGraph(), Pin.GetId().GetNames());
					OldPins.Add(Pin.GetId().ToString(), MakeTuple(Pin, GraphPin));
				}

				for (const FMVVMBlueprintPin& Pin : NewCondition->GetPins())
				{
					UEdGraphPin* GraphPin = UE::MVVM::ConversionFunctionHelper::FindPin(OldCondition->GetWrapperGraph(), Pin.GetId().GetNames());
					NewPins.Add(Pin.GetId().ToString(), MakeTuple(Pin, GraphPin));
				}
			}

			const FText ContextText = GetContextText(Details, NewCondition, OldCondition);
			DiffPins(Details, OldPins, NewPins, ContextText);
		}
	}
}

#endif //WITH_EDITOR


namespace UE::MVVM::FMVVMWidgetBlueprintDiff
{
void FindDiffs(const UWidgetBlueprint* NewBlueprint, const UWidgetBlueprint* OldBlueprint, FDiffResults& OutResults)
{

#if WITH_EDITOR
	UMVVMWidgetBlueprintExtension_View* NewExtension = NewBlueprint != nullptr ? UWidgetBlueprintExtension::GetExtension<UMVVMWidgetBlueprintExtension_View>(NewBlueprint) : nullptr;
	UMVVMWidgetBlueprintExtension_View* OldExtension = OldBlueprint != nullptr ? UWidgetBlueprintExtension::GetExtension<UMVVMWidgetBlueprintExtension_View>(OldBlueprint) : nullptr;

	UE::MVVM::Private::FMVVMBlueprintViewDiffDetails Details;
	Details.NewBlueprint = NewBlueprint;
	Details.NewBlueprintView = NewExtension != nullptr ? NewExtension->GetBlueprintView() : nullptr;
	Details.OldBlueprint = OldBlueprint;
	Details.OldBlueprintView = OldExtension != nullptr ? OldExtension->GetBlueprintView() : nullptr;
	Details.ResultsPtr = &OutResults;

	if (UE::MVVM::Private::DiffCustomObjectProvider.IsValid())
	{
		FDiffSingleResult Diff;
		Diff.Diff = EDiffType::CUSTOM_OBJECT;
		Diff.Category = EDiffType::MODIFICATION;
		Diff.Object1 = Details.NewBlueprintView;
		Diff.Object2 = Details.OldBlueprintView;
		Diff.DisplayString = LOCTEXT("MVVM_Bindings", "ViewModel Bindings");
		Diff.CustomObject = UE::MVVM::Private::DiffCustomObjectProvider->CreateObjectDiff(Details.NewBlueprint, Details.OldBlueprint);
		Details.ResultsPtr->Add(Diff);
	}

	UE::MVVM::Private::DiffBindings(Details);
	UE::MVVM::Private::DiffConditions(Details);
	UE::MVVM::Private::DiffEvents(Details);

#endif //WITH_EDITOR
}

void RegisterCustomDiff(TSharedPtr<FMVVMDiffCustomObjectProvider> DiffCustomObjectProvider)
{	
	UE::MVVM::Private::DiffCustomObjectProvider = DiffCustomObjectProvider;
}

void UnregisterCustomDiff()
{
	UE::MVVM::Private::DiffCustomObjectProvider.Reset();
}
}

#undef LOCTEXT_NAMESPACE
