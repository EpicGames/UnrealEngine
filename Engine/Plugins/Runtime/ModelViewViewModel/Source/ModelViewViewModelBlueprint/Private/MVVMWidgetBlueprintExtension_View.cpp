// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMWidgetBlueprintExtension_View.h"

#include "Blueprint/WidgetTree.h"
#include "Bindings/MVVMConversionFunctionHelper.h"
#include "Extensions/MVVMBlueprintViewExtension.h"
#include "FindInBlueprintManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "MVVMBlueprintInstancedViewModel.h"
#include "MVVMBlueprintPin.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewBinding.h"
#include "MVVMBlueprintViewCondition.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMBlueprintViewModel.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMPropertyPath.h"
#include "MVVMViewBlueprintCompiler.h"
#include "MVVMWidgetBlueprintDiff.h"
#include "ScopedTransaction.h"
#include "View/MVVMViewClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMWidgetBlueprintExtension_View)

#define LOCTEXT_NAMESPACE "MVVMBlueprintExtensionView"

namespace UE::MVVM::Private
{
	bool GAllowViewClass = true;
	static FAutoConsoleVariableRef CVarAllowViewClass(
		TEXT("MVVM.AllowViewClass"),
		GAllowViewClass,
		TEXT("Is the model view viewmodel view is allowed to be added to the generated Widget GeneratedClass."),
		ECVF_ReadOnly
	);

	bool GAutogeneratedFunctionsAreTransient = false;
	static FAutoConsoleVariableRef CVarAutogeneratedFunctionsAreTransient(
		TEXT("MVVM.AutogeneratedFunctionsAreTransient"),
		GAutogeneratedFunctionsAreTransient,
		TEXT("Is the autogenerated function should be mark as transient."),
		ECVF_ReadOnly
	);
}

void UMVVMWidgetBlueprintExtension_View::CreateBlueprintViewInstance()
{
	BlueprintView = NewObject<UMVVMBlueprintView>(this, FName(), RF_Transactional);
	BlueprintViewChangedDelegate.Broadcast();
	FBlueprintEditorUtils::MarkBlueprintAsModified(GetWidgetBlueprint());
}


void UMVVMWidgetBlueprintExtension_View::DestroyBlueprintViewInstance()
{
	BlueprintView = nullptr;
	BlueprintViewChangedDelegate.Broadcast();
}


void UMVVMWidgetBlueprintExtension_View::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_Transactional))
	{
		SetFlags(RF_Transactional);
	}
}

UMVVMBlueprintViewExtension* UMVVMWidgetBlueprintExtension_View::CreateBlueprintWidgetExtension(TSubclassOf<UMVVMBlueprintViewExtension> ExtensionClass, FName WidgetName)
{
	if (ensure(ExtensionClass.Get()))
	{
		UObject* ExtensionObj = NewObject<UObject>(this, ExtensionClass.Get(), NAME_None, RF_Transactional);
		UMVVMBlueprintViewExtension* NewExtension = CastChecked<UMVVMBlueprintViewExtension>(ExtensionObj);

		const FScopedTransaction Transaction(LOCTEXT("AddViewModelExtension", "Add viewmodel extension"));
		NewExtension->Modify();
		Modify();
		FMVVMExtensionItem ExtensionToAdd;
		ExtensionToAdd.WidgetName = WidgetName;
		ExtensionToAdd.ExtensionObj = NewExtension;
		BlueprintExtensions.Add(ExtensionToAdd);

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetWidgetBlueprint());
		return NewExtension;
	}
	return nullptr;
}

void UMVVMWidgetBlueprintExtension_View::RemoveBlueprintWidgetExtension(UMVVMBlueprintViewExtension* ExtensionToRemove, FName WidgetName)
{
	FMVVMExtensionItem Extension;
	Extension.WidgetName = WidgetName;
	Extension.ExtensionObj = ExtensionToRemove;

	const FScopedTransaction Transaction(LOCTEXT("RemoveViewModelExtension", "Remove viewmodel extension"));
	Modify();
	BlueprintExtensions.RemoveSingle(Extension);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetWidgetBlueprint());
}

TArray<UMVVMBlueprintViewExtension*> UMVVMWidgetBlueprintExtension_View::GetBlueprintExtensionsForWidget(FName WidgetName) const
{
	TArray<UMVVMBlueprintViewExtension*> ThisWidgetExtensions;
	for (const FMVVMExtensionItem& Extension : BlueprintExtensions)
	{
		if (Extension.WidgetName == WidgetName && Extension.ExtensionObj)
		{
			ThisWidgetExtensions.Add(Extension.ExtensionObj);
		}
	}
	return ThisWidgetExtensions;
}

TArray<UMVVMBlueprintViewExtension*> UMVVMWidgetBlueprintExtension_View::GetAllBlueprintExtensions() const
{
	TArray<UMVVMBlueprintViewExtension*> AllExtensions;
	AllExtensions.Reset(BlueprintExtensions.Num());
	
	for (const FMVVMExtensionItem& Extension : BlueprintExtensions)
	{
		if (Extension.ExtensionObj)
		{
			AllExtensions.Add(Extension.ExtensionObj);
		}
	}

	AllExtensions.Shrink();
	return AllExtensions;
}

void UMVVMWidgetBlueprintExtension_View::VerifyWidgetExtensions()
{
	if (const UWidgetBlueprint* WidgetBlueprint = GetWidgetBlueprint())
	{
		if (const UWidgetTree* WidgetTree = WidgetBlueprint->WidgetTree)
		{
			bool bModified = false;
			auto UpdateModify = [&bModified, Self=this]()
			{
				if (!bModified)
				{
					bModified = true;
					Self->Modify();
				}
			};

			TArray <FName, TInlineAllocator<4>> WidgetNamesToRemove;
			for (int32 Index = BlueprintExtensions.Num() - 1; Index >= 0; --Index)
			{
				if (BlueprintExtensions[Index].ExtensionObj == nullptr)
				{
					UpdateModify();
					BlueprintExtensions.RemoveAtSwap(Index);
				}
				else if (!BlueprintExtensions[Index].WidgetName.IsNone())
				{
					WidgetNamesToRemove.Add(BlueprintExtensions[Index].WidgetName);
				}
			}

			// Find widgets that are no longer in the tree and delete their extensions.
			if (WidgetNamesToRemove.Num() > 0)
			{
				WidgetTree->ForEachWidget([&WidgetNamesToRemove, this](TObjectPtr<UWidget> Widget) {
					if (Widget)
					{
						for (int32 Index = WidgetNamesToRemove.Num() - 1; Index >= 0; Index--)
						{
							const FName WidgetName = WidgetNamesToRemove[Index];
							if (WidgetName == Widget->GetFName())
							{
								WidgetNamesToRemove.RemoveSingleSwap(WidgetName);
							}
						}
					}
					});
			}

			if (WidgetNamesToRemove.Num() > 0)
			{
				UpdateModify();
				for (int32 Index = BlueprintExtensions.Num() - 1; Index >= 0; --Index)
				{
					if (WidgetNamesToRemove.Contains(BlueprintExtensions[Index].WidgetName))
					{
						BlueprintExtensions.RemoveAtSwap(Index);
					}
				}
			}
		}
	}
}

void UMVVMWidgetBlueprintExtension_View::OnFieldRenamed(UClass* FieldOwnerClass, FName OldName, FName NewName)
{
	const UWidgetBlueprint* WidgetBlueprint = GetWidgetBlueprint();
	if (WidgetBlueprint != nullptr && WidgetBlueprint->GeneratedClass == FieldOwnerClass)
	{
		RenameWidgetExtensions(OldName, NewName);
	}
}

void UMVVMWidgetBlueprintExtension_View::RenameWidgetExtensions(FName OldWidgetName, FName NewWidgetName)
{
	for (FMVVMExtensionItem& Extension : BlueprintExtensions)
	{
		if (Extension.WidgetName == OldWidgetName)
		{
			Modify();
			Extension.WidgetName = NewWidgetName;
			if (Extension.ExtensionObj)
			{
				Extension.ExtensionObj->WidgetRenamed(OldWidgetName, NewWidgetName);
			}
		}
	}
}

void UMVVMWidgetBlueprintExtension_View::HandlePreloadObjectsForCompilation(UBlueprint* OwningBlueprint)
{
	if (IsInGameThread() && BlueprintView)
	{
		BlueprintView->ConditionalPostLoad();

		for (const FMVVMBlueprintViewModelContext& AvailableViewModel : BlueprintView->GetViewModels())
		{
			if (AvailableViewModel.InstancedViewModel)
			{
				UBlueprint::ForceLoad(AvailableViewModel.InstancedViewModel);
				AvailableViewModel.InstancedViewModel->GenerateClass(true);
			}
			if (AvailableViewModel.GetViewModelClass())
			{
				UBlueprint::ForceLoad(AvailableViewModel.GetViewModelClass());
			}
		}
		for (FMVVMBlueprintViewBinding& Binding : BlueprintView->GetBindings())
		{
			if (Binding.Conversion.DestinationToSourceConversion)
			{
				UBlueprint::ForceLoad(Binding.Conversion.DestinationToSourceConversion);
			}
			if (Binding.Conversion.SourceToDestinationConversion)
			{
				UBlueprint::ForceLoad(Binding.Conversion.SourceToDestinationConversion);
			}
		}
	}
}


void UMVVMWidgetBlueprintExtension_View::HandleBeginCompilation(FWidgetBlueprintCompilerContext& InCreationContext)
{
	VerifyWidgetExtensions();

	for (const FMVVMBlueprintViewModelContext& AvailableViewModel : BlueprintView->GetViewModels())
	{
#if WITH_EDITORONLY_DATA
		UWidgetBlueprint* WidgetBlueprint = GetWidgetBlueprint();
		UClass* ViewModelClass = AvailableViewModel.GetViewModelClass();
		if (WidgetBlueprint && ViewModelClass)
		{
			if (UBlueprint* ViewModelBP = Cast<UBlueprint>(ViewModelClass->ClassGeneratedBy))
			{
				ViewModelBP->CachedDependents.Add(WidgetBlueprint);
				WidgetBlueprint->CachedDependencies.Add(ViewModelBP);
			}
		}
#endif // WITH_EDITORONLY_DATA
		
		if (AvailableViewModel.InstancedViewModel)
		{
			AvailableViewModel.InstancedViewModel->GenerateClass(false);
		}
	}

	CurrentCompilerContext.Reset();
	GeneratedFunctions.Reset();
	if (BlueprintView)
	{
		BlueprintView->ResetBindingMessages();
		for (UMVVMBlueprintViewEvent* ViewEvent : BlueprintView->GetEvents())
		{
			if(ViewEvent)
			{
				ViewEvent->ResetCompilationMessages();
			}
		}
		CurrentCompilerContext = MakePimpl<UE::MVVM::Private::FMVVMViewBlueprintCompiler>(InCreationContext, GetBlueprintView());
	}
}


void UMVVMWidgetBlueprintExtension_View::HandleEndCompilation()
{
	CurrentCompilerContext.Reset();
}


void UMVVMWidgetBlueprintExtension_View::HandleCleanAndSanitizeClass(UWidgetBlueprintGeneratedClass* ClassToClean, UObject* OldCDO)
{
	Super::HandleCleanAndSanitizeClass(ClassToClean, OldCDO);

	if (CurrentCompilerContext)
	{
		CurrentCompilerContext->CleanOldData(ClassToClean, OldCDO);
	}
}


void UMVVMWidgetBlueprintExtension_View::HandlePopulateGeneratedVariables(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context)
{
	Super::HandlePopulateGeneratedVariables(Context);

	CurrentCompilerContext->GatherGeneratedVariables(Context);
}


void UMVVMWidgetBlueprintExtension_View::HandleCreateClassVariablesFromBlueprint(const FWidgetBlueprintCompilerContext::FCreateVariableContext& Context)
{
	Super::HandleCreateClassVariablesFromBlueprint(Context);

	CurrentCompilerContext->CreateVariables(Context);
}


void UMVVMWidgetBlueprintExtension_View::HandleCreateFunctionList(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context)
{
	Super::HandleCreateFunctionList(Context);

	if (CurrentCompilerContext)
	{
		CurrentCompilerContext->CreateFunctions(Context);
	}
}


void UMVVMWidgetBlueprintExtension_View::HandleFinishCompilingClass(UWidgetBlueprintGeneratedClass* Class)
{
	Super::HandleFinishCompilingClass(Class);

	check(CurrentCompilerContext);

	if (CurrentCompilerContext->GetCompilerContext().bIsFullCompile)
	{
		UMVVMViewClass* ViewExtension = nullptr;
		bool bCompiled = false;
		if (CurrentCompilerContext->PreCompile(Class))
		{
			FName ClassName = "ViewClass";
			if (UObject* PreviousObj = StaticFindObjectFastInternal(nullptr, Class, ClassName, EFindObjectFlags::ExactClass))
			{
				// Remove previous object.
				ERenameFlags RenameFlags = REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors;
				FName TrashName = MakeUniqueObjectName(GetTransientPackage(), PreviousObj->GetClass(), *FString::Printf(TEXT("TRASH_%s"), *PreviousObj->GetName()));
				PreviousObj->Rename(*TrashName.ToString(), GetTransientPackage(), RenameFlags);
			}
			ViewExtension = NewObject<UMVVMViewClass>(Class);
			bCompiled = CurrentCompilerContext->Compile(Class, ViewExtension);
		}

		if (bCompiled)
		{
			check(ViewExtension);

			// Does it have any bindings
			if (ViewExtension->GetBindings().Num() > 0 || ViewExtension->GetEvents().Num() > 0 || ViewExtension->GetConditions().Num() > 0 || (GetBlueprintView()->GetViewModels().Num() > 0 && ViewExtension->IsCreatedWithoutBindings()))
			{
				// Test if parent also has a view
				if (Class->GetExtension<UMVVMViewClass>(true))
				{
					CurrentCompilerContext->GetCompilerContext().MessageLog.Warning(*LOCTEXT("MoreThanOneViewWarning", "There is more than one view. This can happen when inheriting from a widget blueprint that also contains a view. Please be aware that the parent view model is not available to the child widget.").ToString());
				}

				// If we are not allowed to add the view class
				if (UE::MVVM::Private::GAllowViewClass)
				{
					CurrentCompilerContext->AddExtension(Class, ViewExtension);
				}
			}
		}
	}

	GeneratedFunctions = CurrentCompilerContext->GetGeneratedFunctions();
	
	// If we can't auto-generate function add the transient flags
	if (UE::MVVM::Private::GAutogeneratedFunctionsAreTransient)
	{
		// If we are not allowed to add the view class, add the transient flags on added conversion graphs and event graphs.
		for (TFieldIterator<UFunction> FunctionIter(Class, EFieldIteratorFlags::ExcludeSuper); FunctionIter; ++FunctionIter)
		{
			UFunction* Function = *FunctionIter;
			Function->SetFlags(RF_Transient);
		}
	}
}


UMVVMWidgetBlueprintExtension_View::FSearchData UMVVMWidgetBlueprintExtension_View::HandleGatherSearchData(const UBlueprint* OwningBlueprint) const
{
	UMVVMWidgetBlueprintExtension_View::FSearchData SearchData;
	if (GetBlueprintView())
	{
		{
			TUniquePtr<FSearchArrayData> ViewModelContextSearchData = MakeUnique<FSearchArrayData>();
			ViewModelContextSearchData->Identifier = LOCTEXT("ViewmodelSearchTag", "Viewmodels");
			for (const FMVVMBlueprintViewModelContext& ViewModelContext : GetBlueprintView()->GetViewModels())
			{
				FSearchData& ViewModelSearchData = ViewModelContextSearchData->SearchSubList.AddDefaulted_GetRef();
				ViewModelSearchData.Datas.Emplace(LOCTEXT("ViewmodelGuidSearchTag", "Guid"), FText::FromString(ViewModelContext.GetViewModelId().ToString(EGuidFormats::Digits)));
				ViewModelSearchData.Datas.Emplace(FFindInBlueprintSearchTags::FiB_Name, ViewModelContext.GetDisplayName());
				ViewModelSearchData.Datas.Emplace(FFindInBlueprintSearchTags::FiB_ClassName, ViewModelContext.GetViewModelClass() ? ViewModelContext.GetViewModelClass()->GetDisplayNameText() : FText::GetEmpty());
				ViewModelSearchData.Datas.Emplace(LOCTEXT("ViewmodelCreationTypeSearchTag", "CreationType"), StaticEnum<EMVVMBlueprintViewModelContextCreationType>()->GetDisplayNameTextByValue((int64)ViewModelContext.CreationType));
			}
			SearchData.SearchArrayDatas.Add(MoveTemp(ViewModelContextSearchData));
		}
		{
			TUniquePtr<FSearchArrayData> BindingContextSearchData = MakeUnique<FSearchArrayData>();
			BindingContextSearchData->Identifier = LOCTEXT("BindingSearchTag", "Bindings");
			for (const FMVVMBlueprintViewBinding& Binding : GetBlueprintView()->GetBindings())
			{
				FSearchData& BindingSearchData = BindingContextSearchData->SearchSubList.AddDefaulted_GetRef();
				BindingSearchData.Datas.Emplace(LOCTEXT("ViewBindingSearchTag", "Binding"), FText::FromString(Binding.GetDisplayNameString(GetWidgetBlueprint())));
			}
			SearchData.SearchArrayDatas.Add(MoveTemp(BindingContextSearchData));
		}
	}
	return SearchData;
}

#if WITH_EDITOR

void UMVVMWidgetBlueprintExtension_View::FindDiffs(const UWidgetBlueprint* OwningBlueprint, const UWidgetBlueprint* OtherBlueprint, FDiffResults& Results) const
{
	UE::MVVM::FMVVMWidgetBlueprintDiff::FindDiffs(OwningBlueprint, OtherBlueprint, Results);
}

void UMVVMWidgetBlueprintExtension_View::GatherWidgetMemberReferences(FName WidgetVariableName, const UClass* WidgetClass, TSet<FName>& OutReferencedProperties, TSet<FName>& OutReferencedFunctions) const
{
	if (!BlueprintView || !WidgetClass || WidgetVariableName.IsNone())
	{
		return;
	}

	// Walk every hop in a property path; record any hop whose parent is in the widget class
	// chain.
	auto VisitPath = [&](const FMVVMBlueprintPropertyPath& Path)
	{
		if (Path.GetWidgetName() != WidgetVariableName)
		{
			return;
		}
		for (const FMVVMBlueprintFieldPath& FieldPath : Path.GetFieldPaths())
		{
			const UClass* ParentClass = FieldPath.GetParentClass(WidgetClass);
			if (!ParentClass || !WidgetClass->IsChildOf(ParentClass))
			{
				continue;
			}
			const FName MemberName = FieldPath.GetFieldName(WidgetClass);
			if (MemberName.IsNone())
			{
				continue;
			}
			if (FieldPath.GetBindingKind() == EBindingKind::Function)
			{
				OutReferencedFunctions.Add(MemberName);
			}
			else
			{
				OutReferencedProperties.Add(MemberName);
			}
		}
	};

	auto VisitConversionFunction = [&](const UMVVMBlueprintViewConversionFunction* ConversionFunction)
	{
		if (!ConversionFunction)
		{
			return;
		}

		VisitPath(ConversionFunction->GetDestinationPath());
		// Each saved pin is a conversion function argument whose value can be a widget property
		// path; surface those as references just like the binding's main paths.
		for (const FMVVMBlueprintPin& Pin : ConversionFunction->GetPins())
		{
			VisitPath(Pin.GetPath());
		}
	};

	for (const FMVVMBlueprintViewBinding& Binding : BlueprintView->GetBindings())
	{
		VisitPath(Binding.SourcePath);
		VisitPath(Binding.DestinationPath);
		VisitConversionFunction(Binding.Conversion.GetConversionFunction(/*bSourceToDestination=*/ true));
		VisitConversionFunction(Binding.Conversion.GetConversionFunction(/*bSourceToDestination=*/ false));
	}

	// Events
	for (const UMVVMBlueprintViewEvent* Event : BlueprintView->GetEvents())
	{
		if (!Event)
		{
			continue;
		}
		VisitPath(Event->GetEventPath());
		VisitPath(Event->GetDestinationPath());
		for (const FMVVMBlueprintPin& Pin : Event->GetPins())
		{
			VisitPath(Pin.GetPath());
		}
	}

	// Conditions: same shape as events — predicate path, destination path, and argument pins.
	for (const UMVVMBlueprintViewCondition* Condition : BlueprintView->GetConditions())
	{
		if (!Condition)
		{
			continue;
		}
		VisitPath(Condition->GetConditionPath());
		VisitPath(Condition->GetDestinationPath());
		for (const FMVVMBlueprintPin& Pin : Condition->GetPins())
		{
			VisitPath(Pin.GetPath());
		}
	}
}

#endif //WITH_EDITOR

void UMVVMWidgetBlueprintExtension_View::SetFilterSettings(FMVVMViewBindingFilterSettings InFilterSettings)
{
	FilterSettings = InFilterSettings;
}

#if WITH_EDITORONLY_DATA
void UMVVMWidgetBlueprintExtension_View::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsTemplate())
	{
		SetFilterSettings(GetDefault<UMVVMDeveloperProjectSettings>()->FilterSettings);
	}
}
#endif

#undef LOCTEXT_NAMESPACE
