// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelViewViewModelBlueprintModule.h"
#include "Modules/ModuleManager.h"


IMPLEMENT_MODULE(FDefaultModuleImpl, ModelViewViewModelBlueprint);
