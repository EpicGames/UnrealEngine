// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node.h"
#include "View/MVVMViewTypes.h"

#include "MVVMK2Node_AreSourcesValidForBinding.generated.h"

class UEdGraphPin;
class FKismetCompilerContext;

UCLASS(MinimalAPI, Hidden, HideDropDown)
class UMVVMK2Node_AreSourcesValidForBinding : public UK2Node
{
	GENERATED_BODY()

public:
	//~ Begin K2Node Interface.
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void AllocateDefaultPins() override;
	//~ End K2Node Interface.

	/** Get the then output pin */
	UEdGraphPin* GetThenPin() const;
	/** Get the else output pin */
	UEdGraphPin* GetElsePin() const;

	UPROPERTY()
	FMVVMViewClass_BindingKey BindingKey;
};

