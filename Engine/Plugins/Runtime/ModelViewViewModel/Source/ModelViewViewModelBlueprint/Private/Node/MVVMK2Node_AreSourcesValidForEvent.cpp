// Copyright Epic Games, Inc. All Rights Reserved.

#include "Node/MVVMK2Node_AreSourcesValidForEvent.h"

#include "BlueprintCompiledStatement.h"
#include "BlueprintNodeSpawner.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "MVVMSubsystem.h"
#include "Subsystems/SubsystemBlueprintLibrary.h"
#include "View/MVVMView.h"

#include "K2Node_CallFunction.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_Self.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMK2Node_AreSourcesValidForEvent)

#define LOCTEXT_NAMESPACE "MVVMK2Node_AreSourcesValidForEvent"

//////////////////////////////////////////////////////////////////////////

void UMVVMK2Node_AreSourcesValidForEvent::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	UEdGraphPin* TruePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	TruePin->PinFriendlyName = LOCTEXT("true", "true");

	UEdGraphPin* FalsePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Else);
	FalsePin->PinFriendlyName = LOCTEXT("false", "false");

	Super::AllocateDefaultPins();
}

void UMVVMK2Node_AreSourcesValidForEvent::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (!EventKey.IsValid())
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("NoEventKey", "Node @@ doesn't have a valid event key.").ToString(), this);
		BreakAllNodeLinks();
		return;
	}

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	UK2Node_CallFunction* CallGetSubsystemNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	{
		CallGetSubsystemNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(USubsystemBlueprintLibrary, GetEngineSubsystem), USubsystemBlueprintLibrary::StaticClass());
		CallGetSubsystemNode->AllocateDefaultPins();
		UEdGraphPin* CallCreateClassTypePin = CallGetSubsystemNode->FindPinChecked(FName("Class"));
		CallCreateClassTypePin->DefaultObject = UMVVMSubsystem::StaticClass();
	}

	UK2Node_CallFunction* CallGetViewNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	{
		CallGetViewNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UMVVMSubsystem, K2_GetViewFromUserWidget), UMVVMSubsystem::StaticClass());
		CallGetViewNode->AllocateDefaultPins();
	}

	UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, SourceGraph);
	{
		SelfNode->AllocateDefaultPins();
	}

	UK2Node_CallFunction* CallAreSourcesValidForEventNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	{
		CallAreSourcesValidForEventNode->FunctionReference.SetExternalMember(FName("AreSourcesValidForEvent"), UMVVMView::StaticClass());
		CallAreSourcesValidForEventNode->AllocateDefaultPins();

		UEdGraphPin* EventKeyPin = CallAreSourcesValidForEventNode->FindPinChecked(FName("EventKey"));
		Schema->TrySetDefaultValue(*EventKeyPin, LexToString(EventKey.GetIndex()));
	}

	UK2Node_IfThenElse* BranchNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
	{
		BranchNode->AllocateDefaultPins();
	}

	// Casting subsytem.result into MVVMSubsystem
	CallGetSubsystemNode->GetReturnValuePin()->PinType = CallGetViewNode->FindPinChecked(UEdGraphSchema_K2::PN_Self)->PinType;
	// subsystem.result to GetViewFromUserWidget.target
	ensure(Schema->TryCreateConnection(CallGetSubsystemNode->GetReturnValuePin(), CallGetViewNode->FindPinChecked(UEdGraphSchema_K2::PN_Self)));
	// self to GetViewFromUserWidget.UserWidget
	ensure(Schema->TryCreateConnection(SelfNode->FindPinChecked(UEdGraphSchema_K2::PN_Self), CallGetViewNode->FindPinChecked(FName("UserWidget"))));
	// GetViewFromUserWidget.Result to AreSourcesValidForEvent.target
	ensure(Schema->TryCreateConnection(CallGetViewNode->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue), CallAreSourcesValidForEventNode->FindPinChecked(UEdGraphSchema_K2::PN_Self)));
	// AreSourcesValidForEvent.result to branch.condition
	ensure(Schema->TryCreateConnection(CallAreSourcesValidForEventNode->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue), BranchNode->GetConditionPin()));
	// move this.exec to branch.exec
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *BranchNode->GetExecPin());
	// move this.then to branch.then
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *BranchNode->GetThenPin());

	BreakAllNodeLinks();
}

UEdGraphPin* UMVVMK2Node_AreSourcesValidForEvent::GetElsePin() const
{
	return FindPinChecked(UEdGraphSchema_K2::PN_Else);
}

#undef LOCTEXT_NAMESPACE
