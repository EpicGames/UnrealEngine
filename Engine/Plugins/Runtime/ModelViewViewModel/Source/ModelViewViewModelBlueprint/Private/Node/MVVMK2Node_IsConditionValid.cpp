// Copyright Epic Games, Inc. All Rights Reserved.

#include "Node/MVVMK2Node_IsConditionValid.h"

#include "BlueprintCompiledStatement.h"
#include "BlueprintNodeSpawner.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "MVVMSubsystem.h"
#include "Subsystems/SubsystemBlueprintLibrary.h"
#include "View/MVVMView.h"

#include "K2Node_CallFunction.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_Self.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMK2Node_IsConditionValid)

#define LOCTEXT_NAMESPACE "MVVMK2Node_IsConditionValid"

//////////////////////////////////////////////////////////////////////////

const FName PN_Operation("Operation");
const FName PN_Value("Value");
const FName PN_CompareValue("CompareValue");
const FName PN_CompareMaxValue("CompareMaxValue");

void UMVVMK2Node_IsConditionValid::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Byte, StaticEnum<EMVVMConditionOperation>(), PN_Operation);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Double, PN_Value);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Double, PN_CompareValue);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Double, PN_CompareMaxValue);

	UEdGraphPin* TruePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	TruePin->PinFriendlyName = LOCTEXT("true", "true");

	UEdGraphPin* FalsePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Else);
	FalsePin->PinFriendlyName = LOCTEXT("false", "false");

	Super::AllocateDefaultPins();
}

void UMVVMK2Node_IsConditionValid::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (!ConditionKey.IsValid())
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("NoConditionKey", "Node @@ doesn't have a valid condition key.").ToString(), this);
		BreakAllNodeLinks();
		return;
	}

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	UK2Node_CallFunction* CallGetSubsystemNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	{
		CallGetSubsystemNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(USubsystemBlueprintLibrary, GetEngineSubsystem), USubsystemBlueprintLibrary::StaticClass());
		CallGetSubsystemNode->AllocateDefaultPins();
		UEdGraphPin* CallCreateClassTypePin = CallGetSubsystemNode->FindPinChecked(FName("Class"));
		CallCreateClassTypePin->DefaultObject = UMVVMSubsystem::StaticClass();
	}

	UK2Node_CallFunction* CallCompareFloatValuesNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	{
		CallCompareFloatValuesNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UMVVMSubsystem, K2_CompareFloatValues), UMVVMSubsystem::StaticClass());
		CallCompareFloatValuesNode->AllocateDefaultPins();
	}

	UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, SourceGraph);
	{
		SelfNode->AllocateDefaultPins();
	}

	UK2Node_IfThenElse* BranchNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
	{
		BranchNode->AllocateDefaultPins();
	}

	// Casting subsytem.result into MVVMSubsystem
	CallGetSubsystemNode->GetReturnValuePin()->PinType = CallCompareFloatValuesNode->FindPinChecked(UEdGraphSchema_K2::PN_Self)->PinType;
	// subsystem.result to CompareFloatValues.target
	ensure(Schema->TryCreateConnection(CallGetSubsystemNode->GetReturnValuePin(), CallCompareFloatValuesNode->FindPinChecked(UEdGraphSchema_K2::PN_Self)));
	// CompareFloatValues.result to branch.condition
	ensure(Schema->TryCreateConnection(CallCompareFloatValuesNode->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue), BranchNode->GetConditionPin()));
	CompilerContext.MovePinLinksToIntermediate(*GetOperationPin(), *CallCompareFloatValuesNode->FindPinChecked(FName("Operation")));
	CompilerContext.MovePinLinksToIntermediate(*GetValuePin(), *CallCompareFloatValuesNode->FindPinChecked(FName("Value")));
	CompilerContext.MovePinLinksToIntermediate(*GetCompareValuePin(), *CallCompareFloatValuesNode->FindPinChecked(FName("CompareValue")));
	CompilerContext.MovePinLinksToIntermediate(*GetCompareMaxValuePin(), *CallCompareFloatValuesNode->FindPinChecked(FName("CompareMaxValue")));
	// connect compare.then to branch.exec
	ensure(Schema->TryCreateConnection(CallCompareFloatValuesNode->GetThenPin(), BranchNode->GetExecPin()));

	// move this.exec to compare.exec
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *CallCompareFloatValuesNode->GetExecPin());
	// move this.then to branch.then
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *BranchNode->GetThenPin());

	BreakAllNodeLinks();
}

UEdGraphPin* UMVVMK2Node_IsConditionValid::GetThenPin() const
{
	return FindPinChecked(UEdGraphSchema_K2::PN_Then);
}

UEdGraphPin* UMVVMK2Node_IsConditionValid::GetElsePin() const
{
	return FindPinChecked(UEdGraphSchema_K2::PN_Else);
}

UEdGraphPin* UMVVMK2Node_IsConditionValid::GetOperationPin() const
{
	return FindPinChecked(PN_Operation);
}

UEdGraphPin* UMVVMK2Node_IsConditionValid::GetValuePin() const
{
	return FindPinChecked(PN_Value);
}

UEdGraphPin* UMVVMK2Node_IsConditionValid::GetCompareValuePin() const
{
	return FindPinChecked(PN_CompareValue);
}

UEdGraphPin* UMVVMK2Node_IsConditionValid::GetCompareMaxValuePin() const
{
	return FindPinChecked(PN_CompareMaxValue);
}

#undef LOCTEXT_NAMESPACE
