// Copyright Epic Games, Inc. All Rights Reserved.

#include "Node/MVVMK2Node_LoadSoftResource.h"

#include "BlueprintCompiledStatement.h"
#include "BlueprintNodeSpawner.h"
#include "Blueprint/WidgetBlueprintLibrary.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "MVVMSubsystem.h"
#include "Subsystems/SubsystemBlueprintLibrary.h"
#include "View/MVVMView.h"

#include "K2Node_CallFunction.h"
#include "K2Node_ClassDynamicCast.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_Self.h"

#include "Engine/Texture2D.h"
#include "InputAction.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMK2Node_LoadSoftResource)


#define LOCTEXT_NAMESPACE "MVVMK2Node_LoadSoftResource"

//////////////////////////////////////////////////////////////////////////

void UMVVMK2Node_LoadSoftResource::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	// The immediate continue pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	// The delayed completed pin, this used to be called Then
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Completed);

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_SoftObject, GetInputResourceClass(), GetInputPinName());
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, GetInputResourceClass(), GetOutputPinName());

	UK2Node::AllocateDefaultPins();
}

void UMVVMK2Node_LoadSoftResource::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	UK2Node::ExpandNode(CompilerContext, SourceGraph);
	
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	UK2Node_LoadAsset* CallLoadAssetNode = CompilerContext.SpawnIntermediateNode<UK2Node_LoadAsset>(this, SourceGraph);
	{
		CallLoadAssetNode->AllocateDefaultPins();
	}
	UEdGraphPin* LoadAssetInput = CallLoadAssetNode->FindPinChecked(CallLoadAssetNode->GetInputPinName());
	UEdGraphPin* LoadAssetOutput = CallLoadAssetNode->FindPinChecked(CallLoadAssetNode->GetOutputPinName());

	UK2Node_DynamicCast* CastNode = CompilerContext.SpawnIntermediateNode<UK2Node_DynamicCast>(this, SourceGraph);
	{
		CastNode->TargetType = GetInputResourceClass();
		CastNode->AllocateDefaultPins();
	}
	UEdGraphPin* CastInput = CastNode->GetCastSourcePin();
	UEdGraphPin* CastOutput = CastNode->GetCastResultPin();

	// move this.exec to CallLoadAssetNode.exec
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *CallLoadAssetNode->GetExecPin());
	// move this.then to CallLoadAssetNode.then
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *CallLoadAssetNode->GetThenPin());
	// CallLoadAssetNode.completed to CastNode.exec
	ensure(Schema->TryCreateConnection(CallLoadAssetNode->FindPinChecked(UEdGraphSchema_K2::PN_Completed), CastNode->GetExecPin()));
	// move this.completed to CastNode.then
	CompilerContext.MovePinLinksToIntermediate(*GetCompletedPin(), *CastNode->GetThenPin());

	// move this.resource to CallLoadAssetNode.arg
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(GetInputPinName()), *LoadAssetInput);
	// CallLoadAssetNode.result to CastNode.Input
	ensure(Schema->TryCreateConnection(LoadAssetOutput, CastInput));
	// move this.result to CastNode.Output
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(GetOutputPinName()), *CastOutput);

	BreakAllNodeLinks();
}

UEdGraphPin* UMVVMK2Node_LoadSoftResource::GetThenPin() const
{
	return FindPinChecked(UEdGraphSchema_K2::PN_Then);
}

UEdGraphPin* UMVVMK2Node_LoadSoftResource::GetCompletedPin() const
{
	return FindPinChecked(UEdGraphSchema_K2::PN_Completed);
}

const FName& UMVVMK2Node_LoadSoftResource::GetOutputPinName() const
{
	static const FName OutputPinName("Result");
	return OutputPinName;
}

UClass* UMVVMK2Node_LoadSoftResource::GetInputResourceClass() const
{
	return nullptr;
}


//////////////////////////////////////////////////////////////////////////


FText UMVVMK2Node_LoadSoftTexture::GetTooltipText() const
{
	return FText(LOCTEXT("UMVVMK2Node_LoadSoftTextureGetTooltipText", "Asynchronously loads a Soft Texture Reference and returns that texture on successful load"));
}

FText UMVVMK2Node_LoadSoftTexture::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText(LOCTEXT("UMVVMK2Node_LoadSoftTextureGetNodeTitle", "Load From Soft Texture"));
}

const FName& UMVVMK2Node_LoadSoftTexture::GetInputPinName() const
{
	// Note: Our pin input should match the input of the specified MakeBrush node
	static const FName InputPinName("Texture");
	return InputPinName;
}

UClass* UMVVMK2Node_LoadSoftTexture::GetInputResourceClass() const
{
	return UTexture2D::StaticClass();
}


//////////////////////////////////////////////////////////////////////////


FText UMVVMK2Node_LoadSoftMaterial::GetTooltipText() const
{
	return FText(LOCTEXT("UMVVMK2Node_LoadSoftMaterialGetTooltipText", "Asynchronously loads a Soft Material Reference and returns that material on successful load"));
}

FText UMVVMK2Node_LoadSoftMaterial::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText(LOCTEXT("UMVVMK2Node_LoadSoftMaterialGetNodeTitle", "Load From Soft Material"));
}

const FName& UMVVMK2Node_LoadSoftMaterial::GetInputPinName() const
{
	// Note: Our pin input should match the input of the specified MakeBrush node
	static const FName InputPinName("Material");
	return InputPinName;
}

UClass* UMVVMK2Node_LoadSoftMaterial::GetInputResourceClass() const
{
	return UMaterialInterface::StaticClass();
}


//////////////////////////////////////////////////////////////////////////


FText UMVVMK2Node_LoadSoftInputAction::GetTooltipText() const
{
	return FText(LOCTEXT("UMVVMK2Node_LoadSoftInputActionGetTooltipText", "Asynchronously loads a Soft Input Action Reference and returns that Input Action on successful load"));
}

FText UMVVMK2Node_LoadSoftInputAction::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText(LOCTEXT("UMVVMK2Node_LoadSoftInputActionGetNodeTitle", "Load From Soft Input Action"));
}

const FName& UMVVMK2Node_LoadSoftInputAction::GetInputPinName() const
{
	static const FName InputPinName("InputAction");
	return InputPinName;
}

UClass* UMVVMK2Node_LoadSoftInputAction::GetInputResourceClass() const
{
	return UInputAction::StaticClass();
}

#undef LOCTEXT_NAMESPACE
