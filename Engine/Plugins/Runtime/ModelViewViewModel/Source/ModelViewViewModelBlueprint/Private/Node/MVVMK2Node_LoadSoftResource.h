// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node.h"
#include "K2Node_LoadAsset.h"
#include "View/MVVMViewTypes.h"

#include "MVVMK2Node_LoadSoftResource.generated.h"

class UEdGraphPin;
class FKismetCompilerContext;


/**
 * Utility async node to load a type-specific a soft resource (Texture, Material, etc). 
 * Needed since we current do not support automatic dynamic casting in MVVM
 */
UCLASS(MinimalAPI, Abstract, Hidden, HideDropDown)
class UMVVMK2Node_LoadSoftResource : public UK2Node_LoadAsset
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	//~ End UEdGraphNode Interface.
	
	//~ Begin K2Node Interface.
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//~ End K2Node Interface.

	/** Get the then output pin */
	UEdGraphPin* GetThenPin() const;

	/** Get the then output pin */
	UEdGraphPin* GetCompletedPin() const;

protected:

	//~ Begin UK2Node_LoadAsset Interface.
	virtual const FName& GetOutputPinName() const;
	//~ End UK2Node_LoadAsset Interface.

	/** Get the type of the input arg pin */
	virtual UClass* GetInputResourceClass() const;
};


//////////////////////////////////////////////////////////////////////////


UCLASS(MinimalAPI)
class UMVVMK2Node_LoadSoftTexture : public UMVVMK2Node_LoadSoftResource
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface.
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface.

protected:
	//~ Begin UK2Node_LoadAsset Interface.
	virtual const FName& GetInputPinName() const;
	//~ End UK2Node_LoadAsset Interface.

	//~ Begin UMVVMK2Node_LoadSoftResource Interface.
	virtual UClass* GetInputResourceClass() const override;
	//~ End UMVVMK2Node_LoadSoftResource Interface.
};


//////////////////////////////////////////////////////////////////////////


UCLASS(MinimalAPI)
class UMVVMK2Node_LoadSoftMaterial: public UMVVMK2Node_LoadSoftResource
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface.
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface.

protected:
	//~ Begin UK2Node_LoadAsset Interface.
	virtual const FName& GetInputPinName() const;
	//~ End UK2Node_LoadAsset Interface.

	//~ Begin UMVVMK2Node_LoadSoftResource Interface.
	virtual UClass* GetInputResourceClass() const override;
	//~ End UMVVMK2Node_LoadSoftResource Interface.
};

UCLASS(MinimalAPI)
class UMVVMK2Node_LoadSoftInputAction : public UMVVMK2Node_LoadSoftResource
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface.
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface.

protected:
	//~ Begin UK2Node_LoadAsset Interface.
	virtual const FName& GetInputPinName() const;
	//~ End UK2Node_LoadAsset Interface.

	//~ Begin UMVVMK2Node_LoadSoftResource Interface.
	virtual UClass* GetInputResourceClass() const override;
	//~ End UMVVMK2Node_LoadSoftResource Interface.
};
