// Copyright Epic Games, Inc. All Rights Reserved.

#include "Node/MVVMK2Node_MakeBrushFromSoftResource.h"

#include "BlueprintCompiledStatement.h"
#include "BlueprintNodeSpawner.h"
#include "Blueprint/WidgetBlueprintLibrary.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "MVVMSubsystem.h"
#include "Subsystems/SubsystemBlueprintLibrary.h"
#include "View/MVVMView.h"

#include "K2Node_CallFunction.h"
#include "K2Node_ClassDynamicCast.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_Self.h"

#include "Engine/Texture2D.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMK2Node_MakeBrushFromSoftResource)


#define LOCTEXT_NAMESPACE "MVVMK2Node_MakeBrushFromSoftResource"

namespace UE::MVVM::Private
{
	namespace PinNames
	{
		static const FName Width = "Width";
		static const FName Height = "Height";
	}
}

//////////////////////////////////////////////////////////////////////////

void UMVVMK2Node_MakeBrushFromSoftResource::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	// The immediate continue pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	// The delayed completed pin, this used to be called Then
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Completed);

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_SoftObject, GetInputResourceClass(), GetInputPinName());
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Int, nullptr, UE::MVVM::Private::PinNames::Width);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Int, nullptr, UE::MVVM::Private::PinNames::Height);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FSlateBrush::StaticStruct(), GetOutputPinName());

	UK2Node::AllocateDefaultPins();
}

void UMVVMK2Node_MakeBrushFromSoftResource::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	UK2Node::ExpandNode(CompilerContext, SourceGraph);
	
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	UK2Node_LoadAsset* CallLoadAssetNode = CompilerContext.SpawnIntermediateNode<UK2Node_LoadAsset>(this, SourceGraph);
	{
		CallLoadAssetNode->AllocateDefaultPins();
	}
	UEdGraphPin* LoadAssetInput = CallLoadAssetNode->FindPinChecked(CallLoadAssetNode->GetInputPinName());
	UEdGraphPin* LoadAssetOutput = CallLoadAssetNode->FindPinChecked(CallLoadAssetNode->GetOutputPinName());

	UK2Node_CallFunction* CallMakeBrushFrom = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	{
		CallMakeBrushFrom->FunctionReference.SetExternalMember(GetMakeBrushFunctionName(), UWidgetBlueprintLibrary::StaticClass());
		CallMakeBrushFrom->AllocateDefaultPins();
	}
	// Note: Our pin input should match the input of the specified MakeBrush node
	UEdGraphPin* CallMakeBrushFromResourcePin = CallMakeBrushFrom->FindPinChecked(GetInputPinName());
	UEdGraphPin* CallMakeBrushFromWidthPin = CallMakeBrushFrom->FindPinChecked(UE::MVVM::Private::PinNames::Width);
	UEdGraphPin* CallMakeBrushFromHeightPin = CallMakeBrushFrom->FindPinChecked(UE::MVVM::Private::PinNames::Height);
	UEdGraphPin* CallMakeBrushFromResultPin = CallMakeBrushFrom->GetReturnValuePin();

	UK2Node_DynamicCast* CastNode = CompilerContext.SpawnIntermediateNode<UK2Node_DynamicCast>(this, SourceGraph);
	{
		CastNode->TargetType = GetInputResourceClass();
		CastNode->AllocateDefaultPins();
	}
	UEdGraphPin* CastInput = CastNode->GetCastSourcePin();
	UEdGraphPin* CastOutput = CastNode->GetCastResultPin();

	// move this.exec to CallLoadAssetNode.exec
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *CallLoadAssetNode->GetExecPin());
	// move this.then to CallLoadAssetNode.then
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *CallLoadAssetNode->GetThenPin());
	// CallLoadAssetNode.completed to CastNode.exec
	ensure(Schema->TryCreateConnection(CallLoadAssetNode->FindPinChecked(UEdGraphSchema_K2::PN_Completed), CastNode->GetExecPin()));
	// move this.completed to CastNode.then
	CompilerContext.MovePinLinksToIntermediate(*GetCompletedPin(), *CastNode->GetThenPin());

	// move this.resource to CallLoadAssetNode.arg
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(GetInputPinName()), *LoadAssetInput);
	// move this.width to CallMakeBrushFrom.width
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UE::MVVM::Private::PinNames::Width), *CallMakeBrushFromWidthPin);
	// move this.height to CallMakeBrushFrom.height
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UE::MVVM::Private::PinNames::Height), *CallMakeBrushFromHeightPin);
	// CallLoadAssetNode.result to CastNode.Input
	ensure(Schema->TryCreateConnection(LoadAssetOutput, CastInput));
	// CastNode.Output to CallMakeBrushFrom.Resource
	ensure(Schema->TryCreateConnection(CastOutput, CallMakeBrushFromResourcePin));
	// move this.result to CallLoadAssetNode.result
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(GetOutputPinName()), *CallMakeBrushFromResultPin);

	BreakAllNodeLinks();
}

UEdGraphPin* UMVVMK2Node_MakeBrushFromSoftResource::GetThenPin() const
{
	return FindPinChecked(UEdGraphSchema_K2::PN_Then);
}

UEdGraphPin* UMVVMK2Node_MakeBrushFromSoftResource::GetCompletedPin() const
{
	return FindPinChecked(UEdGraphSchema_K2::PN_Completed);
}

const FName& UMVVMK2Node_MakeBrushFromSoftResource::GetOutputPinName() const
{
	static const FName OutputPinName("SlateBrush");
	return OutputPinName;
}

UClass* UMVVMK2Node_MakeBrushFromSoftResource::GetInputResourceClass() const
{
	return nullptr;
}

FName UMVVMK2Node_MakeBrushFromSoftResource::GetMakeBrushFunctionName() const
{
	return NAME_None;
}


//////////////////////////////////////////////////////////////////////////


FText UMVVMK2Node_MakeBrushFromSoftTexture::GetTooltipText() const
{
	return FText(LOCTEXT("UMVVMK2Node_MakeBrushFromSoftTextureGetTooltipText", "Asynchronously loads a Soft Texture Reference and returns a slate brush using that texture on successful load"));
}

FText UMVVMK2Node_MakeBrushFromSoftTexture::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText(LOCTEXT("UMVVMK2Node_MakeBrushFromSoftTextureGetNodeTitle", "Make Brush From Soft Texture"));
}

const FName& UMVVMK2Node_MakeBrushFromSoftTexture::GetInputPinName() const
{
	// Note: Our pin input should match the input of the specified MakeBrush node
	static const FName InputPinName("Texture");
	return InputPinName;
}

UClass* UMVVMK2Node_MakeBrushFromSoftTexture::GetInputResourceClass() const
{
	return UTexture2D::StaticClass();
}

FName UMVVMK2Node_MakeBrushFromSoftTexture::GetMakeBrushFunctionName() const
{
	return GET_FUNCTION_NAME_CHECKED(UWidgetBlueprintLibrary, MakeBrushFromTexture);
}


//////////////////////////////////////////////////////////////////////////


FText UMVVMK2Node_MakeBrushFromSoftMaterial::GetTooltipText() const
{
	return FText(LOCTEXT("UMVVMK2Node_MakeBrushFromSoftMaterialGetTooltipText", "Asynchronously loads a Soft Material Reference and returns a slate brush using that material on successful load"));
}

FText UMVVMK2Node_MakeBrushFromSoftMaterial::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText(LOCTEXT("UMVVMK2Node_MakeBrushFromSoftMaterialGetNodeTitle", "Make Brush From Soft Material"));
}

const FName& UMVVMK2Node_MakeBrushFromSoftMaterial::GetInputPinName() const
{
	// Note: Our pin input should match the input of the specified MakeBrush node
	static const FName InputPinName("Material");
	return InputPinName;
}

UClass* UMVVMK2Node_MakeBrushFromSoftMaterial::GetInputResourceClass() const
{
	return UMaterialInterface::StaticClass();
}

FName UMVVMK2Node_MakeBrushFromSoftMaterial::GetMakeBrushFunctionName() const
{
	return GET_FUNCTION_NAME_CHECKED(UWidgetBlueprintLibrary, MakeBrushFromMaterial);
}

#undef LOCTEXT_NAMESPACE
