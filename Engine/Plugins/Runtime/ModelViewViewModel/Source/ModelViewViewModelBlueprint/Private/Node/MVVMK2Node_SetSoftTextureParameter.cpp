// Copyright Epic Games, Inc. All Rights Reserved.

#include "Node/MVVMK2Node_SetSoftTextureParameter.h"

#include "Bindings/ConversionLibraries/MVVMSlateBrushConversionLibrary.h"
#include "KismetCompiler.h"
#include "K2Node_CallFunction.h"
#include "K2Node_ClassDynamicCast.h"

#include "Engine/Texture2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMK2Node_SetSoftTextureParameter)

#define LOCTEXT_NAMESPACE "MVVMK2Node_SetSoftTextureParameter"

namespace UE::MVVM::Private
{
	namespace PinNames
	{
		static const FName TargetBrushName("TargetBrush");
		static const FName InputPinName("Texture");
		static const FName ParamName("ParameterName");
		static const FName OutputPinName("SlateBrush");
	}
}

//////////////////////////////////////////////////////////////////////////

void UMVVMK2Node_SetSoftTextureParameter::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	// The immediate continue pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	// The delayed completed pin, this used to be called Then
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Completed);
	
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FSlateBrush::StaticStruct(), UE::MVVM::Private::PinNames::TargetBrushName);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Name, nullptr, UE::MVVM::Private::PinNames::ParamName);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_SoftObject, UTexture2D::StaticClass(), UE::MVVM::Private::PinNames::InputPinName);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FSlateBrush::StaticStruct(), UE::MVVM::Private::PinNames::OutputPinName);

	UK2Node::AllocateDefaultPins();
}

void UMVVMK2Node_SetSoftTextureParameter::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	UK2Node::ExpandNode(CompilerContext, SourceGraph);
	
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	UK2Node_LoadAsset* CallLoadAssetNode = CompilerContext.SpawnIntermediateNode<UK2Node_LoadAsset>(this, SourceGraph);
	{
		CallLoadAssetNode->AllocateDefaultPins();
	}
	UEdGraphPin* LoadAssetInput = CallLoadAssetNode->FindPinChecked(CallLoadAssetNode->GetInputPinName());
	UEdGraphPin* LoadAssetOutput = CallLoadAssetNode->FindPinChecked(CallLoadAssetNode->GetOutputPinName());

	UK2Node_CallFunction* CallSetTextureParameter = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	{
		CallSetTextureParameter->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UMVVMSlateBrushConversionLibrary, Conv_SetTextureParameter), UMVVMSlateBrushConversionLibrary::StaticClass());
		CallSetTextureParameter->AllocateDefaultPins();
	}

	UEdGraphPin* CallSetTextureParameterTexturePin = CallSetTextureParameter->FindPinChecked(FName("Value"));
	UEdGraphPin* CallSetTextureParameterBrushPin = CallSetTextureParameter->FindPinChecked(FName("Brush"));
	UEdGraphPin* CallSetTextureParameterParameterPin = CallSetTextureParameter->FindPinChecked(FName("ParameterName"));
	UEdGraphPin* CallSetTextureParameterResultPin = CallSetTextureParameter->GetReturnValuePin();

	UK2Node_DynamicCast* CastNode = CompilerContext.SpawnIntermediateNode<UK2Node_DynamicCast>(this, SourceGraph);
	{
		CastNode->TargetType = UTexture2D::StaticClass();
		CastNode->AllocateDefaultPins();
	}
	UEdGraphPin* CastInput = CastNode->GetCastSourcePin();
	UEdGraphPin* CastOutput = CastNode->GetCastResultPin();

	// move this.exec to CallLoadAssetNode.exec
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *CallLoadAssetNode->GetExecPin());
	// move this.then to CallLoadAssetNode.then
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *CallLoadAssetNode->GetThenPin());
	// CallLoadAssetNode.completed to CastNode.exec
	ensure(Schema->TryCreateConnection(CallLoadAssetNode->FindPinChecked(UEdGraphSchema_K2::PN_Completed), CastNode->GetExecPin()));
	// move this.completed to CastNode.then
	CompilerContext.MovePinLinksToIntermediate(*GetCompletedPin(), *CastNode->GetThenPin());

	// move this.resource to CallLoadAssetNode.arg
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UE::MVVM::Private::PinNames::InputPinName), *LoadAssetInput);
	// move the target brush pin to the SetTextureParameter brush pin
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UE::MVVM::Private::PinNames::TargetBrushName), *CallSetTextureParameterBrushPin);
	// set the parameter name on the SetTextureParameter name pin
	CallSetTextureParameterParameterPin->DefaultValue = FindPinChecked(UE::MVVM::Private::PinNames::ParamName)->DefaultValue;
	// CallLoadAssetNode.result to CastNode.Input
	ensure(Schema->TryCreateConnection(LoadAssetOutput, CastInput));
	// CastNode.Output to SetTextureParameter.Value
	ensure(Schema->TryCreateConnection(CastOutput, CallSetTextureParameterTexturePin));
	// move this.result to SetTextureParameter.result
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(GetOutputPinName()), *CallSetTextureParameterResultPin);

	BreakAllNodeLinks();
}

UEdGraphPin* UMVVMK2Node_SetSoftTextureParameter::GetThenPin() const
{
	return FindPinChecked(UEdGraphSchema_K2::PN_Then);
}

UEdGraphPin* UMVVMK2Node_SetSoftTextureParameter::GetCompletedPin() const
{
	return FindPinChecked(UEdGraphSchema_K2::PN_Completed);
}

const FName& UMVVMK2Node_SetSoftTextureParameter::GetOutputPinName() const
{
	return UE::MVVM::Private::PinNames::OutputPinName;
}

const FName& UMVVMK2Node_SetSoftTextureParameter::GetInputPinName() const
{
	return UE::MVVM::Private::PinNames::InputPinName;
}

FText UMVVMK2Node_SetSoftTextureParameter::GetTooltipText() const
{
	return FText(LOCTEXT("UMVVMK2Node_SetSoftTextureParameterGetTooltipText", "Asynchronously loads a Soft Texture Reference and sets the texture property on the slate brush"));
}

FText UMVVMK2Node_SetSoftTextureParameter::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText(LOCTEXT("UMVVMK2Node_SetSoftTextureParameterGetNodeTitle", "Set Soft Texture Parameter"));
}

#undef LOCTEXT_NAMESPACE
