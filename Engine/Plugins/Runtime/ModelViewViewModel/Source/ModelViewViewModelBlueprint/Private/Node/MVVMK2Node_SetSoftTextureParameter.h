// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node.h"
#include "K2Node_LoadAsset.h"
#include "View/MVVMViewTypes.h"

#include "MVVMK2Node_SetSoftTextureParameter.generated.h"

class UEdGraphPin;
class FKismetCompilerContext;


/**
 * Utility async node to create a slate brush from a soft resource (Texture, Material, etc). 
 * Needed since we cannot nest conversion functions (Ex: LoadAsset->MakeBrushFromTexture).
 */
UCLASS(MinimalAPI, Category = "Widget", meta = (MVVMBindToDestination = "TargetBrush"))
class UMVVMK2Node_SetSoftTextureParameter : public UK2Node_LoadAsset
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	//~ End UEdGraphNode Interface.
	
	//~ Begin K2Node Interface.
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//~ End K2Node Interface.

	/** Get the then output pin */
	UEdGraphPin* GetThenPin() const;

	/** Get the completed output pin */
	UEdGraphPin* GetCompletedPin() const;

protected:

	//~ Begin UK2Node_LoadAsset Interface.
	virtual const FName& GetOutputPinName() const override;
	virtual const FName& GetInputPinName() const override;
	//~ End UK2Node_LoadAsset Interface.

public:
	//~ Begin UEdGraphNode Interface.
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface.
};


