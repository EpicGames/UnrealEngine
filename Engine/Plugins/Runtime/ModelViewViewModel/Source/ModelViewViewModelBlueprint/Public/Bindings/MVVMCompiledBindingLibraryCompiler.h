// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MVVMCompiledBindingLibrary.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

namespace UE::MVVM { struct FMVVMConstFieldVariant; }
template <typename T> class TSubclassOf;


namespace UE::MVVM::Private
{
	class FCompiledBindingLibraryCompilerImpl;
} //namespace UE::MVVM::Private


namespace UE::MVVM
{

/**  */
class FCompiledBindingLibraryCompiler
{
public:
	/** */
	struct FFieldPathHandle
	{
	public:
		explicit FFieldPathHandle()
			: Id(0)
		{
		}

		static FFieldPathHandle MakeHandle()
		{
			FFieldPathHandle Handle;
			++IdGenerator;
			Handle.Id = IdGenerator;
			return Handle;
		}

		bool IsValid() const
		{
			return Id != 0;
		}

		bool operator==(const FFieldPathHandle& Other) const
		{
			return Id == Other.Id;
		}

		bool operator!=(const FFieldPathHandle& Other) const
		{
			return Id != Other.Id;
		}

		friend uint32 GetTypeHash(const FFieldPathHandle& Handle)
		{
			return ::GetTypeHash(Handle.Id);
		}

	private:
		static int32 IdGenerator;
		int32 Id;
	};

	/** */
	struct FBindingHandle
	{
	public:
		explicit FBindingHandle()
			: Id(0)
		{
		}

		static FBindingHandle MakeHandle()
		{
			FBindingHandle Handle;
			++IdGenerator;
			Handle.Id = IdGenerator;
			return Handle;
		}

		bool IsValid() const
		{
			return Id != 0;
		}

		bool operator==(const FBindingHandle& Other) const
		{
			return Id == Other.Id;
		}

		bool operator!=(const FBindingHandle& Other) const
		{
			return Id != Other.Id;
		}

		friend uint32 GetTypeHash(const FBindingHandle& Handle)
		{
			return ::GetTypeHash(Handle.Id);
		}

	private:
		static int32 IdGenerator;
		int32 Id;
	};

	/** */
	struct FFieldIdHandle
	{
	public:
		explicit FFieldIdHandle()
			: Id(0)
		{
		}

		static FFieldIdHandle MakeHandle()
		{
			FFieldIdHandle Handle;
			++IdGenerator;
			Handle.Id = IdGenerator;
			return Handle;
		}

		bool IsValid() const
		{
			return Id != 0;
		}

		bool operator==(const FFieldIdHandle& Other) const
		{
			return Id == Other.Id;
		}

		bool operator!=(const FFieldIdHandle& Other) const
		{
			return Id != Other.Id;
		}

		friend uint32 GetTypeHash(const FFieldIdHandle& Handle)
		{
			return ::GetTypeHash(Handle.Id);
		}

	private:
		static int32 IdGenerator;
		int32 Id;
	};

public:
	UE_API FCompiledBindingLibraryCompiler(UBlueprint* Context);

public:
	/** */
	UE_API TValueOrError<FFieldIdHandle, FText> AddFieldId(const UClass* SourceClass, FName FieldName);
	
	/** */
	UE_API TValueOrError<FFieldPathHandle, FText> AddFieldPath(TArrayView<const UE::MVVM::FMVVMConstFieldVariant> FieldPath, bool bRead);

	/** */
	UE_API TValueOrError<FFieldPathHandle, FText> AddObjectFieldPath(TArrayView<const UE::MVVM::FMVVMConstFieldVariant> FieldPath, const UClass* ExpectedType, bool bRead);

	/** */
	UE_API TValueOrError<FFieldPathHandle, FText> AddConversionFunctionFieldPath(const UClass* SourceClass, const UFunction* Function);

	/** */
	UE_API TValueOrError<FFieldPathHandle, FText> AddDelegateSignatureFieldPath(const UClass* SourceClass, const UFunction* Function);

	/** */
	UE_API TValueOrError<FBindingHandle, FText> AddBinding(FFieldPathHandle Source, FFieldPathHandle Destination);

	/** */
	UE_API TValueOrError<FBindingHandle, FText> AddBinding(FFieldPathHandle Source, FFieldPathHandle Destination, FFieldPathHandle ConversionFunction);

	/** */
	UE_API TValueOrError<FBindingHandle, FText> AddComplexBinding(FFieldPathHandle Destination, FFieldPathHandle ConversionFunction);

	struct FCompileResult
	{
		FCompileResult(FGuid LibraryId);
		FMVVMCompiledBindingLibrary Library;
		TMap<FFieldPathHandle, FMVVMVCompiledFieldPath> FieldPaths;
		TMap<FBindingHandle, FMVVMVCompiledBinding> Bindings;
		TMap<FFieldIdHandle, UE::FieldNotification::FFieldId> FieldIds;
	};

	/** */
	UE_API TValueOrError<FCompileResult, FText> Compile(FGuid LibraryId);

private:
	/** */
	UE_API TValueOrError<FFieldPathHandle, FText> AddFieldPathImpl(TArrayView<const UE::MVVM::FMVVMConstFieldVariant> FieldPath, bool bRead);
	UE_API TValueOrError<FBindingHandle, FText> AddBindingImpl(FFieldPathHandle Source, FFieldPathHandle Destination, FFieldPathHandle ConversionFunction, bool bIsComplexBinding);
	UE_API TValueOrError<FFieldPathHandle, FText> AddFunctionFieldPathImpl(const UClass* InSourceClass, const UFunction* InFunction, TFunctionRef<FText(const UFunction*)> ValidateFunctionCallback);

	TPimplPtr<Private::FCompiledBindingLibraryCompilerImpl> Impl;
};

} //namespace UE::MVVM

#undef UE_API
