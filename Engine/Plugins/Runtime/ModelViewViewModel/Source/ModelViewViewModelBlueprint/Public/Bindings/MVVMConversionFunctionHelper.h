// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Templates/Function.h"
#include "Templates/SubclassOf.h"
#include "Templates/ValueOrError.h"

struct FMVVMBlueprintFunctionReference;
struct FMVVMBlueprintPropertyPath;
struct FMVVMBlueprintViewBinding;

class UBlueprint;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UK2Node;
class UK2Node_CallFunction;
class FKismetCompilerContext;
class UMVVMBlueprintView;
namespace UE::MVVM { struct FMVVMConstFieldVariant; }

namespace UE::MVVM::ConversionFunctionHelper
{
	/** Conversion function requires a wrapper. */
	MODELVIEWVIEWMODELBLUEPRINT_API bool RequiresWrapper(const UFunction* Function);

	/** The pin is valid to use with a PropertyPath. */
	MODELVIEWVIEWMODELBLUEPRINT_API bool IsInputPin(const UEdGraphPin* Pin);
	
	/** Find the property path of a given argument in the conversion function. */
	MODELVIEWVIEWMODELBLUEPRINT_API FMVVMBlueprintPropertyPath GetPropertyPathForPin(const UBlueprint* WidgetBlueprint, const UEdGraphPin* Pin, bool bSkipResolve);
	
	/** Set the property path of a given argument in the conversion function. */
	MODELVIEWVIEWMODELBLUEPRINT_API void SetPropertyPathForPin(const UBlueprint* Blueprint, const FMVVMBlueprintPropertyPath& PropertyPath, UEdGraphPin* Pin);
	
	/** Find the property path of a given argument in the conversion function. */
	MODELVIEWVIEWMODELBLUEPRINT_API FMVVMBlueprintPropertyPath GetPropertyPathForArgument(const UBlueprint* WidgetBlueprint, const UK2Node_CallFunction* Function, FName ArgumentName, bool bSkipResolve);

	/** Create the name of the conversion function wrapper this binding should have. */
	MODELVIEWVIEWMODELBLUEPRINT_API FName CreateWrapperName(const FMVVMBlueprintViewBinding& Binding, bool bSourceToDestination);

	/**
	 * If we can create a graph to set a property/function.
	 */
	MODELVIEWVIEWMODELBLUEPRINT_API TValueOrError<void, FText> CanCreateSetterGraph(UBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& PropertyPath);

	struct FCreateGraphResult
	{
		/** The new graph created. */
		UEdGraph* NewGraph = nullptr;
		/** Node that owns the pins. */
		UK2Node* WrappedNode = nullptr;
		/** Nodes of relevance beyond the wrapped node, keyed by name. */
		TMap<FName, UK2Node*> NamedNodes;
		/** True if this graph belongs in an ubergraph page */
		bool bIsUbergraphPage = false;
	};

	struct FCreateGraphParams
	{
		bool bIsConst = false;
		bool bTransient = false;
		bool bIsForEvent = false;

		/** If true implies this graph will create events*/
		bool bCreateUbergraphPage = false;
	};

	/**
	 * Create a graph to set a property/function. Used by Event Bindings
	 */
	MODELVIEWVIEWMODELBLUEPRINT_API TValueOrError<FCreateGraphResult, FText> CreateSetterGraph(UBlueprint* WidgetBlueprint, FName GraphName, const UFunction* Signature, const FMVVMBlueprintPropertyPath& PropertyPath, FCreateGraphParams InParams);

	/**
	 * Create a graph to set a property/function. Used by conversion functions for async K2 Nodes.
	 */
	MODELVIEWVIEWMODELBLUEPRINT_API TValueOrError<FCreateGraphResult, FText> CreateSetterGraph(UBlueprint* WidgetBlueprint, FName GraphName, const TSubclassOf<UK2Node> Node, const FMVVMBlueprintPropertyPath& PropertyPath, FCreateGraphParams InParams);

	/** Create a graph, used by conversion functions for UFunctions */
	MODELVIEWVIEWMODELBLUEPRINT_API FCreateGraphResult CreateGraph(UBlueprint* WidgetBlueprint, FName GraphName, const UFunction* Signature, const UFunction* FunctionToWrap, FCreateGraphParams FCreateGraInParamsphParams);

	/** Create a graph, used by conversion functions for K2 Nodes */
	MODELVIEWVIEWMODELBLUEPRINT_API FCreateGraphResult CreateGraph(UBlueprint* WidgetBlueprint, FName GraphName, const UFunction* Signature, const TSubclassOf<UK2Node> Node, FCreateGraphParams InParams, TFunctionRef<void(UK2Node*)> InitNodeCallback);

	UE_DEPRECATED(5.5, "Call the version of CreateSetterGraph that takes a FCreateGraphParams instead.")
	MODELVIEWVIEWMODELBLUEPRINT_API TValueOrError<FCreateGraphResult, FText> CreateSetterGraph(UBlueprint* WidgetBlueprint, FName GraphName, const UFunction* Signature, const FMVVMBlueprintPropertyPath& PropertyPath, bool bIsConst, bool bTransient, const bool bIsForEvent);
	UE_DEPRECATED(5.5, "Call the version of CreateGraph that takes a FCreateGraphParams instead.")
	MODELVIEWVIEWMODELBLUEPRINT_API FCreateGraphResult CreateGraph(UBlueprint* WidgetBlueprint, FName GraphName, const UFunction* Signature, const UFunction* FunctionToWrap, bool bIsConst, bool bTransient);
	UE_DEPRECATED(5.5, "Call the version of CreateGraph that takes a FCreateGraphParams instead.")
	MODELVIEWVIEWMODELBLUEPRINT_API FCreateGraphResult CreateGraph(UBlueprint* WidgetBlueprint, FName GraphName, const UFunction* Signature, const TSubclassOf<UK2Node> Node, bool bIsConst, bool bTransient, TFunctionRef<void(UK2Node*)> InitNodeCallback);

	/** Insert a branch node to the existing graph to test before executing the rest of the . */
	MODELVIEWVIEWMODELBLUEPRINT_API UK2Node* InsertEarlyExitBranchNode(UEdGraph* Graph, TSubclassOf<UK2Node> BranchNode);

	/** Find the main conversion function node from the given graph. */
	MODELVIEWVIEWMODELBLUEPRINT_API UK2Node* GetWrapperNode(const UEdGraph* Graph);

	/** Find the conversion function node from the given graph. */
	MODELVIEWVIEWMODELBLUEPRINT_API UEdGraphPin* FindPin(const UEdGraph* Graph, const TArrayView<const FName> PinNames);

	/** Find the conversion function node from the given graph. */
	MODELVIEWVIEWMODELBLUEPRINT_API UEdGraphPin* FindPin(const UEdGraphNode* Node, const TArrayView<const FName> PinNames);

	/** Find the conversion function node from the given graph. */
	MODELVIEWVIEWMODELBLUEPRINT_API TArray<FName> FindPinId(const UEdGraphPin* GraphPin);

	/** Return the pin used for arguments. */
	MODELVIEWVIEWMODELBLUEPRINT_API TArray<UEdGraphPin*> FindInputPins(const UK2Node* Node);
	
	/** Return the pin used as the return value. */
	MODELVIEWVIEWMODELBLUEPRINT_API UEdGraphPin* FindOutputPin(const UK2Node* Node);

	/** Add metadata to the Graph/Function. */
	MODELVIEWVIEWMODELBLUEPRINT_API void SetMetaData(UEdGraph* NewGraph, FName MetaData, FStringView Value);

	/** Mark the node a auto promote. We try to hide those node in the editor. */
	MODELVIEWVIEWMODELBLUEPRINT_API void MarkNodeAsAutoPromote(UEdGraphNode* Node);

	/** Is the node an auto promote node. */
	MODELVIEWVIEWMODELBLUEPRINT_API bool IsAutoPromoteNode(const UEdGraphNode* Node);

	/** Is the node an async node. */
	MODELVIEWVIEWMODELBLUEPRINT_API bool IsAsyncNode(const TSubclassOf<UK2Node> Node);

	UE_DEPRECATED(5.6, "This function is deprecated and will always return false.")
	MODELVIEWVIEWMODELBLUEPRINT_API bool IsNodeMarkedToKeepConnections(const UK2Node* Node);
	UE_DEPRECATED(5.6, "This function is deprecated and is not taken into consideration anymore.")
	MODELVIEWVIEWMODELBLUEPRINT_API void MarkNodeToKeepConnections(const UK2Node* Node);

} //namespace
