// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MVVMBlueprintViewCompilerInterface.h"

#include "MVVMBlueprintViewExtension.generated.h"

class UMVVMViewClass;
class UWidgetBlueprintGeneratedClass;

/**
 * An extension class to define MVVM-related properties and behaviour. When WBP compiled, this information is copied 
 * into UMVVMViewClassExtension. This class provides a hook into the MVVM compiler and partially exposes it for injecting
 * user-defined behaviour at compile-time.
 */
UCLASS(MinimalAPI)
class UMVVMBlueprintViewExtension : public UObject
{
	GENERATED_BODY()

public:
	//~ Functions to be overriden in a user-defined UMVVMViewBlueprintMyWidgetExtension class
	virtual TArray<UE::MVVM::Compiler::FBlueprintViewUserWidgetProperty> AddProperties()
	{
		return TArray<UE::MVVM::Compiler::FBlueprintViewUserWidgetProperty>();
	};

	virtual TArray<UE::MVVM::Compiler::FBlueprintViewUserWidgetWidgetProperty> AddWidgetProperties()
	{
		return TArray<UE::MVVM::Compiler::FBlueprintViewUserWidgetWidgetProperty>();
	};

	virtual void Precompile(UE::MVVM::Compiler::IMVVMBlueprintViewPrecompile* Compiler, UWidgetBlueprintGeneratedClass* Class) {}
	virtual void Compile(UE::MVVM::Compiler::IMVVMBlueprintViewCompile* Compiler, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension) {}
	virtual bool WidgetRenamed(FName OldName, FName NewName) 
	{ 
		return false; 
	}

	virtual void OnPreviewContentChanged(TSharedRef<SWidget> NewContent) {}
};