// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MVVMCompiledBindingLibraryCompiler.h"
#include "MVVMBlueprintViewExtension.h"
#include "MVVMViewBlueprintListViewBaseExtension.generated.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

class UMVVMBlueprintView;
class UMVVMViewClass;
class UUserWidget;
class UWidgetBlueprint;
class UWidgetBlueprintGeneratedClass;

namespace UE::MVVM
{
	class FMVVMListViewBaseExtensionCustomizationExtender;
}

namespace UE::MVVM::Compiler
{
	class IMVVMBlueprintViewPrecompile;
	class IMVVMBlueprintViewCompile;
}

UCLASS(MinimalAPI)
class UMVVMBlueprintViewExtension_ListViewBase : public UMVVMBlueprintViewExtension
{
	GENERATED_BODY()

public:
	//~ Begin UMVVMBlueprintViewExtension overrides
	UE_API virtual void Precompile(UE::MVVM::Compiler::IMVVMBlueprintViewPrecompile* Compiler, UWidgetBlueprintGeneratedClass* Class) override;
	UE_API virtual void Compile(UE::MVVM::Compiler::IMVVMBlueprintViewCompile* Compiler, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension) override;
	UE_API virtual bool WidgetRenamed(FName OldName, FName NewName) override;
	//~ End UMVVMBlueprintViewExtension overrides

	FGuid GetEntryViewModelId() const
	{
		return EntryViewModelId;
	}

private:
	UE_API const UMVVMBlueprintView* GetEntryWidgetBlueprintView(const UWidgetBlueprint* EntryWidgetBlueprint) const;

private:
	UPROPERTY()
	FName WidgetName;

	UPROPERTY()
	FGuid EntryViewModelId;

	UE::MVVM::FCompiledBindingLibraryCompiler::FFieldPathHandle WidgetPathHandle;

	friend UE::MVVM::FMVVMListViewBaseExtensionCustomizationExtender;
};

#undef UE_API
