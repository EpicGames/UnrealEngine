// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"
#include "Bindings/MVVMCompiledBindingLibraryCompiler.h"
#include "MVVMBlueprintViewExtension.h"
#include "Components/PanelSlot.h"
#include "MVVMViewBlueprintPanelWidgetExtension.generated.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

class UMVVMBlueprintView;
class UMVVMViewClass;
class UMVVMViewPanelWidgetClassExtension;
class UUserWidget;
class UWidgetBlueprintGeneratedClass;
class UWidgetBlueprint;

namespace UE::MVVM
{
	class FMVVMPanelWidgetExtensionCustomizationExtender;
}

namespace UE::MVVM::Compiler
{
	class IMVVMBlueprintViewPrecompile;
	class IMVVMBlueprintViewCompile;
	struct FBlueprintViewUserWidgetProperty;
}

UCLASS(MinimalAPI)
class UMVVMBlueprintViewExtension_PanelWidget : public UMVVMBlueprintViewExtension
{
	GENERATED_BODY()

public:
	//~ Begin UMVVMBlueprintViewExtension overrides
	UE_API virtual TArray<UE::MVVM::Compiler::FBlueprintViewUserWidgetProperty> AddProperties() override;
	UE_API virtual TArray<UE::MVVM::Compiler::FBlueprintViewUserWidgetWidgetProperty> AddWidgetProperties() override;
	UE_API virtual void Precompile(UE::MVVM::Compiler::IMVVMBlueprintViewPrecompile* Compiler, UWidgetBlueprintGeneratedClass* Class) override;
	UE_API virtual void Compile(UE::MVVM::Compiler::IMVVMBlueprintViewCompile* Compiler, UWidgetBlueprintGeneratedClass* Class, UMVVMViewClass* ViewExtension) override;
	UE_API virtual bool WidgetRenamed(FName OldName, FName NewName) override;
	UE_API virtual void OnPreviewContentChanged(TSharedRef<SWidget> NewContent) override;
	//~ End UMVVMBlueprintViewExtension overrides

	FGuid GetEntryViewModelId() const
	{
		return EntryViewModelId;
	}

private:
	UE_API const UMVVMBlueprintView* GetEntryWidgetBlueprintView(const UWidgetBlueprint* EntryWidgetBlueprint) const;

	static UE_API void RefreshDesignerPreviewEntries(UPanelWidget* PanelWidget, TSubclassOf<UUserWidget> EntryWidgetClass, UPanelSlot* SlotTemplate, int32 NumDesignerPreviewEntries, bool bFullRebuild);

private:
	UPROPERTY()
	FName WidgetName;

	UPROPERTY()
	FGuid EntryViewModelId;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "EntryClass", meta = (AllowPrivateAccess = "true"))
	TSubclassOf<UUserWidget> EntryWidgetClass;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Instanced, Category = "Slot", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UPanelSlot> SlotObj;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NumDesignerPreviewEntries", meta = (AllowPrivateAccess = "true", ClampMin = 0, ClampMax = 20))
	int32 NumDesignerPreviewEntries = 3;

	UPROPERTY()
	FName PanelPropertyName;

	UPROPERTY()
	TObjectPtr<UMVVMViewPanelWidgetClassExtension> ExtensionObj;

	UE::MVVM::FCompiledBindingLibraryCompiler::FFieldPathHandle WidgetPathHandle;

	friend UE::MVVM::FMVVMPanelWidgetExtensionCustomizationExtender;
};

#undef UE_API
