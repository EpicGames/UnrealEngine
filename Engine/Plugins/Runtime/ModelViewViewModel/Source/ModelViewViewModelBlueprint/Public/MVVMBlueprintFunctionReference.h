// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/MemberReference.h"
#include "K2Node.h"
#include "Templates/SubclassOf.h"
#include "MVVMBlueprintFunctionReference.generated.h"

class UBlueprint;
class UFunction;

/**
 *
 */
UENUM()
enum class EMVVMBlueprintFunctionReferenceType : uint8
{
	None,
	Function,
	Node
};

/**
 * A type that point to a function or a node that can be used to create a conversion function.
 */
USTRUCT(BlueprintType)
struct FMVVMBlueprintFunctionReference
{
	GENERATED_BODY()

public:
	FMVVMBlueprintFunctionReference() = default;
	MODELVIEWVIEWMODELBLUEPRINT_API explicit FMVVMBlueprintFunctionReference(const UBlueprint* InContext, const UFunction* InFunction);
	MODELVIEWVIEWMODELBLUEPRINT_API explicit FMVVMBlueprintFunctionReference(FMemberReference InFunctionReference);
	MODELVIEWVIEWMODELBLUEPRINT_API explicit FMVVMBlueprintFunctionReference(TSubclassOf<UK2Node> InNode);

	/** If the node is of type Function, resolves the function. Uses the blueprint, if the function is defined on the BP itself. */
	MODELVIEWVIEWMODELBLUEPRINT_API const UFunction* GetFunction(const UBlueprint* SelfContext) const;
	/** If the node is of type Function, resolves the function. Uses the blueprint, if the function is defined on the BP itself. */
	MODELVIEWVIEWMODELBLUEPRINT_API const UFunction* GetFunction(const UClass* SelfContext) const;
	/** If the node is of type Node, return the node class. */
	MODELVIEWVIEWMODELBLUEPRINT_API TSubclassOf<UK2Node> GetNode() const;

	EMVVMBlueprintFunctionReferenceType GetType() const
	{
		return Type;
	}

	/** return a string representation of the reference. */
	MODELVIEWVIEWMODELBLUEPRINT_API FString ToString() const;

	/** return a name of the reference. */
	MODELVIEWVIEWMODELBLUEPRINT_API FName GetName() const;

	/** The reference is a valid UFunction or a valid Node. */
	MODELVIEWVIEWMODELBLUEPRINT_API bool IsValid(const UBlueprint* SelfContext) const;
	/** The reference is a valid UFunction or a valid Node. */
	MODELVIEWVIEWMODELBLUEPRINT_API bool IsValid(const UClass* SelfContext) const;

	MODELVIEWVIEWMODELBLUEPRINT_API bool operator==(const FMVVMBlueprintFunctionReference& Other) const;

	friend uint32 GetTypeHash(const FMVVMBlueprintFunctionReference& Value)
	{
		uint32 Hash = GetTypeHash(static_cast<uint8>(Value.Type));
		Hash = HashCombine(Hash, GetTypeHash(Value.FunctionReference.GetMemberName()));
		Hash = HashCombine(Hash, GetTypeHash(Value.Node));
		return Hash;
	}

private:
	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	FMemberReference FunctionReference;

	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	TSubclassOf<UK2Node> Node;

	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	EMVVMBlueprintFunctionReferenceType Type = EMVVMBlueprintFunctionReferenceType::None;
};