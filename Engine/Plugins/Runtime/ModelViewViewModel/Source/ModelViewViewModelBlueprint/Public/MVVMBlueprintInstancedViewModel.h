// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "StructUtils/PropertyBag.h"
#include "Templates/SubclassOf.h"
#include "ViewModel/MVVMInstancedViewModelGeneratedClass.h"
#include "MVVMBlueprintInstancedViewModel.generated.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

/**
 *
 */
UCLASS(MinimalAPI, Abstract, Within = MVVMBlueprintView)
class UMVVMBlueprintInstancedViewModelBase : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMVVMBlueprintInstancedViewModelBase();

	UE_API void GenerateClass(bool bForceGeneration);

	UClass* GetGeneratedClass() const
	{
		return GeneratedClass;
	}

protected:
	UE_API virtual void PreloadObjectsForCompilation();
	UE_API virtual bool IsClassDirty() const;
	UE_API virtual void CleanClass();
	UE_API virtual void AddProperties();
	UE_API virtual void ConstructClass();
	UE_API virtual void SetDefaultValues();
	UE_API virtual void ClassGenerated();

protected:
	struct FInitializePropertyArgs
	{
		bool bFieldNotify = false;
		bool bReadOnly = false;
		bool bNetwork = false;
		bool bPrivate = false;
	};

	UE_API bool IsValidFieldName(const FName NewPropertyName) const;
	UE_API bool IsValidFieldName(const FName NewPropertyName, UStruct* NewOwner) const;
	UE_API FProperty* CreateProperty(const FProperty* FromProperty, UStruct* NewOwner);
	UE_API FProperty* CreateProperty(const FProperty* FromProperty, UStruct* NewOwner, FName NewPropertyName);
	UE_API void InitializeProperty(FProperty* NewProperty, FInitializePropertyArgs& Args);
	UE_API void LinkProperty(FProperty* NewProperty) const;
	UE_API void LinkProperty(FProperty* NewProperty, UStruct* NewOwner) const;
	UE_API void AddOnRepFunction(FProperty* NewProperty);
	UE_API void SafeRename(UObject* Object);
	UE_API void SetDefaultValue(const FProperty* SourceProperty, void const* SourceValuePtr, const FProperty* DestinationProperty);

public:
	/** The base object of the generated class. */
	UPROPERTY(EditAnywhere, Category = "Viewmodel", meta=(AllowedClasses = "/Script/FieldNotification.NotifyFieldValueChanged", DisallowedClasses = "/Script/UMG.Widget"))
	TSubclassOf<UObject> ParentClass;

	UPROPERTY()
	TObjectPtr<UMVVMInstancedViewModelGeneratedClass> GeneratedClass;

protected:
	//~ if it changes, the GeneratedClass needs to be removed.
	UPROPERTY()
	TSubclassOf<UMVVMInstancedViewModelGeneratedClass> GeneratedClassType;
};

/**
 *
 */
UCLASS(MinimalAPI)
class UMVVMBlueprintInstancedViewModel_PropertyBag : public UMVVMBlueprintInstancedViewModelBase
{
	GENERATED_BODY()

public:
	const UStruct* GetSourceStruct() const
	{
		return Variables.GetValue().GetScriptStruct();
}

	const uint8* GetSourceDefaults() const
	{
		return Variables.GetValue().GetMemory();
	}

protected:
	UE_API virtual bool IsClassDirty() const override;
	UE_API virtual void CleanClass() override;
	UE_API virtual void AddProperties() override;
	UE_API virtual void SetDefaultValues() override;
	UE_API virtual void ClassGenerated() override;

private:
	UPROPERTY(EditAnywhere, Category = "Viewmodel")
	FInstancedPropertyBag Variables;

	UPROPERTY()
	uint64 PropertiesHash = 0;

	TMap<const FProperty*, FProperty*> FromPropertyToCreatedProperty;

public:
#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif
};

#undef UE_API
