// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMPropertyPath.h"

#include "MVVMBlueprintPin.generated.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

class UWidgetBlueprint;
class UEdGraphNode;
class UEdGraphPin;

/**
*
*/
UENUM()
enum class EMVVMBlueprintPinStatus : uint8
{
	Valid,
	Orphaned,
};

/**
 * Pin name type to help with compare operation and moving it around.
 */
USTRUCT()
struct FMVVMBlueprintPinId
{
	GENERATED_BODY()

	FMVVMBlueprintPinId() = default;
	UE_API explicit FMVVMBlueprintPinId(const TArrayView<const FName> Names);
	UE_API explicit FMVVMBlueprintPinId(TArray<FName>&& Names);

	UE_API bool IsValid() const;

	const TArrayView<const FName> GetNames() const
	{
		return PinNames;
	}

	/** return true if the Pin is part of the Other pin. It can be a grand child. */
	UE_API bool IsChildOf(const FMVVMBlueprintPinId& Other) const;

	/** return true if the Pin is the directly child of the Other pin. It can be a child but not a grand child. */
	UE_API bool IsDirectChildOf(const FMVVMBlueprintPinId& Other) const;

	UE_API bool operator==(const FMVVMBlueprintPinId& Other) const;
	UE_API bool operator==(const TArrayView<const FName> Other) const;

	UE_API FString ToString() const;

private:
	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	TArray<FName> PinNames;
};

/**
 *
 */
USTRUCT()
struct FMVVMBlueprintPin
{
	GENERATED_BODY()

private:	
	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	FMVVMBlueprintPinId Id;

	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	FMVVMBlueprintPropertyPath Path;

	/** Default value for this pin (used if the pin has no connections), stored as a string */
	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	FString DefaultString;

	/** If the default value for this pin should be an FText, it is stored here. */
	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	FText DefaultText;

	/** If the default value for this pin should be an object, we store a pointer to it */
	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	TObjectPtr<class UObject> DefaultObject;

	/** The pin is split. */
	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	bool bSplit = false;

	/** The pin could not be set. */
	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	mutable EMVVMBlueprintPinStatus Status = EMVVMBlueprintPinStatus::Valid;

	UPROPERTY()
	FName PinName_DEPRECATED;
	
	UPROPERTY()
	FGuid PinId_DEPRECATED;

public:
	FMVVMBlueprintPin() = default;
	UE_DEPRECATED(5.4, "FMVVMBlueprintPin with a single name is deprecated. Use the TArrayView constructor instead")
	UE_API FMVVMBlueprintPin(FName PinName);
	UE_API explicit FMVVMBlueprintPin(FMVVMBlueprintPinId PinId);
	UE_API explicit FMVVMBlueprintPin(const TArrayView<const FName> PinName);

	UE_DEPRECATED(5.4, "GetName is deprecated. Use GetId().GetNames instead")
	FName GetName() const
	{
		return Id.GetNames().Num() > 0 ? Id.GetNames().Last() : FName();
	}

	const FMVVMBlueprintPinId& GetId() const
	{
		return Id;
	}

	bool IsValid() const
	{
		return Id.IsValid();
	}

	/** The pin is split into its different components. */
	bool IsSplit() const
	{
		return bSplit;
	}

	/** The pin could not be assigned to the graph pin. */
	EMVVMBlueprintPinStatus GetStatus() const
	{
		return Status;
	}

	/** Are we using the path. */
	bool UsedPathAsValue() const
	{
		return !bSplit && Path.IsValid();
	}

	/** Get the path used by this pin. */
	const FMVVMBlueprintPropertyPath& GetPath() const
	{
		return Path;
	}

	UE_API FString GetValueAsString(const UClass* SelfContext) const;

	UE_API void SetDefaultValue(UObject* Value);
	UE_API void SetDefaultValue(const FText& Value);
	UE_API void SetDefaultValue(const FString& Value);
	UE_API void SetPath(const FMVVMBlueprintPropertyPath& Value);

public:
	UE_API void PostSerialize(const FArchive& Ar);

public:
	static UE_API bool IsInputPin(const UEdGraphPin* Pin);
	static UE_API TArray<FMVVMBlueprintPin> CopyAndReturnMissingPins(UBlueprint* Blueprint, UEdGraphNode* GraphNode, const TArray<FMVVMBlueprintPin>& Pins);
	static UE_API TArray<FMVVMBlueprintPin> CreateFromNode(UBlueprint* Blueprint, UEdGraphNode* GraphNode);
	static UE_API FMVVMBlueprintPin CreateFromPin(const UBlueprint* Blueprint, const UEdGraphPin* Pin);

	UE_API void CopyTo(const UBlueprint* WidgetBlueprint, UEdGraphNode* Node) const;
	UE_API UEdGraphPin* FindGraphPin(const UEdGraph* Graph) const;

	UE_API void Reset();
};

template<>
struct TStructOpsTypeTraits<FMVVMBlueprintPin> : public TStructOpsTypeTraitsBase2<FMVVMBlueprintPin>
{
	enum
	{
		WithPostSerialize = true,
	};
};

#undef UE_API
