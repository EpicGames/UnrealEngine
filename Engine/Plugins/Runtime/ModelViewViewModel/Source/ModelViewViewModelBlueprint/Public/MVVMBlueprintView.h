// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMBlueprintViewBinding.h"
#include "MVVMBlueprintViewModelContext.h"

#include "MVVMBlueprintView.generated.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

class UMVVMWidgetBlueprintExtension_View;
class UMVVMBlueprintViewEvent;
class UMVVMBlueprintViewCondition;

class UWidget;
class UWidgetBlueprint;

namespace  UE::MVVM
{
	enum class EBindingMessageType : uint8
	{
		Info,
		Warning,
		Error
	};

	struct FBindingMessage
	{
		FText MessageText;
		EBindingMessageType MessageType;
	};
}

/**
 *
 */
UCLASS(MinimalAPI)
class UMVVMBlueprintViewSettings : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Auto initialize the view sources when the Widget is constructed.
	 * If false, the user will have to initialize the sources manually.
	 * It prevents the sources evaluating until you are ready.
	 */
	UPROPERTY(EditAnywhere, Category = "View")
	bool bInitializeSourcesOnConstruct = true;

	/**
	 * Auto initialize the view bindings when the Widget is constructed.
	 * If false, the user will have to initialize the bindings manually.
	 * It prevents bindings execution and improves performance when you know the widget won't be visible.
	 * @note All bindings are executed when the view is automatically initialized or manually initialized.
	 * @note Sources needs to be initialized before initializing the bindings.
	 * @note When Sources is manually initialized, the bindings will also be initialized if this is true.
	 */
	UPROPERTY(EditAnywhere, Category = "View", meta=(EditCondition="bInitializeSourcesOnConstruct"))
	bool bInitializeBindingsOnConstruct = true;

	/**
	 * Auto initialize the view events when the Widget is constructed.
	 * If false, the user will have to initialize the event manually.
	 */
	UPROPERTY(EditAnywhere, Category = "View")
	bool bInitializeEventsOnConstruct = true;
	
	/**
	 * Create the view even when there are no view bindings or events.
	 * If false, the view models will not be automatically available for use in blueprints if there are no bindings.
	 */
	UPROPERTY(EditAnywhere, Category = "View")
	bool bCreateViewWithoutBindings = false;
};

/**
 * 
 */
UCLASS(MinimalAPI, Within=MVVMWidgetBlueprintExtension_View)
class UMVVMBlueprintView : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMVVMBlueprintView();

public:
	UMVVMBlueprintViewSettings* GetSettings()
	{
		return Settings;
	}

	UE_API FMVVMBlueprintViewModelContext* FindViewModel(FGuid ViewModelId);
	UE_API const FMVVMBlueprintViewModelContext* FindViewModel(FGuid ViewModelId) const;
	UE_API const FMVVMBlueprintViewModelContext* FindViewModel(FName ViewModelName) const;

	UE_API void AddViewModel(const FMVVMBlueprintViewModelContext& NewContext);
	UE_API bool RemoveViewModel(FGuid ViewModelId);
	UE_API int32 RemoveViewModels(const TArrayView<FGuid> ViewModelIds);
	UE_API bool RenameViewModel(FName OldViewModelName, FName NewViewModelName);
	UE_API bool ReparentViewModel(FGuid ViewModelId, const UClass* ViewModelClass);

	const TArrayView<const FMVVMBlueprintViewModelContext> GetViewModels() const
	{
		return AvailableViewModels; 
	}

	UE_API const FMVVMBlueprintViewBinding* FindBinding(const UWidget* Widget, const FProperty* Property) const;
	UE_API FMVVMBlueprintViewBinding* FindBinding(const UWidget* Widget, const FProperty* Property);

	UE_API void RemoveBinding(const FMVVMBlueprintViewBinding* Binding);
	UE_API const FMVVMBlueprintViewBinding* DuplicateBinding(const FMVVMBlueprintViewBinding* Binding);

	UE_API void RemoveBindingAt(int32 Index);

	UE_API FMVVMBlueprintViewBinding& AddDefaultBinding();

	int32 GetNumBindings() const
	{
		return Bindings.Num();
	}

	UE_API FMVVMBlueprintViewBinding* GetBindingAt(int32 Index);
	UE_API const FMVVMBlueprintViewBinding* GetBindingAt(int32 Index) const;
	UE_API FMVVMBlueprintViewBinding* GetBinding(FGuid Id);
	UE_API const FMVVMBlueprintViewBinding* GetBinding(FGuid Id) const;

	TArrayView<FMVVMBlueprintViewBinding> GetBindings()
	{
		return Bindings;
	}

	const TArrayView<const FMVVMBlueprintViewBinding> GetBindings() const
	{
		return Bindings;
	}

	UE_API UMVVMBlueprintViewEvent* AddDefaultEvent();
	UE_API void AddEvent(UMVVMBlueprintViewEvent* Event);
	UE_API void RemoveEvent(UMVVMBlueprintViewEvent* Event);
	UE_API void ReplaceEvent(UMVVMBlueprintViewEvent* OldEvent, UMVVMBlueprintViewEvent* NewEvent);
	UE_API UMVVMBlueprintViewEvent* DuplicateEvent(UMVVMBlueprintViewEvent* Event);

	TArrayView<TObjectPtr<UMVVMBlueprintViewEvent>> GetEvents()
	{
		return Events;
	}

	const TArrayView<const TObjectPtr<UMVVMBlueprintViewEvent>> GetEvents() const
	{
		return Events;
	}

	UE_API UMVVMBlueprintViewCondition* AddDefaultCondition();
	UE_API void AddCondition(UMVVMBlueprintViewCondition* Condition);
	UE_API void RemoveCondition(UMVVMBlueprintViewCondition* Condition);
	UE_API void ReplaceCondition(UMVVMBlueprintViewCondition* OldCondition, UMVVMBlueprintViewCondition* NewCondition);
	UE_API UMVVMBlueprintViewCondition* DuplicateCondition(UMVVMBlueprintViewCondition* Condition);

	TArrayView<TObjectPtr<UMVVMBlueprintViewCondition>> GetConditions()
	{
		return Conditions;
	}

	const TArrayView<const TObjectPtr<UMVVMBlueprintViewCondition>> GetConditions() const
	{
		return Conditions;
	}

	bool HasAnyTypeOfBinding()
	{
		return !Bindings.IsEmpty() || !Events.IsEmpty() || !Conditions.IsEmpty();
	}

	UE_API TArray<FText> GetBindingMessages(FGuid Id, UE::MVVM::EBindingMessageType InMessageType) const;
	UE_API bool HasBindingMessage(FGuid Id) const;
	UE_API bool HasBindingMessage(FGuid Id, UE::MVVM::EBindingMessageType InMessageType) const;
	UE_API void AddMessageToBinding(FGuid Id, UE::MVVM::FBindingMessage MessageToAdd);
	UE_API void ResetBindingMessages();

	FGuid GetCompiledBindingLibraryId() const
	{
		return CompiledBindingLibraryId;
	}

#if WITH_EDITOR
	UE_API virtual void PostLoad() override;
	UE_API virtual void PreSave(FObjectPreSaveContext Context) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChainEvent) override;
	UE_API virtual void PostEditUndo() override;

	UE_API void AddAssetTags(FAssetRegistryTagsContext Context) const;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	UE_API void AddAssetTags(TArray<FAssetRegistryTag>& OutTags) const;
	UE_API void OnFieldRenamed(UClass* FieldOwnerClass, FName OldObjectName, FName NewObjectName);
#endif

	UE_API virtual void Serialize(FArchive& Ar) override;

	DECLARE_EVENT(UMVVMBlueprintView, FOnBindingsUpdated);
	FOnBindingsUpdated OnBindingsUpdated;

	DECLARE_EVENT(UMVVMBlueprintView, FOnBindingsAdded);
	FOnBindingsAdded OnBindingsAdded;

	DECLARE_EVENT(UMVVMBlueprintView, FOnEventsUpdated);
	FOnEventsUpdated OnEventsUpdated;

	DECLARE_EVENT(UMVVMBlueprintView, FOnConditionsUpdated);
	FOnConditionsUpdated OnConditionsUpdated;

	DECLARE_EVENT_OneParam(UMVVMBlueprintView, FOnEventParametersRegenerate, UMVVMBlueprintViewEvent*);
	FOnEventParametersRegenerate OnEventParametersRegenerate;

	DECLARE_EVENT_OneParam(UMVVMBlueprintView, FOnConditionParametersRegenerate, UMVVMBlueprintViewCondition*);
	FOnConditionParametersRegenerate OnConditionParametersRegenerate;

	DECLARE_EVENT(UMVVMBlueprintView, FOnViewModelsUpdated);
	FOnViewModelsUpdated OnViewModelsUpdated;

	// Use during compilation to clean the automatically generated graph.
	UPROPERTY(Transient, NonTransactional)
	TArray<TObjectPtr<UEdGraph>> TemporaryGraph;

	// Use during compilation to clean the automatically generated graph.
	UPROPERTY(Transient)
	TArray<FName> TemporaryGraphNames;

private:
	UPROPERTY(Instanced)
	TObjectPtr<UMVVMBlueprintViewSettings> Settings;

	UPROPERTY(EditAnywhere, Category = "Viewmodel")
	TArray<FMVVMBlueprintViewBinding> Bindings;
	
	UPROPERTY(Instanced, EditAnywhere, Category = "Viewmodel")
	TArray<TObjectPtr<UMVVMBlueprintViewEvent>> Events;

	UPROPERTY(Instanced, EditAnywhere, Category = "Viewmodel")
	TArray<TObjectPtr<UMVVMBlueprintViewCondition>> Conditions;

	UPROPERTY(EditAnywhere, Category = "Viewmodel")
	TArray<FMVVMBlueprintViewModelContext> AvailableViewModels;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel", meta = (IgnoreForMemberInitializationTest))
	FGuid CompiledBindingLibraryId;

	TMap<FGuid, TArray<UE::MVVM::FBindingMessage>> BindingMessages;

	bool bIsContextSensitive;
};

#undef UE_API
