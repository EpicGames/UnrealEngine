// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMPropertyPath.h"
#include "Types/MVVMBindingMode.h"
#include "Types/MVVMExecutionMode.h"

#include "MVVMBlueprintViewBinding.generated.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

class UWidgetBlueprint;
class UMVVMBlueprintView;
class UMVVMBlueprintViewConversionFunction;

/**
*
*/
USTRUCT(BlueprintType)
struct FMVVMBlueprintViewConversionPath
{
	GENERATED_BODY()

	UE_DEPRECATED(5.3, "DestinationToSourceFunction was moved to MVVMBlueprintViewConversionFunction.")
	/**
	 * The Conversion function when converting the value from the destination to the source.
	 * @note if a wrapper is needed, this is the function that is wrapped.
	 */
	UPROPERTY()
	FMemberReference DestinationToSourceFunction_DEPRECATED;
	
	UE_DEPRECATED(5.3, "DestinationToSourceWrapper was moved to MVVMBlueprintViewConversionFunction.")
	/**
	 * The name of the graph that contains the conversion function when converting the value from the destination to the source.
	 * Valid when the function needs a wrapper and when the graph is generated on the fly.
	 */
	UPROPERTY()
	FName DestinationToSourceWrapper_DEPRECATED;

	UE_DEPRECATED(5.3, "SourceToDestinationFunction was moved to MVVMBlueprintViewConversionFunction.")
	/**
	 * The Conversion function when converting the value from the source to the destination.
	 * @note if a wrapper is needed, this is the function that is wrapped.
	 */
	UPROPERTY()
	FMemberReference SourceToDestinationFunction_DEPRECATED;

	UE_DEPRECATED(5.3, "SourceToDestinationWrapper was moved to MVVMBlueprintViewConversionFunction.")
	/**
	 * The name of the graph that contains the conversion function when converting the value from the source to the destination.
	 * When the function needs a wrapper. Valid also when the graph is generated on the fly.
	 */
	UPROPERTY()
	FName SourceToDestinationWrapper_DEPRECATED;

	/**
	 * The graph that contains the conversion function when converting the value from the destination to the source.
	 * When the function doesn't need a wrapper.
	 */
	UPROPERTY(VisibleAnywhere, NoClear, Category = "Viewmodel", Instanced)
	TObjectPtr<UMVVMBlueprintViewConversionFunction> DestinationToSourceConversion;

	/**
	 * The graph that contains the conversion function when converting the value from the source to the destination.
	 * When the function doesn't need a wrapper.
	 */
	UPROPERTY(VisibleAnywhere, NoClear, Category = "Viewmodel", Instanced)
	TObjectPtr<UMVVMBlueprintViewConversionFunction> SourceToDestinationConversion;

public:
	UMVVMBlueprintViewConversionFunction* GetConversionFunction(bool bSourceToDestination) const
	{
		return bSourceToDestination ? SourceToDestinationConversion : DestinationToSourceConversion;
	}

	UE_API void GenerateWrapper(UBlueprint* Blueprint);
	UE_API void SavePinValues(UBlueprint* Blueprint);
	UE_API void DeprecateViewConversionFunction(UBlueprint* Blueprint, FMVVMBlueprintViewBinding& Owner);

public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMVVMBlueprintViewConversionPath() = default;
	FMVVMBlueprintViewConversionPath(const FMVVMBlueprintViewConversionPath&) = default;
	FMVVMBlueprintViewConversionPath& operator=(const FMVVMBlueprintViewConversionPath&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/**
*
*/
USTRUCT(BlueprintType)
struct FMVVMBlueprintViewBinding
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FMVVMBlueprintPropertyPath SourcePath;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FMVVMBlueprintPropertyPath DestinationPath;

	/** */
	UPROPERTY(EditAnywhere, Category = "Viewmodel")
	EMVVMBindingMode BindingType = EMVVMBindingMode::OneWayToDestination;

	UPROPERTY()
	bool bOverrideExecutionMode = false;

	UPROPERTY(EditAnywhere, Category = "Viewmodel", meta=(EditCondition="bOverrideExecutionMode"))
	EMVVMExecutionMode OverrideExecutionMode = EMVVMExecutionMode::Immediate;

	/** */
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FMVVMBlueprintViewConversionPath Conversion;

	/** The unique ID of this binding. */
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FGuid BindingId;

	/** Whether the binding is enabled or disabled by default. The instance may enable the binding at runtime. */
	UPROPERTY(EditAnywhere, Category = "Viewmodel")
	bool bEnabled = true;

	/** The binding is visible in the editor, but is not compiled and cannot be used at runtime. */
	UPROPERTY(EditAnywhere, Category = "Viewmodel")
	bool bCompile = true;

	/**
	 * Get an internal name. For use in the UI, use GetDisplayNameString()
	 */
	UE_API FName GetFName() const;

	/** 
	 * Get a string that identifies this binding. 
	 * This is of the form: Widget.Property <- ViewModel.Property
	 */
	UE_API FString GetDisplayNameString(const UWidgetBlueprint* WidgetBlueprint, bool bUseDisplayName = false) const;

	/**
	 * Get a string that identifies this binding and is specifically formatted for search. 
	 * This includes the display name and variable name of all fields and widgets, as well as all function keywords.
	 * For use in the UI, use GetDisplayNameString()
	 */
	UE_API FString GetSearchableString(const UWidgetBlueprint* WidgetBlueprint) const;
};

#undef UE_API
