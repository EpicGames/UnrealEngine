// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Bindings/MVVMCompiledBindingLibraryCompiler.h"
#include "Extensions/MVVMViewClassExtension.h"
#include "View/MVVMViewClass.h"

class UMVVMBlueprintView;
class UWidget;
class UWidgetBlueprintGeneratedClass;

namespace UE::MVVM::Compiler
{

struct FCompilerBindingHandle
{
public:
	explicit FCompilerBindingHandle()
	: Id(0)
	{
	}

	static FCompilerBindingHandle MakeHandle()
	{
		FCompilerBindingHandle Handle;
		++IdGenerator;
		Handle.Id = IdGenerator;
		return Handle;
	}

	bool IsValid() const
	{
		return Id != 0;
	}

	bool operator==(const FCompilerBindingHandle& Other) const
	{
		return Id == Other.Id;
	}

	bool operator!=(const FCompilerBindingHandle& Other) const
	{
		return Id != Other.Id;
	}

	friend uint32 GetTypeHash(const FCompilerBindingHandle& Handle)
	{
		return ::GetTypeHash(Handle.Id);
	}

private:
	static int32 IdGenerator;
	int32 Id;
};

enum class EMessageType
{
	Info = 0, 
	Warning = 1, 
	Error = 2
};

/**
 * Data to be used by the compiler to create a property for a widget in the widget tree
 */
struct FBlueprintViewUserWidgetWidgetProperty
{
	FName WidgetName;
};

/**
 * Data to be used by the compiler to create new user widget properties
 */
struct FBlueprintViewUserWidgetProperty
{
	UClass* AuthoritativeClass = nullptr;
	FName Name;
	FText DisplayName;
	FString CategoryName;
	FGuid Guid;
	bool bExposeOnSpawn = false;
	bool bPrivate = false;
	bool bReadOnly = false;

	const FProperty* Property = nullptr;
};

/**
 * Exposed interface of MVVMViewBlueprintCompiler to be used in view extensions (MVVMViewBlueprintViewExtension) at precompile step
 */
class IMVVMBlueprintViewPrecompile
{
public:
	struct FObjectFieldPathArgs
	{
		FObjectFieldPathArgs(UWidgetBlueprintGeneratedClass* InClass, const FString& InObjectPath, UClass* InExpectedType)
			: Class(InClass)
			, ObjectPath(InObjectPath)
			, ExpectedType(InExpectedType)
		{
		}

		UWidgetBlueprintGeneratedClass* Class;
		FString ObjectPath;
		UClass* ExpectedType;
	};

	virtual const UMVVMBlueprintView* GetBlueprintView() const = 0;

	/* Returns a map of widget name to the widget pointer for all widgets in the blueprint. */
	virtual const TMap<FName, UWidget*>& GetWidgetNameToWidgetPointerMap() const = 0;

	/* Returns all the bindings via handles */
	virtual TArray<Compiler::FCompilerBindingHandle> GetAllBindings() const = 0;

	/* Returns the field path for all the read fields of a binding (passed by handle) */
	virtual TArray<TArray<UE::MVVM::FMVVMConstFieldVariant>> GetBindingReadFields(Compiler::FCompilerBindingHandle BindingHandle) = 0;

	/* Returns the field path for the write field of a binding (passed by handle) */
	virtual TArray<UE::MVVM::FMVVMConstFieldVariant> GetBindingWriteFields(Compiler::FCompilerBindingHandle BindingHandle) = 0;

	/* Returns the source of a binding (passed by handle) if it's valid */
	virtual const FProperty* GetBindingSourceProperty(Compiler::FCompilerBindingHandle BindingHandle) = 0;

	/**
	 * Add a new object property to the compiler.
	 * Store the returned the handle of this function to access the compiled path during Compile step by calling GetFieldPath.
	 */
	virtual TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> AddObjectFieldPath(const FObjectFieldPathArgs& Args) = 0;
	
	/**
	 * Adds a new field to the compiler.
	 * Store the returned the handle of this function to access the compiled path during Compile step by calling GetFieldPath.
	 */
	virtual TValueOrError<FCompiledBindingLibraryCompiler::FFieldPathHandle, FText> AddFieldPath(TArrayView<const FMVVMConstFieldVariant> InFieldPath, bool bInRead) = 0;

	/*
	 * Shows a message for the given binding.
	 * If this is an error message, it will cause the blueprint compilation to fail. Next steps will still be executed.
	 */
	virtual void AddMessageForBinding(Compiler::FCompilerBindingHandle BindingHandle, const FText& MessageText, EMessageType MessageType) const = 0;

	/*
	 * Shows a message in the compile logs.
	 * If this is an error message, it will cause the blueprint compilation to fail. Next steps will still be executed.
	 */
	virtual void AddMessage(const FText& MessageText, EMessageType MessageType) = 0;

	/*
	 * Lets the compiler know that the precompile step for this extension failed.
	 * Calling this will cause the blueprint compilation to fail and next steps won't be executed.
	 */
	virtual void MarkPrecompileStepInvalid() = 0;
};

/**
 * Exposed interface of MVVMViewBlueprintCompiler to be used in view extensions (MVVMViewBlueprintViewExtension) at compile step
 */
class IMVVMBlueprintViewCompile
{
public:
	virtual const UMVVMBlueprintView* GetBlueprintView() const = 0;

	/* Returns a map of widget name to the widget pointer for all widgets in the blueprint. */
	virtual const TMap<FName, UWidget*>& GetWidgetNameToWidgetPointerMap() const = 0;

	/**
	 * Get the compiled field path of the passed handle. 
	 * The field path should have been added in the precompile step using AddObjectFieldPath or AddFieldPath and the returned handle should be stored.
	 */
	virtual TValueOrError<FMVVMVCompiledFieldPath, void> GetFieldPath(FCompiledBindingLibraryCompiler::FFieldPathHandle FieldPath) = 0;

	/*
	 * Shows a message for the given binding.
	 * If this is an error message, it will cause the blueprint compilation to fail. Next steps will still be executed.
	 */
	virtual void AddMessageForBinding(Compiler::FCompilerBindingHandle BindingHandle, const FText& MessageText, EMessageType MessageType) const = 0;

	/*
	 * Shows a message in the compile logs.
	 * If this is an error message, it will cause the blueprint compilation to fail. Next steps will still be executed.
	 */
	virtual void AddMessage(const FText& MessageText, EMessageType MessageType) = 0;

	/*
	 * Lets the compiler know that the compile step for this extension failed.
	 * Calling this will cause the blueprint compilation to fail and next steps won't be executed.
	 */
	virtual void MarkCompileStepInvalid() = 0;

	/* Creates an extension object of the given class and adds it to the view class to be invoked at runtime. */
	virtual UMVVMViewClassExtension* CreateViewClassExtension(TSubclassOf<UMVVMViewClassExtension> ExtensionClass) = 0;
};
}