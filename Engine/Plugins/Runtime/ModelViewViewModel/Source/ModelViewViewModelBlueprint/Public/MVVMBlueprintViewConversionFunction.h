// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMBlueprintFunctionReference.h"
#include "MVVMBlueprintPin.h"
#include "MVVMBlueprintView.h"
#include "Engine/MemberReference.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MVVMBlueprintViewConversionFunction.generated.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

class UK2Node;
class UEdGraphPin;
class UEdGraph;

struct FEdGraphEditAction;

/**
 * A conversion function converts between the source and destiation of a binding.
 * 
 * Internally that function may be using native C++, K2Nodes, UFunctions, Events, etc.
 */
UCLASS(MinimalAPI)
class UMVVMBlueprintViewConversionFunction : public UObject
{
	GENERATED_BODY()

public:
	static UE_API bool IsValidConversionFunction(const UBlueprint* WidgetBlueprint, const UFunction* Function);
	static UE_API bool IsValidConversionNode(const UBlueprint* WidgetBlueprint, const TSubclassOf<UK2Node> Function);

public:
	/** @return the conversion function uses at runtime. The wrapper function if complex or GetFunction is simple. */
	UE_API const UFunction* GetCompiledFunction(const UClass* SelfContext) const;

	/** @return the conversion function uses at runtime. The wrapper function if complex or GetFunction is simple. */
	UE_API FName GetCompiledFunctionName(const UClass* SelfContext) const;

	/** @return the conversion function. */
	UE_DEPRECATED(5.4, "GetConversionFunction that returns a variant is deprecated.")
	UE_API TVariant<const UFunction*, TSubclassOf<UK2Node>> GetConversionFunction(const UBlueprint* SelfContext) const;

	/** @return the conversion function. */
	UE_API FMVVMBlueprintFunctionReference GetConversionFunction() const;

	/** Set the function. Generate a Graph. */
	UE_API void Initialize(UBlueprint* SelfContext, FName GraphName, FMVVMBlueprintFunctionReference Function);

	/** Set the function. Generate a Graph. */
	UE_API void InitializeFromFunction(UBlueprint* SelfContext, FName GraphName, const UFunction* Function);

	// For deprecation
	UE_API void Deprecation_InitializeFromWrapperGraph(UBlueprint* SelfContext, UEdGraph* Graph);

	// For deprecation
	UE_API void Deprecation_InitializeFromMemberReference(UBlueprint* SelfContext, FName GraphName, FMemberReference MemberReference, const FMVVMBlueprintPropertyPath& Source);

	// For deprecation
	UE_API void Deprecation_SetWrapperGraphName(UBlueprint* Context, FName GraphName, const FMVVMBlueprintPropertyPath& Source);

	/**
	 * The conversion is valid.
	 * The function was valid when created but may not be anymore.
	 * It doesn't check if the source and destination are valid.
	 */
	UE_API bool IsValid(const UBlueprint* SelfContext) const;

	/** The function has more than one argument and requires a wrapper or it uses a FunctionNode. */
	UE_API bool NeedsWrapperGraph(const UBlueprint* SelfContext) const;

	/** The wrapper Graph is generated on load/compile and is not saved. */
	UE_API bool IsWrapperGraphTransient() const;

	/** True if the graph is going to be used for an ubergraph page. */
	UE_API bool IsUbergraphPage() const;

	const FMVVMBlueprintPropertyPath& GetDestinationPath() const
	{
		return DestinationPath;
	}
	UE_API void SetDestinationPath(FMVVMBlueprintPropertyPath DestinationPath);

	/** Return the wrapper graph, if it exists. */
	UEdGraph* GetWrapperGraph() const
	{
		return CachedWrapperGraph;
	}

	FName GetWrapperGraphName() const
	{
		return GraphName;
	}

	/** Return the wrapper node, if it exists. */
	UK2Node* GetWrapperNode() const
	{
		return CachedWrapperNode;
	}

	/** Return latent UUID Node. */
	UEdGraphNode* GetLatentNodeUUID() const
	{
		return LatentEventNodeUUID;
	}

	/**
	 * If needed, create the graph and all the nodes for that graph when compiling.
	 * Returns the existing one, if one was created from GetOrCreateWrapperGraph.
	 */
	UE_API UEdGraph* GetOrCreateIntermediateWrapperGraph(FKismetCompilerContext& Context);

	/** If needed, create the graph and all the nodes for that graph. */
	UE_API UEdGraph* GetOrCreateWrapperGraph(UBlueprint* Blueprint);

	UE_API void RecreateWrapperGraph(UBlueprint* Blueprint);

	/**
	 * The conversion function is going to be removed from the Blueprint.
	 * Do any cleanup that is needed.
	 */
	UE_API void RemoveWrapperGraph(UBlueprint* Blueprint);

	/**
	 * Returns the pin from the graph.
	 * Create the graph and all the nodes for that graph if the graph doesn't exist and it's needed.
	 */
	UE_API UEdGraphPin* GetOrCreateGraphPin(UBlueprint* Blueprint, const FMVVMBlueprintPinId& PinId);

	const TArrayView<const FMVVMBlueprintPin> GetPins() const
	{
		return SavedPins;
	}

	/** */
	UE_API void SetGraphPin(UBlueprint* Blueprint, const FMVVMBlueprintPinId& PinId, const FMVVMBlueprintPropertyPath& Value);

	/** Generates SavedPins from the wrapper graph, if it exists. */
	UE_API void SavePinValues(UBlueprint* Blueprint);
	/** Keep the orphaned pins. Add the missing pins. */
	UE_API void UpdatePinValues(UBlueprint* Blueprint);
	/** Keep the orphaned pins. Add the missing pins. */
	UE_API bool HasOrphanedPin() const;

	FSimpleMulticastDelegate OnWrapperGraphModified;

	UE_API virtual void PostLoad() override;

private:
	UE_API void HandleGraphChanged(const FEdGraphEditAction& Action, TWeakObjectPtr<UBlueprint> Context);
	UE_API void HandleUserDefinedPinRenamed(UK2Node* InNode, FName OldPinName, FName NewPinName, TWeakObjectPtr<UBlueprint> WeakBlueprint);
	UE_API void SetCachedWrapperGraph(UBlueprint* Blueprint, UEdGraph* CachedGraph, UK2Node* CachedNode);
	UE_API UEdGraph* CreateWrapperGraphInternal(FKismetCompilerContext& Context);
	UE_API UEdGraph* CreateWrapperGraphInternal(UBlueprint* Blueprint);
	UE_API bool NeedsWrapperGraphInternal(const UClass* SkeletalSelfContext) const;
	UE_API void LoadPinValuesInternal(UBlueprint* Blueprint);
	UE_API FName GenerateUniqueGraphName() const;
	UE_API void CreateWrapperGraphName();
	UE_API void Reset();

	UE_API bool IsConversionFunctionAsyncNode();

private:

	/**
	 * Destination of the binding, currently only saved when the conversion function uses 
	 * async nodes. Async graphs will handle value setting internally rather than using the MVVM subsystem.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FMVVMBlueprintPropertyPath DestinationPath;

	/**
	 * Conversion reference. It can be simple, complex or a K2Node.
	 * @note The conversion is complex
	 */
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FMVVMBlueprintFunctionReference ConversionFunction;

	/** Name of the generated graph if a wrapper is needed. */
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FName GraphName;

	/**
	 * The pin that are modified and we saved data.
	 * The data may not be modified. We used the default value of the K2Node in that case.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	TArray<FMVVMBlueprintPin> SavedPins;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	bool bWrapperGraphTransient = true;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	bool bIsUbergraphPage = false;

	UPROPERTY(Transient, DuplicateTransient)
	mutable TObjectPtr<UEdGraph> CachedWrapperGraph;
	
	UPROPERTY(Transient, DuplicateTransient)
	mutable TObjectPtr<UK2Node> CachedWrapperNode;

	/** 
	 * Events require a node UUID for the latent manager to handle lantents with 
	 */
	UPROPERTY(Transient)
	TObjectPtr<UEdGraphNode> LatentEventNodeUUID;

	FDelegateHandle OnGraphChangedHandle;
	FDelegateHandle OnUserDefinedPinRenamedHandle;
	bool bLoadingPins = false;

	UPROPERTY()
	FMemberReference FunctionReference_DEPRECATED;
	UPROPERTY()
	TSubclassOf<UK2Node> FunctionNode_DEPRECATED;
};

#undef UE_API
