// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMPropertyPath.h"
#include "MVVMBlueprintPin.h"
#include "View/MVVMViewTypes.h"

#include "MVVMBlueprintViewEvent.generated.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

struct FEdGraphEditAction;
class UEdGraph;
class UK2Node;
class UMVVMK2Node_AreSourcesValidForEvent;
class UWidgetBlueprint;

/**
 * Binding for an event that MVVM will listen too. Does not imply 
 * the MVVM graph itself will use events.
 * 
 * Ex: UButton::OnClick 
 */
UCLASS(MinimalAPI, Within = MVVMBlueprintView)
class UMVVMBlueprintViewEvent : public UObject
{
	GENERATED_BODY()

public:
	enum class EMessageType : uint8
	{
		Info,
		Warning,
		Error
	};

	struct FMessage
	{
		FText MessageText;
		EMessageType MessageType;
	};

	/** @return true if the property path contains a multicast delegate property. */
	static UE_API bool Supports(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& PropertyPath);

public:
	/** Whether the event is enabled or disabled by default. The instance may enable the event at runtime. */
	UPROPERTY(EditAnywhere, Category = "Viewmodel")
	bool bEnabled = true;

	/** The event is visible in the editor, but is not compiled and cannot be used at runtime. */
	UPROPERTY(EditAnywhere, Category = "Viewmodel")
	bool bCompile = true;

public:
	const FMVVMBlueprintPropertyPath& GetEventPath() const
	{
		return EventPath;
	}
	UE_API void SetEventPath(FMVVMBlueprintPropertyPath EventPath);

	const FMVVMBlueprintPropertyPath& GetDestinationPath() const
	{
		return DestinationPath;
	}
	UE_API void SetDestinationPath(FMVVMBlueprintPropertyPath DestinationPath);

public:
	UEdGraph* GetWrapperGraph() const
	{
		return CachedWrapperGraph;
	}

	FName GetWrapperGraphName() const
	{
		return GraphName;
	}

	enum ERemoveWrapperGraphParam
	{
		RemoveConversionFunctionCurrentValues, // when removing or changing the conversion function, we want to remove all the conversion function parameters
		LeaveConversionFunctionCurrentValues // when we remove the wrapper graph because the event path has changed, we want to keep the conversion function parameters
	};
	UE_API void RemoveWrapperGraph(ERemoveWrapperGraphParam ActionForCurrentValues = RemoveConversionFunctionCurrentValues);

	UK2Node* GetWrapperNode() const
	{
		return CachedWrapperNode;
	}

	UE_API UEdGraph* GetOrCreateWrapperGraph();

	UE_API void RecreateWrapperGraph();

public:
	TArrayView<const FMVVMBlueprintPin> GetPins() const
	{
		return SavedPins;
	}

	/** Generates SavedPins from the wrapper graph, if it exists. */
	UE_API void SavePinValues();
	/** Keep the orphaned pins. Add the missing pins. */
	UE_API void UpdatePinValues();
	/** Keep the orphaned pins. Add the missing pins. */
	UE_API bool HasOrphanedPin() const;
	/** Event sources are tested at runtime to check if they are valid. */
	UE_API void UpdateEventKey(FMVVMViewClass_EventKey EventKey);

	UE_API UEdGraphPin* GetOrCreateGraphPin(const FMVVMBlueprintPinId& Pin);

	UE_API FMVVMBlueprintPropertyPath GetPinPath(const FMVVMBlueprintPinId& Pin) const;
	UE_API void SetPinPath(const FMVVMBlueprintPinId& Pin, const FMVVMBlueprintPropertyPath& Path);
	// To set a pin when loading the asset (no graph generation)
	UE_API void SetPinPathNoGraphGeneration(const FMVVMBlueprintPinId& Pin, const FMVVMBlueprintPropertyPath& Path);

	FSimpleMulticastDelegate OnWrapperGraphModified;

public:
	UE_API TArray<FText> GetCompilationMessages(EMessageType InMessageType) const;
	UE_API bool HasCompilationMessage(EMessageType InMessageType) const;
	UE_API void AddCompilationToBinding(FMessage MessageToAdd) const;
	UE_API void ResetCompilationMessages();

public:
	/** 
	 * Get a string that identifies this event. 
	 */
	UE_API FText GetDisplayName(bool bUseDisplayName) const;

	/**
	 * Get a string that identifies this event and is specifically formatted for search. 
	 * This includes the display name and variable name of all fields and widgets, as well as all function keywords.
	 * For use in the UI, use GetDisplayNameString()
	 */
	UE_API FString GetSearchableString() const;

public:
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChainEvent) override;

private:
	static UE_API const UFunction* GetEventSignature(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& PropertyPath);
	UE_API const UFunction* GetEventSignature() const;
	UE_API void HandleGraphChanged(const FEdGraphEditAction& Action);
	UE_API void HandleUserDefinedPinRenamed(UK2Node* InNode, FName OldPinName, FName NewPinName);
	UE_API UWidgetBlueprint* GetWidgetBlueprintInternal() const;
	UE_API void SetCachedWrapperGraphInternal(UEdGraph* Graph, UK2Node* Node, UMVVMK2Node_AreSourcesValidForEvent* SourceNode);
	UE_API UEdGraph* CreateWrapperGraphInternal();
	UE_API void LoadPinValuesInternal();
	UE_API void UpdateEventKeyInternal();

private:
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FMVVMBlueprintPropertyPath EventPath;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FMVVMBlueprintPropertyPath DestinationPath;

	/**
	 * The pin that are modified and we saved data.
	 * The data may not be modified. We used the default value of the K2Node in that case.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	TArray<FMVVMBlueprintPin> SavedPins;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FName GraphName;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FMVVMViewClass_EventKey EventKey;

	mutable TArray<FMessage> Messages;
	bool bLoadingPins = false;

	UPROPERTY(Transient, DuplicateTransient)
	mutable TObjectPtr<UEdGraph> CachedWrapperGraph;

	UPROPERTY(Transient, DuplicateTransient)
	mutable TObjectPtr<UK2Node> CachedWrapperNode;

	UPROPERTY(Transient, DuplicateTransient)
	mutable TObjectPtr<UMVVMK2Node_AreSourcesValidForEvent> CachedSourceValidNode;

	FDelegateHandle OnGraphChangedHandle;
	FDelegateHandle OnUserDefinedPinRenamedHandle;

	bool bNeedsToRegenerateChildren;
};

#undef UE_API
