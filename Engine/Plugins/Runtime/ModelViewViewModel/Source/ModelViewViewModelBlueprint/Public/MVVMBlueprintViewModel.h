// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MVVMBlueprintViewModel.generated.h"


UCLASS(MinimalAPI, DisplayName="ViewModel Blueprint")
class UMVVMBlueprintViewModel : public UObject
{
	GENERATED_BODY()

public:
};
