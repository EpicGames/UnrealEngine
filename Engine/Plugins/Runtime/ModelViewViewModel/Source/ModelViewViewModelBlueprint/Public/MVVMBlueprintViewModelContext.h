// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "MVVMViewModelBase.h"

#include "MVVMBlueprintViewModelContext.generated.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

class UMVVMBlueprintInstancedViewModelBase;
class UMVVMViewModelContextResolver;

/**
 *
 */
UENUM()
enum class EMVVMBlueprintViewModelContextCreationType : uint8
{
	// The viewmodel will be assigned later.
	Manual,
	// A new instance of the viewmodel will be created when the widget is created.
	CreateInstance,
	// The viewmodel exists and is added to the MVVMSubsystem. It will be fetched there.
	GlobalViewModelCollection,
	// The viewmodel will be fetched by evaluating a function or a property path.
	PropertyPath,
	// The viewmodel will be fetched by evaluating the resolver object.
	Resolver,
	// The viewmodel will be fetched from any local context provider, can also be set manually
	Context,
};

namespace UE::MVVM
{
#if WITH_EDITOR
	[[nodiscard]] MODELVIEWVIEWMODELBLUEPRINT_API TArray<EMVVMBlueprintViewModelContextCreationType> GetAllowedContextCreationType(const UClass* Class);
#endif
}

/**
 *
 */
USTRUCT()
struct FMVVMBlueprintViewModelContext
{
	GENERATED_BODY()

public:
	FMVVMBlueprintViewModelContext() = default;
	UE_API FMVVMBlueprintViewModelContext(const UClass* InClass, FName InViewModelName);

	FGuid GetViewModelId() const
	{
		return ViewModelContextId;
	}

	FName GetViewModelName() const
	{
		return ViewModelName;
	}

	UE_API FText GetDisplayName() const;

	UClass* GetViewModelClass() const
	{
		return NotifyFieldValueClass;
	}

	void PostSerialize(const FArchive& Ar)
	{
		if (Ar.IsLoading())
		{
			if (ViewModelName.IsNone())
			{
				ViewModelName = *OverrideDisplayName_DEPRECATED.ToString();
			}
			if (ViewModelName.IsNone() )
			{
				ViewModelName = *ViewModelContextId.ToString();
			}
			if (ViewModelClass_DEPRECATED.Get())
			{
				NotifyFieldValueClass = ViewModelClass_DEPRECATED.Get();
			}
			if (!bCreateSetterFunction_Deprecation)
			{
				bCreateSetterFunction_Deprecation = true;
				if (CreationType == EMVVMBlueprintViewModelContextCreationType::Manual)
				{
					bOptional = true;
					bCreateSetterFunction = true;
				}
			}
		}
	}

	bool IsValid() const
	{
		return NotifyFieldValueClass != nullptr;
	}

	bool CanRename() const
	{
		return bCanRename && !bUseAsInterface;
	}

#if WITH_EDITOR
	[[nodiscard]] UE_API TObjectPtr<UMVVMViewModelContextResolver> CreateDefaultResolver(UObject* Outer) const;
#endif

private:
	/** When the view is spawn, create an instance of the viewmodel. */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Viewmodel", meta = (DisplayName = "Viewmodel Context Id", NoResetToDefault))
	FGuid ViewModelContextId;

public:
	UPROPERTY(EditAnywhere, Category = "Viewmodel", NoClear, meta = (DisallowCreateNew, AllowedClasses = "/Script/FieldNotification.NotifyFieldValueChanged", DisallowedClasses = "/Script/UMG.Widget", NoResetToDefault))
	TObjectPtr<UClass> NotifyFieldValueClass = nullptr;

	UPROPERTY()
	TSubclassOf<UMVVMViewModelBase> ViewModelClass_DEPRECATED;

	UPROPERTY()
	FText OverrideDisplayName_DEPRECATED;

public:
	/** Property name that will be generated. */
	UPROPERTY(EditAnywhere, Category = "Viewmodel", meta = (DisplayName = "Viewmodel Name", NoResetToDefault))
	FName ViewModelName;

	/** When the view is spawn, create an instance of the viewmodel. */
	UPROPERTY(EditAnywhere, Category = "Viewmodel")
	EMVVMBlueprintViewModelContextCreationType CreationType = EMVVMBlueprintViewModelContextCreationType::CreateInstance;

	/** Identifier of an already registered viewmodel. */
	UPROPERTY(EditAnywhere, Category = "Viewmodel", meta = (DisplayName = "Global Viewmodel Identifier"))
	FName GlobalViewModelIdentifier;

	/** The Path to get the viewmodel instance. */
	UPROPERTY(EditAnywhere, Category = "Viewmodel", meta = (DisplayName = "Viewmodel Property Path"))
	FString ViewModelPropertyPath;

	UPROPERTY(EditAnywhere, Category = "Viewmodel", Instanced, meta = (EditInline))
	TObjectPtr<UMVVMViewModelContextResolver> Resolver = nullptr;

	UPROPERTY(EditAnywhere, Category = "Viewmodel", Instanced, NoClear, meta = (ShowOnlyInnerProperties))
	TObjectPtr<UMVVMBlueprintInstancedViewModelBase> InstancedViewModel;

	/**
	 * Generate a public setter for this viewmodel.
	 * @note Always true when the Creation Type is Manual.
	 */
	UPROPERTY(EditAnywhere, Category = "Viewmodel", AdvancedDisplay, meta = (DisplayName="Create Public Setter"))
	bool bCreateSetterFunction = false;

	/**
	 * Generate a public getter for this viewmodel.
	 * @note Always false when using a Instanced viewmodel.
	 */
	UPROPERTY(EditAnywhere, Category = "Viewmodel", AdvancedDisplay, meta = (DisplayName = "Create Public Getter"))
	bool bCreateGetterFunction = true;

	/**
	 * Optional. Will not warn if the instance is not set or found.
	 * @note Always true when the Creation Type is Manual.
	 */
	UPROPERTY(EditAnywhere, Category = "Viewmodel", AdvancedDisplay)
	bool bOptional = false;

	/** Expose the viewmodel instance on every instance of the user widget for modification in editor. */
	UPROPERTY(EditAnywhere, Category = "Viewmodel", AdvancedDisplay)
	bool bExposeInstanceInEditor = false;
	
	/** Auto update the instance when the viewmodel is added/removed/modifed from the global viewmodel collection. */
	UPROPERTY(EditAnywhere, Category = "Viewmodel", AdvancedDisplay)
	bool bGlobalViewModelCollectionUpdate = false;

	UPROPERTY()
	bool bOverrideForceExecuteBindingsOnSetSource = false;

	/**
	 * When a viewmodel is set manually and the viewmodel already initialized, then always execute the bindings associated with that viewmodel.
	 * For performance and to keep the same pattern in all UMG, the bindings are usually skip if the new viewmodel value match the previous viewmodel value.
	 * This behavior can be desired if the widget is inside a pool or a binding has a side effect with another widget.
	 */
	UPROPERTY(EditAnywhere, Category = "View", meta = (EditCondition = "bOverrideForceExecuteBindingsOnSetSource"))
	bool bForceExecuteBindingsOnSetSource = false;

	/** Can change the name in the editor. */
	UPROPERTY()
	bool bCanRename = true;

	/** Can change properties in the editor. */
	UPROPERTY()
	bool bCanEdit = true;

	/** Can remove the viewmodel in the editor. */
	UPROPERTY()
	bool bCanRemove = true;

	/** Will the viewmodel be handled as a property or as an interface in Verse. */
	UPROPERTY(EditAnywhere, Category = "View")
	bool bUseAsInterface = false;

private:
	UPROPERTY()
	bool bCreateSetterFunction_Deprecation = false;
};

template<>
struct TStructOpsTypeTraits<FMVVMBlueprintViewModelContext> : public TStructOpsTypeTraitsBase2<FMVVMBlueprintViewModelContext>
{
	enum
	{
		WithPostSerialize = true,
	};
};

#undef UE_API
