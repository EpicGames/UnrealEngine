// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Misc/NamePermissionList.h"

#include "MVVMDeveloperProjectSettings.generated.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

class UK2Node;
class UListViewBase;
class UMVVMViewModelContextResolver;
class UPanelWidget;
enum class EMVVMBlueprintViewModelContextCreationType : uint8;
enum class EMVVMExecutionMode : uint8;

/**
 * 
 */
USTRUCT()
struct FMVVMDeveloperProjectWidgetSettings
{
	GENERATED_BODY()

	/** Properties or functions name that should not be use for binding (read or write). */
	UPROPERTY(EditAnywhere, config, Category = "Viewmodel")
	TSet<FName> DisallowedFieldNames;
	
	/** Properties or functions name that are displayed in the advanced category. */
	UPROPERTY(EditAnywhere, config, Category = "Viewmodel")
	TSet<FName> AdvancedFieldNames;
};

UENUM()
enum class EFilterFlag : uint8
{
	None = 0,
	All = 1 << 0
};
ENUM_CLASS_FLAGS(EFilterFlag)

USTRUCT()
struct FMVVMViewBindingFilterSettings
{
	GENERATED_BODY()

	/** Filter out the properties and functions that are not valid in the context of the binding. */
	UPROPERTY(EditAnywhere, config, Category = "Viewmodel")
	EFilterFlag FilterFlags = EFilterFlag::None;
};

/**
 *
 */
UENUM()
enum class EMVVMDeveloperConversionFunctionFilterType : uint8
{
	BlueprintActionRegistry,
	AllowedList UMETA(DisplayName="Conversion Function Library"),
};


/**
 * Implements the settings for the MVVM Editor
 */
UCLASS(MinimalAPI, config=ModelViewViewModel, defaultconfig)
class UMVVMDeveloperProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UMVVMDeveloperProjectSettings();

	UE_API virtual FName GetCategoryName() const override;
	UE_API virtual FText GetSectionText() const override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	UE_API bool PropertyHasFiltering(const UStruct* ObjectStruct, const FProperty* Property) const;
	UE_API bool IsPropertyAllowed(const UBlueprint* Context, const UStruct* ObjectStruct, const FProperty* Property) const;
	UE_API bool IsFunctionAllowed(const UBlueprint* Context, const UClass* ObjectClass, const UFunction* Function) const;
	UE_API bool IsConversionFunctionAllowed(const UBlueprint* Context, const UFunction* Function) const;
	UE_API bool IsConversionFunctionAllowed(const UBlueprint* Context, const TSubclassOf<UK2Node> Function) const;

	bool IsExecutionModeAllowed(EMVVMExecutionMode ExecutionMode) const
	{
		return AllowedExecutionMode.Contains(ExecutionMode);
	}

	bool IsContextCreationTypeAllowed(EMVVMBlueprintViewModelContextCreationType ContextCreationType) const
	{
		return AllowedContextCreationType.Contains(ContextCreationType);
	}

	EMVVMDeveloperConversionFunctionFilterType GetConversionFunctionFilter() const
	{
		return ConversionFunctionFilter;
	}

	UE_API TArray<const UClass*> GetAllowedConversionFunctionClasses() const;
	UE_API TArray<const UClass*> GetDeniedConversionFunctionClasses() const;

	UE_API bool IsExtensionSupportedForPanelClass(TSubclassOf<UPanelWidget> ClassToSupport) const;
	UE_API bool IsExtensionSupportedForListViewBaseClass(TSubclassOf<UListViewBase> ClassToSupport) const;

	/** Get generated functions permission list. */
	FPathPermissionList& GetGeneratedFunctionPermissions() { return GeneratedFunctionPermissions; }

private:
	/** Permission list for filtering which properties are visible in UI. */
	UPROPERTY(EditAnywhere, config, Category = "UX", meta=(AllowAbstract = true))
	TMap<FSoftClassPath, FMVVMDeveloperProjectWidgetSettings> FieldSelectorPermissions;

	/** Permission list for filtering which execution mode is allowed. */
	UPROPERTY(EditAnywhere, config, Category = "Defaults")
	TSet<EMVVMExecutionMode> AllowedExecutionMode;
	
	/** Permission list for filtering which context creation type is allowed. */
	UPROPERTY(EditAnywhere, config, Category = "Defaults")
	TSet<EMVVMBlueprintViewModelContextCreationType> AllowedContextCreationType;

public:
	/** Binding can be made from the DetailView Bind option. */
	UPROPERTY(EditAnywhere, config, Category = "UX")
	bool bAllowBindingFromDetailView = true;

	/** When generating a source in the viewmodel editor, allow the compiler to generate a setter function. */
	UPROPERTY(EditAnywhere, config, Category = "Features", meta=(DisplayName="Allow Generated Viewmodel Setter"))
	bool bAllowGeneratedViewModelSetter = true;

	/** When generating a binding with a long source path, allow the compiler to generate a new viewmodel source. */
	UPROPERTY(EditAnywhere, config, Category = "Features")
	bool bAllowLongSourcePath = true;
	
	/** For the binding list widget, allow the user to edit the binding in the detail view. */
	UPROPERTY(EditAnywhere, config, Category = "UX")
	bool bShowDetailViewOptionInBindingPanel = true;
	
	/** For the binding list widget and the viewmodel panel, allow the user to edit the view settings in the detail view. */
	UPROPERTY(EditAnywhere, config, Category = "UX")
	bool bShowViewSettings = true;

	/** For the binding list widget, allow the user to generate a copy of the binding/event graph. */
	UPROPERTY(EditAnywhere, config, Category = "UX")
	bool bShowDeveloperGenerateGraphSettings = true;

	UE_DEPRECATED(5.5, "MVVM AllowConversionFunctionGeneratedGraphInEditor feature is disable. The graphs are now transient.")
	/**
	 * When a conversion function requires a wrapper graph, add and save the generated graph to the blueprint.
	 * It is strongly suggested to not use this feature. It can easly break deprecation and the graph will not auto update with new features.
	 */
	UPROPERTY()
	bool bAllowConversionFunctionGeneratedGraphInEditor_DEPRECATED = false;

	/** When binding to a multicast delegate property, allow to create an event. */
	UPROPERTY(EditAnywhere, config, Category = "Features")
	bool bAllowBindingEvent = true;

	UPROPERTY(EditAnywhere, config, Category = "Features")
	bool bAllowConditionBinding = true;

	/** Allow to create an instanced viewmodel directly in the view editor. */
	UPROPERTY(EditAnywhere, config, Category = "Features", meta=(DisplayName="Experimental - Can Create Viewmodel In View"))
	bool bCanCreateViewModelInView = false;

	/**
	 * When a viewmodel is set to Create Instance, allow modifying the viewmodel instance in the editor on all instances of the owning widget.
	 * The per-viewmodel setting "Expose Instance In Editor" overrides this.
	 */
	UPROPERTY(EditAnywhere, config, Category = "Features")
	bool bExposeViewModelInstanceInEditor = false;

	/** Permission list for filtering which execution mode is allowed. */
	UPROPERTY(EditAnywhere, config, Category = "Defaults")
	EMVVMDeveloperConversionFunctionFilterType ConversionFunctionFilter = EMVVMDeveloperConversionFunctionFilterType::BlueprintActionRegistry;

	/** Classes to include in conversion function list. It includes the child class. */
	UPROPERTY(EditAnywhere, config, Category = "Defaults", meta = (AllowAbstract = true, EditCondition = "ConversionFunctionFilter == EMVVMDeveloperConversionFunctionFilterType::AllowedList"))
	TSet<FSoftClassPath> AllowedClassForConversionFunctions;

	/** Classes excluded for conversion function list. */
	UPROPERTY(EditAnywhere, config, Category = "Defaults", meta = (AllowAbstract = true, EditCondition = "ConversionFunctionFilter == EMVVMDeveloperConversionFunctionFilterType::AllowedList"))
	TSet<FSoftClassPath> DeniedClassForConversionFunctions;

	/** Modules excluded for conversion function list. ie. "/Script/MyModule" */
	UPROPERTY(EditAnywhere, config, Category = "Defaults", meta = (EditCondition = "ConversionFunctionFilter == EMVVMDeveloperConversionFunctionFilterType::AllowedList"))
	TSet<FName> DeniedModuleForConversionFunctions;

	FSimpleMulticastDelegate OnLibrarySettingChanged;

	/** The default value of UMVVMBlueprintViewSettings::bForceExecuteBindingOnSetSource. */
	UPROPERTY(EditAnywhere, config, Category = "Defaults")
	bool bForceExecuteBindingsOnSetSource = false;

	/** Settings for filtering the list of available properties and functions on binding creation. */
	UPROPERTY(EditAnywhere, config, Category = "UX")
	FMVVMViewBindingFilterSettings FilterSettings;

	/** Sub-classes of panel widget that are supported to have an extension for binding their entries to viewmodels. */
	UPROPERTY(EditAnywhere, config, Category = "Widget Extension")
	TSet<TSoftClassPtr<UPanelWidget>> SupportedPanelClassesForExtension;

	/** Sub-classes of ListViewBase that are supported to have an extension for binding their entries to viewmodels. */
	UPROPERTY(EditAnywhere, config, Category = "Widget Extension")
	TSet<TSoftClassPtr<UListViewBase>> SupportedListViewBaseClassesForExtension;

	/** Resolver class to use as the default value when selecting resolver creation mode */
	UPROPERTY(EditAnywhere, config, Category = "Defaults")
	TSoftClassPtr<UMVVMViewModelContextResolver> DefaultResolverValue;

	/** Useful in the case we want to create binders directly on widget fields, without requiring to add a viewmodel. */
	UPROPERTY()
	bool bAllowBindingEditingWithoutViewModel = false;

	/** Useful in the case we want to create conditions directly on widget fields. */
	UPROPERTY()
	bool bAllowWidgetInConditionSource = false;

private:
	/** MVVM Generated function permission list */
	FPathPermissionList GeneratedFunctionPermissions;
};

#undef UE_API
