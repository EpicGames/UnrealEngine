// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Engine/MemberReference.h"
#include "Types/MVVMFieldVariant.h"

#include "MVVMPropertyPath.generated.h"

class UBlueprint;
class UWidgetBlueprint;

/**
 * A single item in a Property Path
 */
USTRUCT()
struct FMVVMBlueprintFieldPath
{
	GENERATED_BODY()

private:
	/** Reference to property for this binding. */
	UPROPERTY(EditAnywhere, Category = "MVVM")
	FMemberReference BindingReference;

	/** If we are referencing a UFunction or FProperty */
	UPROPERTY()
	EBindingKind BindingKind = EBindingKind::Function;

public:
	FMVVMBlueprintFieldPath() = default;

	MODELVIEWVIEWMODELBLUEPRINT_API FMVVMBlueprintFieldPath(const UBlueprint* InContext, UE::MVVM::FMVVMConstFieldVariant InField);

	/**
	 * Get the binding name. It won't resolve the field.
	 * That should only be used while compiling the skeletal class.
	 */
	MODELVIEWVIEWMODELBLUEPRINT_API FName GetRawFieldName() const;

	/** Get the binding name, resolves reference deprecation / redirectors / etc before returning */
	MODELVIEWVIEWMODELBLUEPRINT_API FName GetFieldName(const UClass* SelfContext) const;

	/** */
	MODELVIEWVIEWMODELBLUEPRINT_API UE::MVVM::FMVVMConstFieldVariant GetField(const UClass* SelfContext) const;

	bool IsFieldLocalScope() const
	{
		return BindingReference.IsLocalScope();
	}
	
	bool IsFieldSelfContext() const
	{
		return BindingReference.IsSelfContext();
	}

	MODELVIEWVIEWMODELBLUEPRINT_API UClass* GetParentClass(const UClass* SelfContext) const;

	EBindingKind GetBindingKind() const
	{
		return BindingKind;
	}

#if WITH_EDITOR
	/** */
	void SetDeprecatedBindingReference(const FMemberReference& InBindingReference, EBindingKind InBindingKind);
	/** */
	void SetDeprecatedSelfReference(const UBlueprint* Context);
	
	void SetMemberName(FName NewName)
	{
		BindingReference.SetMemberName(NewName);
	}
#endif

public:
	bool operator==(const FMVVMBlueprintFieldPath& Other) const
	{
		return BindingReference.IsSameReference(Other.BindingReference)
			&& BindingKind == Other.BindingKind;
	}
	bool operator!=(const FMVVMBlueprintFieldPath& Other) const
	{
		return !operator==(Other);
	}

private:
	UE::MVVM::FMVVMConstFieldVariant GetFieldInternal(const UClass* SelfContext) const;
};

/**
 * 
 */
UENUM()
enum class EMVVMBlueprintFieldPathSource : uint8
{
	None,
	Widget,
	ViewModel,
	SelfContext,
};

/**
 * Base path to properties for MVVM view models and widgets.
 * 
 * Used to associate properties within MVVM bindings in editor & during MVVM compilation
 */
USTRUCT(BlueprintType)
struct FMVVMBlueprintPropertyPath
{
	GENERATED_BODY()

private:
	/** Reference to property for this binding. */
	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	TArray<FMVVMBlueprintFieldPath> Paths;

	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	FName WidgetName;

	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	FGuid ContextId;

	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	EMVVMBlueprintFieldPathSource Source = EMVVMBlueprintFieldPathSource::None;

	/** True if this property path is for a UI component. This affects how some fields in the path are filtered out. */
	UPROPERTY(VisibleAnywhere, Category = "MVVM")
	bool bIsComponent = false;

#if WITH_EDITORONLY_DATA
	// Use the Paths. BindingReference and BindingKind are deprecated.
	UPROPERTY()
	FMemberReference BindingReference_DEPRECATED;
	UPROPERTY()
	EBindingKind BindingKind_DEPRECATED = EBindingKind::Function;
	// Is the deprecation source done.
	UPROPERTY()
	bool bDeprecatedSource = false;
#endif

public:
	bool HasPaths() const
	{
		return Paths.Num() > 0;
	}

	const TArrayView<const FMVVMBlueprintFieldPath> GetFieldPaths() const
	{
		return Paths;
	}

	/**
	 * Get the binding names. Resolves reference deprecation / redirectors.
	 * Returns Field.SubProperty.SubProperty from ViewModel.Field.SubProperty.SubProperty
	 */
	MODELVIEWVIEWMODELBLUEPRINT_API TArray<FName> GetFieldNames(const UClass* SelfContext) const;

	/**
	 * Get the binding fields. Resolves reference deprecation / redirectors.
	 * Returns Field.SubProperty.SubProperty from ViewModel.Field.SubProperty.SubProperty
	 */
	MODELVIEWVIEWMODELBLUEPRINT_API TArray<UE::MVVM::FMVVMConstFieldVariant> GetFields(const UClass* SelfContext) const;

	/**
	 * Get the binding fields. Resolves reference deprecation / redirectors.
	 * Returns Viewmodel.Field.SubProperty.SubProperty from ViewModel.Field.SubProperty.SubProperty
	 */
	MODELVIEWVIEWMODELBLUEPRINT_API TArray<UE::MVVM::FMVVMConstFieldVariant> GetCompleteFields(const UBlueprint* SelfContext, bool bIsForDisplay = false) const;

	/**
	 * Get the full path without the first property name.
	 * Returns Field.SubProperty.SubProperty from ViewModel.Field.SubProeprty.SubProperty
	 */
	MODELVIEWVIEWMODELBLUEPRINT_API FString GetPropertyPath(const UClass* SelfContext) const;

	MODELVIEWVIEWMODELBLUEPRINT_API bool HasFieldInLocalScope() const;

	bool PropertyPathContains(const UClass* InSelfContext, UE::MVVM::FMVVMConstFieldVariant Field) const
	{
		return Paths.ContainsByPredicate([InSelfContext, Field](const FMVVMBlueprintFieldPath& FieldPath) { return FieldPath.GetField(InSelfContext) == Field; });
	}

	void SetPropertyPath(const UBlueprint* InContext, UE::MVVM::FMVVMConstFieldVariant InField)
	{
		ResetPropertyPath();
		AppendPropertyPath(InContext, InField);
	}

	void AppendPropertyPath(const UBlueprint* InContext, UE::MVVM::FMVVMConstFieldVariant InField)
	{
		if (!InField.IsEmpty() && !InField.GetName().IsNone())
		{
			Paths.Emplace(InContext, InField);
		}
	}

	void ResetPropertyPath()
	{
		Paths.Reset();
	}

	void ResetSource()
	{
		Source = EMVVMBlueprintFieldPathSource::None;
		ContextId = FGuid();
		WidgetName = FName();
		bDeprecatedSource = true;
	}

	EMVVMBlueprintFieldPathSource GetSource(const UBlueprint* InContext) const
	{
		if (!bDeprecatedSource)
		{
			const_cast<FMVVMBlueprintPropertyPath*>(this)->DeprecationUpdateSource(InContext);
		}
		return Source;
	}

	UE_DEPRECATED(5.4, "Use GetSource instead.")
	bool IsFromWidget() const
	{
		return !WidgetName.IsNone();
	}

	UE_DEPRECATED(5.4, "Use GetSource instead.")
	bool IsFromViewModel() const
	{
		return ContextId.IsValid();
	}

	FGuid GetViewModelId() const
	{
		return ContextId;
	}

	void SetViewModelId(FGuid InViewModelId)
	{
		ResetSource();
		ContextId = InViewModelId;
		Source = EMVVMBlueprintFieldPathSource::ViewModel;
	}

	FName GetWidgetName() const
	{
		return WidgetName;
	}

	void SetWidgetName(FName InWidgetName)
	{
		ResetSource();
		WidgetName = InWidgetName;
		Source = EMVVMBlueprintFieldPathSource::Widget;
	}

	void SetSelfContext()
	{
		ResetSource();
		Source = EMVVMBlueprintFieldPathSource::SelfContext;
	}

	void SetIsComponent(bool InIsComponent)
	{
		bIsComponent = InIsComponent;
	}

	bool IsComponent() const
	{
		return bIsComponent;
	}

	/**
	 * Handle when a field that is part of the path is renamed
	 * Returns true if the renamed field was part of the path
	 */
	bool OnFieldRenamed(const UBlueprint* InContext, UClass* FieldClass, FName OldObjectName, FName NewObjectName);

	bool IsValid() const
	{
		bool bHasValidSource = bDeprecatedSource && Source != EMVVMBlueprintFieldPathSource::None;
		bool bHasValidDeprecatedSource = !bDeprecatedSource && (!WidgetName.IsNone() || ContextId.IsValid());
		return bHasValidSource || bHasValidDeprecatedSource;
	}

	UE_DEPRECATED(5.4, "Use IsValid  instead.")
	bool IsEmpty() const
	{
		return !IsValid();
	}

	bool operator==(const FMVVMBlueprintPropertyPath& Other) const
	{
		return WidgetName == Other.WidgetName
			&& ContextId == Other.ContextId
			&& Source == Other.Source
			&& Paths == Other.Paths
			&& bIsComponent == Other.bIsComponent;
	}

	bool operator!=(const FMVVMBlueprintPropertyPath& Other) const
	{
		return !operator==(Other);
	}

	void PostSerialize(const FArchive& Ar)
	{
		if (Ar.IsLoading())
		{
			if (!BindingReference_DEPRECATED.GetMemberName().IsNone())
			{
				Paths.AddDefaulted_GetRef().SetDeprecatedBindingReference(BindingReference_DEPRECATED, BindingKind_DEPRECATED);
				BindingReference_DEPRECATED = FMemberReference();
			}
		}
	}

public:
	MODELVIEWVIEWMODELBLUEPRINT_API FText ToText(const UWidgetBlueprint* Blueprint, bool bUseDisplayName) const;
	MODELVIEWVIEWMODELBLUEPRINT_API FString ToString(const UWidgetBlueprint* Blueprint, bool bUseDisplayName, bool bIncludeMetaData) const;

private:
	MODELVIEWVIEWMODELBLUEPRINT_API void DeprecationUpdateSource(const UBlueprint* InContext);
	TArray<UE::MVVM::FMVVMConstFieldVariant> GetComponentFields(const UClass* InSelfContext) const;
};

template<>
struct TStructOpsTypeTraits<FMVVMBlueprintPropertyPath> : public TStructOpsTypeTraitsBase2<FMVVMBlueprintPropertyPath>
{
	enum
	{
		WithPostSerialize = true,
	};
};
