// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

struct FDiffResults;
struct FMVVMBlueprintPinId;

class IDiffCustomObject;
class IDiffCustomProperty;
class UWidgetBlueprint;
class UMVVMBlueprintViewEvent;
class UMVVMBlueprintViewCondition;

namespace UE::MVVM
{
struct FMVVMDiffCustomObjectProvider : TSharedFromThis<FMVVMDiffCustomObjectProvider>
{
	virtual ~FMVVMDiffCustomObjectProvider() = default;
	virtual TSharedPtr<IDiffCustomObject> CreateObjectDiff(const UWidgetBlueprint* NewBlueprint, const UWidgetBlueprint* OldBlueprint) { return nullptr; }
	virtual TSharedPtr<IDiffCustomProperty> CreatePropertyBindingDiff(FGuid NewBinding, FGuid OldBinding) {return nullptr;}
	virtual TSharedPtr<IDiffCustomProperty> CreatePropertyConditionDiff(const UMVVMBlueprintViewCondition* NewCondition, const UMVVMBlueprintViewCondition* OldCondition) {return nullptr;}
	virtual TSharedPtr<IDiffCustomProperty> CreatePropertyEventDiff(const UMVVMBlueprintViewEvent* NewEvent, const UMVVMBlueprintViewEvent* OldEvent) {return nullptr;}
	virtual TSharedPtr<IDiffCustomProperty> CreatePropertyParameterDiff(const FMVVMBlueprintPinId& NewParameter, const FMVVMBlueprintPinId& OldParameter) {return nullptr;}
};

namespace FMVVMWidgetBlueprintDiff
{
	void MODELVIEWVIEWMODELBLUEPRINT_API FindDiffs(const UWidgetBlueprint* NewBlueprint, const UWidgetBlueprint* OldBlueprint, FDiffResults& Results);
	void MODELVIEWVIEWMODELBLUEPRINT_API RegisterCustomDiff(TSharedPtr<FMVVMDiffCustomObjectProvider> DiffCustomObject);
	void MODELVIEWVIEWMODELBLUEPRINT_API UnregisterCustomDiff();
}
}