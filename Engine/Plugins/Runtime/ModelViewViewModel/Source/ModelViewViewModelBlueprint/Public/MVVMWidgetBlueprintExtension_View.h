// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WidgetBlueprintExtension.h"
#include "MVVMDeveloperProjectSettings.h"

#include "MVVMWidgetBlueprintExtension_View.generated.h"

#define UE_API MODELVIEWVIEWMODELBLUEPRINT_API

class UMVVMBlueprintView;
class UMVVMBlueprintViewExtension;
class FWidgetBlueprintCompilerContext;
class UWidgetBlueprintGeneratedClass;

namespace UE::MVVM
{
	class FClipboardExtension;
}

namespace UE::MVVM::Private
{
	struct FMVVMViewBlueprintCompiler;
} //namespace

USTRUCT()
struct FMVVMExtensionItem
{
	GENERATED_BODY()

	UPROPERTY()
	FName WidgetName = NAME_None;

	UPROPERTY()
	FGuid ViewmodelId = FGuid();

	UPROPERTY()
	TObjectPtr<UMVVMBlueprintViewExtension> ExtensionObj;

	bool operator==(const FMVVMExtensionItem& OtherExt) const
	{
		return OtherExt.WidgetName == WidgetName 
			&&	OtherExt.ExtensionObj == ExtensionObj
			&&	OtherExt.ViewmodelId == ViewmodelId;
	}
};

/**
 *
 */
UCLASS(MinimalAPI)
class UMVVMWidgetBlueprintExtension_View : public UWidgetBlueprintExtension
{
	GENERATED_BODY()

public:
	UE_API void CreateBlueprintViewInstance();
	UE_API void DestroyBlueprintViewInstance();

	UMVVMBlueprintView* GetBlueprintView()
	{
		return BlueprintView;
	}

	const UMVVMBlueprintView* GetBlueprintView() const
	{
		return BlueprintView;
	}

	FSimpleMulticastDelegate& OnBlueprintViewChangedDelegate()
	{
		return BlueprintViewChangedDelegate;
	}

	const TArrayView<const FName> GetGeneratedFunctions() const
	{
		return GeneratedFunctions;
	}

public:
	//~ Begin UObject interface
	UE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	UE_API virtual void PostInitProperties() override;
#endif
	//~ End UObject interface

	//~ Begin UWidgetBlueprintExtension interface
	UE_API virtual void HandlePreloadObjectsForCompilation(UBlueprint* OwningBlueprint) override;
	UE_API virtual void HandleBeginCompilation(FWidgetBlueprintCompilerContext& InCreationContext) override;
	UE_API virtual void HandleCleanAndSanitizeClass(UWidgetBlueprintGeneratedClass* ClassToClean, UObject* OldCDO) override;
	UE_API virtual void HandlePopulateGeneratedVariables(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context) override;
	UE_API virtual void HandleCreateClassVariablesFromBlueprint(const FWidgetBlueprintCompilerContext::FCreateVariableContext& Context) override;
	UE_API virtual void HandleCreateFunctionList(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& Context) override;
	UE_API virtual void HandleFinishCompilingClass(UWidgetBlueprintGeneratedClass* Class) override;
	UE_API virtual void HandleEndCompilation() override;
	UE_API virtual FSearchData HandleGatherSearchData(const UBlueprint* OwningBlueprint) const override;

#if WITH_EDITOR
	UE_API virtual void FindDiffs(const UWidgetBlueprint* OwningBlueprint, const UWidgetBlueprint* OtherBlueprint, FDiffResults& Results) const override;
	UE_API virtual void GatherWidgetMemberReferences(FName WidgetVariableName, const UClass* WidgetClass, TSet<FName>& OutReferencedProperties, TSet<FName>& OutReferencedFunctions) const override;
#endif //WITH_EDITOR
	//~ End UWidgetBlueprintExtension interface

	UE_API void VerifyWidgetExtensions();
	UE_API void OnFieldRenamed(UClass* FieldOwnerClass, FName OldName, FName NewName);
	UE_API void RenameWidgetExtensions(FName OldWidgetName, FName NewWidgetName);
	UE_API UMVVMBlueprintViewExtension* CreateBlueprintWidgetExtension(TSubclassOf<UMVVMBlueprintViewExtension> ExtensionClass, FName WidgetName);
	UE_API void RemoveBlueprintWidgetExtension(UMVVMBlueprintViewExtension* ExtensionToRemove, FName WidgetName);
	UE_API TArray<UMVVMBlueprintViewExtension*> GetBlueprintExtensionsForWidget(FName WidgetName) const;
	UE_API TArray<UMVVMBlueprintViewExtension*> GetAllBlueprintExtensions() const;

	UE_API void SetFilterSettings(FMVVMViewBindingFilterSettings InFilterSettings);
	FMVVMViewBindingFilterSettings GetFilterSettings() const
	{
		return FilterSettings;
	}

	UPROPERTY(Transient)
	TMap<FGuid, TWeakObjectPtr<UObject>> TemporaryViewModelInstances;

private:
	UPROPERTY(Instanced)
	TObjectPtr<UMVVMBlueprintView> BlueprintView;

	UPROPERTY(Transient)
	FMVVMViewBindingFilterSettings FilterSettings;
	
	UPROPERTY(Transient)
	TArray<FName> GeneratedFunctions;

	FSimpleMulticastDelegate BlueprintViewChangedDelegate;
	TPimplPtr<UE::MVVM::Private::FMVVMViewBlueprintCompiler> CurrentCompilerContext;

	UPROPERTY()
	TArray<FMVVMExtensionItem> BlueprintExtensions;

	friend UE::MVVM::Private::FMVVMViewBlueprintCompiler;
	friend UE::MVVM::FClipboardExtension;
};

#undef UE_API
