// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"

#include "MVVMViewModelBlueprint.generated.h"

class FKismetCompilerContext;

class FCompilerResultsLog;
class UEdGraph;
class UMVVMViewModelBlueprintGeneratedClass;


/**
 * Will be deprecated in the next version.
 */
UCLASS(MinimalAPI)
class UMVVMViewModelBlueprint : public UBlueprint
{
	GENERATED_BODY()
};
