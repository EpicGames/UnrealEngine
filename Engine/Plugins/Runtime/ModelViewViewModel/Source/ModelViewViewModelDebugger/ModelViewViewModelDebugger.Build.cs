// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using UnrealBuildTool;

public class ModelViewViewModelDebugger : ModuleRules 
{
	public ModelViewViewModelDebugger(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"FieldNotification",
				"ModelViewViewModel",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"SlateCore",
				"Slate",
				"UMG",
			});
	}
}
