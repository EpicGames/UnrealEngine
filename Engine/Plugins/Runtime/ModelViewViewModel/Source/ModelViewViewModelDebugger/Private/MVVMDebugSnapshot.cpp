// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMDebugSnapshot.h"
#include "MVVMDebugView.h"
#include "MVVMDebugViewClass.h"

#include "Bindings/MVVMCompiledBindingLibrary.h"
#include "Blueprint/UserWidget.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "Templates/ValueOrError.h"
#include "MVVMDebugView.h"
#include "MVVMDebugViewModel.h"
#include "MVVMViewModelBase.h"
#include "UObject/UObjectIterator.h"
#include "View/MVVMView.h"
#include "View/MVVMViewClass.h"

namespace UE::MVVM
{
namespace Private
{

FMVVMViewSourceDebugEntry CreateSourceInstanceEntry(const UMVVMView* View, const FMVVMView_Source& ViewSource)
{
	FMVVMViewSourceDebugEntry Result;
	Result.SourceInstanceName = View->GetViewClass()->GetSource(ViewSource.ClassKey).GetName();
	Result.SourceAsset = ViewSource.Source ? FAssetData(ViewSource.Source->GetClass()) : FAssetData();
	Result.ViewModelDebugId = FGuid::NewGuid();
	Result.LiveSource = ViewSource.Source;
	return Result;
}

FMVVMViewBindingDebugEntry CreateViewBindingDebugEntry(const UMVVMViewClass* ViewClass, const FMVVMViewClass_Binding& Binding, int32 Index)
{
	FMVVMViewBindingDebugEntry Result;
#if WITH_EDITOR
	Result.BlueprintViewBindingId = Binding.GetEditorId();
#endif
	if (ViewClass->GetBindingLibrary().IsLoaded())
	{
		if (Binding.GetBinding().GetSourceFieldPath().IsValid())
		{
			TValueOrError<FString, FString> SourceFieldPathValue = ViewClass->GetBindingLibrary().FieldPathToString(Binding.GetBinding().GetSourceFieldPath(), false);
			Result.SourceFieldPath = SourceFieldPathValue.HasValue() ? SourceFieldPathValue.StealValue() : SourceFieldPathValue.StealError();
		}
		if (Binding.GetBinding().GetDestinationFieldPath().IsValid())
		{
			TValueOrError<FString, FString> DestinationFieldPathValue = ViewClass->GetBindingLibrary().FieldPathToString(Binding.GetBinding().GetDestinationFieldPath(), false);
			Result.DestinationFieldPath = DestinationFieldPathValue.HasValue() ? DestinationFieldPathValue.StealValue() : DestinationFieldPathValue.StealError();
		}
		if (Binding.GetBinding().GetConversionFunctionFieldPath().IsValid())
		{
			TValueOrError<FString, FString> ConversionFieldPathValue = ViewClass->GetBindingLibrary().FieldPathToString(Binding.GetBinding().GetConversionFunctionFieldPath(), false);
			Result.ConversionFunctionFieldPath = ConversionFieldPathValue.HasValue() ? ConversionFieldPathValue.StealValue() : ConversionFieldPathValue.StealError();
		}
	}
	Result.CompiledBindingIndex = Index;
	return Result;
}

FMVVMViewModelFieldBoundDebugEntry CreateViewModelFieldBoundDebugEntry(const UE::FieldNotification::FFieldMulticastDelegate::FDelegateView& DelegateView)
{
	FMVVMViewModelFieldBoundDebugEntry Result;
	Result.KeyObjectName = DelegateView.KeyObject ? DelegateView.KeyObject->GetFName() : FName();
	Result.KeyFieldId = FFieldNotificationId(DelegateView.KeyField.GetName());
	Result.BindingFunctionName = DelegateView.BindingFunctionName;
	Result.BindingObjectPathName = DelegateView.BindingObject ? DelegateView.BindingObject->GetPathName() : FString();
	Result.LiveInstanceKeyObject = DelegateView.KeyObject;
	Result.LiveInstanceBindingObject = DelegateView.BindingObject;
	return Result;
}
}//namespace private


TSharedPtr<FMVVMViewDebugEntry> FDebugSnapshot::FindView(FGuid Id) const
{
	const TSharedPtr<FMVVMViewDebugEntry>* Result = Views.FindByPredicate([Id](const TSharedPtr<FMVVMViewDebugEntry>& Other){ return Other->ViewInstanceDebugId == Id; });
	return Result ? *Result : TSharedPtr<FMVVMViewDebugEntry>();
}


TSharedPtr<FMVVMViewModelDebugEntry> FDebugSnapshot::FindViewModel(FGuid Id) const
{
	const TSharedPtr<FMVVMViewModelDebugEntry>* Result = ViewModels.FindByPredicate([Id](const TSharedPtr<FMVVMViewModelDebugEntry>& Other) { return Other->ViewModelDebugId == Id; });
	return Result ? *Result : TSharedPtr<FMVVMViewModelDebugEntry>();
}


TSharedPtr<FDebugSnapshot> FDebugSnapshot::CreateSnapshot()
{
	TSharedPtr<FDebugSnapshot> Snapshot = MakeShared<FDebugSnapshot>();

	for (FThreadSafeObjectIterator It(UMVVMView::StaticClass()); It; ++It)
	{
		if (It->IsTemplate(RF_ClassDefaultObject))
		{
			continue;
		}

		UMVVMView* View = CastChecked<UMVVMView>(*It);
		UUserWidget* UserWidget = View->GetOuterUUserWidget();

		TSharedPtr<FMVVMViewDebugEntry> DebugEntry = MakeShared<FMVVMViewDebugEntry>();
		if (UserWidget)
		{
			DebugEntry->UserWidgetInstanceName = UserWidget->GetFName();
			ULocalPlayer* LocalPlayer = UserWidget->GetOwningLocalPlayer();
			DebugEntry->LocalPlayerName = LocalPlayer ? LocalPlayer->GetFName() : FName();
			UWorld* World = UserWidget->GetWorld();
			DebugEntry->WorldName = World ? World->GetFName() : FName();
			DebugEntry->UserWidgetAsset = FAssetData(UserWidget->GetClass());
		}

		for (const FMVVMView_Source& ViewSource : View->GetSources())
		{
			DebugEntry->Sources.Add(Private::CreateSourceInstanceEntry(View, ViewSource));
		}
		DebugEntry->ViewClassDebugId = Snapshot->FindOrAddViewClassEntry(View->GetViewClass())->ViewClassDebugId;
		DebugEntry->ViewInstanceDebugId = FGuid::NewGuid();
		DebugEntry->LiveView = View;

		Snapshot->Views.Add(DebugEntry);
	}

	for (FThreadSafeObjectIterator It(UMVVMViewModelBase::StaticClass()); It; ++It)
	{
		if (It->IsTemplate(RF_ClassDefaultObject))
		{
			continue;
		}

		UMVVMViewModelBase* ViewModel = CastChecked<UMVVMViewModelBase>(*It);

		TSharedPtr<FMVVMViewModelDebugEntry> DebugEntry = MakeShared<FMVVMViewModelDebugEntry>();
		DebugEntry->Name = ViewModel->GetFName();
		DebugEntry->PathName = ViewModel->GetPathName();
		DebugEntry->ViewModelAsset = FAssetData(ViewModel->GetClass());
		for (const UE::FieldNotification::FFieldMulticastDelegate::FDelegateView& DelegateView : ViewModel->GetNotificationDelegateView())
		{
			DebugEntry->FieldBound.Add(Private::CreateViewModelFieldBoundDebugEntry(DelegateView));
		}
		//DebugEntry.PropertyBag;
		DebugEntry->ViewModelDebugId = FGuid::NewGuid();
		DebugEntry->LiveViewModel = ViewModel;

		Snapshot->ViewModels.Add(DebugEntry);
	}

	return Snapshot;
}


TSharedRef<FMVVMViewClassDebugEntry> FDebugSnapshot::FindOrAddViewClassEntry(const UMVVMViewClass* ViewClass)
{
	TSharedPtr<FMVVMViewClassDebugEntry>* Entry = ViewClasses.FindByPredicate([ViewClass](const TSharedPtr<FMVVMViewClassDebugEntry>& Entry) { return Entry->LiveViewClass == ViewClass; });
	if (Entry)
	{
		return Entry->ToSharedRef();
	}

	TSharedRef<FMVVMViewClassDebugEntry> NewEntry = MakeShared<FMVVMViewClassDebugEntry>();
	ViewClasses.Add(NewEntry);
	NewEntry->ViewClassDebugId = FGuid::NewGuid();
	NewEntry->LiveViewClass = ViewClass;

	int32 Index = 0;
	for (const FMVVMViewClass_Binding& Binding : ViewClass->GetBindings())
	{
		NewEntry->Bindings.Add(Private::CreateViewBindingDebugEntry(ViewClass, Binding, Index));
		++Index;
	}
	return NewEntry;
}
}//namespace UE::MVVM