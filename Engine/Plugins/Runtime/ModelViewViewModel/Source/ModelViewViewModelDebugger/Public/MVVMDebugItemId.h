// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::MVVM
{

struct FDebugItemId
{
	enum class EType : uint8
	{
		Invalid,
		View,
		ViewModel,
	} Type = EType::Invalid;
	FGuid Id;


	FDebugItemId() = default;
	FDebugItemId(EType InType, FGuid InId)
		: Type(InType), Id(InId)
	{ }

	bool operator==(const FDebugItemId& Other) const
	{
		return Other.Type == Type && Other.Id == Id;
	}
};

}
