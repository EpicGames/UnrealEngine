// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/ObjectMacros.h"
#include "MVVMDebugItemId.h"

#define UE_API MODELVIEWVIEWMODELDEBUGGER_API

struct FMVVMViewDebugEntry;
struct FMVVMViewClassDebugEntry;
struct FMVVMViewModelDebugEntry;
class UMVVMViewClass;

namespace UE::MVVM
{

class FDebugSnapshot
{
private:
	TArray<TSharedPtr<FMVVMViewDebugEntry>> Views;
	TArray<TSharedPtr<FMVVMViewClassDebugEntry>> ViewClasses;
	TArray<TSharedPtr<FMVVMViewModelDebugEntry>> ViewModels;

public:
	TArrayView<TSharedPtr<FMVVMViewDebugEntry>> GetViews()
	{
		return Views;
	}
	TArrayView<TSharedPtr<FMVVMViewModelDebugEntry>> GetViewModels()
	{
		return ViewModels;
	}

	UE_API TSharedPtr<FMVVMViewDebugEntry> FindView(FGuid Id) const;
	UE_API TSharedPtr<FMVVMViewModelDebugEntry> FindViewModel(FGuid Id) const;

	static UE_API TSharedPtr<FDebugSnapshot> CreateSnapshot();

private:
	UE_API TSharedRef<FMVVMViewClassDebugEntry> FindOrAddViewClassEntry(const UMVVMViewClass* ViewClass);
};

};

#undef UE_API
