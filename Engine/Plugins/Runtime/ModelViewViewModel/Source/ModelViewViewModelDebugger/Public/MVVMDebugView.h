// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/ObjectMacros.h"

#include "MVVMDebugView.generated.h"

class UMVVMView;


USTRUCT()
struct FMVVMViewSourceDebugEntry
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FName SourceInstanceName;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FAssetData SourceAsset;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FGuid ViewModelDebugId; // if it exist and if it's a viewmodel

	TWeakObjectPtr<UObject> LiveSource; // TScriptInterface<INotifyFieldValueChanged>
};


USTRUCT()
struct FMVVMViewDebugEntry
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FName UserWidgetInstanceName;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FName LocalPlayerName;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FName WorldName;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FAssetData UserWidgetAsset;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	TArray<FMVVMViewSourceDebugEntry> Sources;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FGuid ViewClassDebugId;
	
	UPROPERTY()
	FGuid ViewInstanceDebugId;

	TWeakObjectPtr<UMVVMView> LiveView;
};