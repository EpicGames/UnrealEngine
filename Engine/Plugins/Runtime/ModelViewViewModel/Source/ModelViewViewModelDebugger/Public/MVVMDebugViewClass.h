// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/ObjectMacros.h"

#include "MVVMDebugViewClass.generated.h"

class UMVVMViewClass;


USTRUCT()
struct FMVVMViewBindingDebugEntry
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FGuid BlueprintViewBindingId;
	
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FString SourceFieldPath;
	
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FString DestinationFieldPath;
	
	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FString ConversionFunctionFieldPath;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	int32 CompiledBindingIndex = INDEX_NONE;
};


USTRUCT()
struct FMVVMViewClassDebugEntry
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	TArray<FMVVMViewBindingDebugEntry> Bindings;

	UPROPERTY()
	FGuid ViewClassDebugId;

	TWeakObjectPtr<const UMVVMViewClass> LiveViewClass;
};