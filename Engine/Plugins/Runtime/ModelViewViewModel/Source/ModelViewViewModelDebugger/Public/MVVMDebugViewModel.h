// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/ObjectMacros.h"

#include "FieldNotificationId.h"

#include "MVVMDebugViewModel.generated.h"


USTRUCT()
struct FMVVMViewModelFieldBoundDebugEntry
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FName KeyObjectName;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FFieldNotificationId KeyFieldId;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FString BindingObjectPathName;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FName BindingFunctionName;

	TWeakObjectPtr<const UObject> LiveInstanceKeyObject;
	TWeakObjectPtr<const UObject> LiveInstanceBindingObject;
};


USTRUCT()
struct FMVVMViewModelDebugEntry
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FName Name;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FString PathName;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FAssetData ViewModelAsset;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	TArray<FMVVMViewModelFieldBoundDebugEntry> FieldBound;

	//UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	//FInstancedPropertyBag PropertyBag;

	UPROPERTY(VisibleAnywhere, Category = "Viewmodel")
	FGuid ViewModelDebugId;

	TWeakObjectPtr<UObject> LiveViewModel;
};