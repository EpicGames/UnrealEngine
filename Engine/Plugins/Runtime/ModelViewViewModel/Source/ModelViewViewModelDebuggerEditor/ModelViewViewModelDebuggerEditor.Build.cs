// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using UnrealBuildTool;

public class ModelViewViewModelDebuggerEditor : ModuleRules 
{
	public ModelViewViewModelDebuggerEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "MVVMDebuggerEditor";

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"ModelViewViewModel",
				"ModelViewViewModelDebugger",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"InputCore",
				"LevelEditor",
				"MessageLog",
				"Projects",
				"SlateCore",
				"Slate",
				"ToolMenus",
				"ToolWidgets",
				"WorkspaceMenuStructure",
			});
	}
}
