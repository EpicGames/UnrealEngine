// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditor.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include "Framework/Docking/TabManager.h"
#include "Styling/MVVMDebuggerEditorStyle.h"
#include "Widgets/SMainDebugTab.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "MVVMDebuggerModule"

namespace UE::MVVM::Private
{
	const FName MainTabName("MVVMDebug");
}

class FMVVMDebuggerEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		UE::MVVM::FMVVMDebuggerEditorStyle::CreateInstance();

		UToolMenu* ModesToolbar = UToolMenus::Get()->RegisterMenu("ModelViewViewModel.Debug.Toolbar", FName(), EMultiBoxType::ToolBar);
		ModesToolbar->StyleName = "AssetEditorToolbar";

		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
			UE::MVVM::Private::MainTabName, FOnSpawnTab::CreateRaw(this, &FMVVMDebuggerEditorModule::SpawnDebugTab))
			.SetDisplayName(LOCTEXT("TabTitle", "Viewmodel Debugger"))
			.SetIcon(FSlateIcon("MVVMDebuggerEditorStyle", "Viewmodel.TabIcon"))
			.SetTooltipText(LOCTEXT("TooltipText", "Opens Viewmodel Debugger."))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory());
	}

	virtual void ShutdownModule() override
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(UE::MVVM::Private::MainTabName);
		UE::MVVM::FMVVMDebuggerEditorStyle::DestroyInstance();
	}

private:
	TSharedRef<SDockTab> SpawnDebugTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);
		MajorTab->SetContent(SNew(UE::MVVM::SMainDebug, MajorTab));
		return MajorTab;
	}
};


IMPLEMENT_MODULE(FMVVMDebuggerEditorModule, ModelViewViewModelDebuggerEditor);

#undef LOCTEXT_NAMESPACE
