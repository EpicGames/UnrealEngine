// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/MVVMDebuggerEditorStyle.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleMacros.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Interfaces/IPluginManager.h"

namespace UE::MVVM
{

TUniquePtr<FMVVMDebuggerEditorStyle, FMVVMDebuggerEditorStyle::FCustomDeleter> FMVVMDebuggerEditorStyle::Instance;


FMVVMDebuggerEditorStyle::FMVVMDebuggerEditorStyle()
	: FSlateStyleSet("MVVMDebuggerEditorStyle")
{
	const FVector2D Icon10x10(10.0f, 10.0f);
	const FVector2D Icon14x14(14.0f, 14.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);

	TSharedPtr<IPlugin> MVVMPlugin = IPluginManager::Get().FindPlugin("ModelViewViewModel");
	if (ensure(MVVMPlugin))
	{
		SetContentRoot(MVVMPlugin->GetContentDir() / TEXT("Editor"));
	}

	Set("Viewmodel.TabIcon", new IMAGE_BRUSH_SVG("Slate/ViewModel", Icon16x16));
	Set("Icon.TakeSnapshot", new IMAGE_BRUSH_SVG("Starship/Common/SaveThumbnail", Icon24x24));


	FSlateStyleRegistry::RegisterSlateStyle(*this);
}


FMVVMDebuggerEditorStyle::~FMVVMDebuggerEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}


void FMVVMDebuggerEditorStyle::CreateInstance()
{
	Instance.Reset(new FMVVMDebuggerEditorStyle);
}


void FMVVMDebuggerEditorStyle::DestroyInstance()
{
	Instance.Reset();
}

} //namespace