// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::MVVM
{

/**
 * 
 */
class FMVVMDebuggerEditorStyle : public FSlateStyleSet
{

public:
	static void CreateInstance();
	static void DestroyInstance();

	static FMVVMDebuggerEditorStyle& Get()
	{
		check(Instance);
		return *Instance;
	}

private:
	FMVVMDebuggerEditorStyle();
	FMVVMDebuggerEditorStyle(const FMVVMDebuggerEditorStyle&) = delete;
	FMVVMDebuggerEditorStyle& operator=(const FMVVMDebuggerEditorStyle&) = delete;
	~FMVVMDebuggerEditorStyle();
	struct FCustomDeleter
	{
		void operator()(FMVVMDebuggerEditorStyle* Ptr) const
		{
			delete Ptr;
		}
	};
	static TUniquePtr<FMVVMDebuggerEditorStyle, FCustomDeleter> Instance;
};

} // namespace
