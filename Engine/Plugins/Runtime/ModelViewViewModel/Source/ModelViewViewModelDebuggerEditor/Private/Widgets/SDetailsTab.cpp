// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDetailsTab.h"

#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "MVVMDebuggerSDetailsTab"

namespace UE::MVVM
{

void SDetailsTab::Construct(const FArguments& InArgs)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
		DetailsViewArgs.bAllowFavoriteSystem = true;
		DetailsViewArgs.bShowObjectLabel = false;
		DetailsViewArgs.bHideSelectionTip = true;
	}

	if (InArgs._UseStructDetailView)
	{
		FStructureDetailsViewArgs StructureViewArgs;
		{
			StructureViewArgs.bShowObjects = true;
			StructureViewArgs.bShowAssets = true;
			StructureViewArgs.bShowClasses = true;
			StructureViewArgs.bShowInterfaces = true;
		}

		TSharedRef<IStructureDetailsView> PropertyView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, TSharedPtr<FStructOnScope>());
		StructDetailView = PropertyView;
		ChildSlot
		[
			PropertyView->GetWidget().ToSharedRef()
		];
	}
	else
	{
		TSharedRef<IDetailsView> PropertyView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailView = PropertyView;

		ChildSlot
		[
			PropertyView
		];
	}
}

void SDetailsTab::SetObjects(const TArray<UObject*>& InObjects)
{
	if (DetailView)
	{
		DetailView->SetObjects(InObjects);
	}
}

void SDetailsTab::SetStruct(TSharedPtr<FStructOnScope> InStructData)
{
	if (StructDetailView)
	{
		StructDetailView->SetStructureData(InStructData);
	}
}

} //namespace

#undef LOCTEXT_NAMESPACE