// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class IDetailsView;
class IStructureDetailsView;
class FStructOnScope;

namespace UE::MVVM
{

class SDetailsTab : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDetailsTab) { }
	SLATE_ARGUMENT_DEFAULT(bool, UseStructDetailView) = false;
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	void SetObjects(const TArray<UObject*>& InObjects);
	void SetStruct(TSharedPtr<FStructOnScope> InStructData);

private:
	TSharedPtr<IDetailsView> DetailView;
	TSharedPtr<IStructureDetailsView> StructDetailView;
};

} //namespace
