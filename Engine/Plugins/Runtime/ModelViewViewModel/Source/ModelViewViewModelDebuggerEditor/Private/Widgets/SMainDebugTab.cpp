// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMainDebugTab.h"

#include "MVVMDebugSnapshot.h"
#include "MVVMDebugViewModel.h"
#include "UObject/StructOnScope.h"
#include "View/MVVMView.h"

#include "ToolMenus.h"

#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SDetailsTab.h"
#include "Widgets/SMessagesLog.h"
#include "Widgets/SViewModelBindingDetail.h"
#include "Widgets/SViewModelSelection.h"
#include "Widgets/SViewSelection.h"


#define LOCTEXT_NAMESPACE "MVVMDebuggerMainDebug"

namespace UE::MVVM
{

namespace Private
{
const FLazyName Stack_ViewSelection = "ViewSelection";
const FLazyName Stack_ViewmodelSelection = "ViewmodelSelection";
const FLazyName Stack_Binding = "Binding";
const FLazyName Stack_LiveDetail = "LiveDetail";
const FLazyName Stack_EntryDetail = "EntryDetail";
const FLazyName Stack_Messages = "Messages";
}

void SMainDebug::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& InParentTab)
{
	//ULevelEditorMenuContext* LevelEditorMenuContext = NewObject<ULevelEditorMenuContext>();
	//FToolMenuContext MenuContext = FToolMenuContext(LevelEditorMenuContext);
	FToolMenuContext MenuContext;
	BuildToolMenu();

	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateSP(this, &SMainDebug::HandlePullDownWindowMenu),
		"Window"
	);

	const TSharedRef<SWidget> MenuWidget = MenuBarBuilder.MakeWidget();

	TSharedRef<SWidget> DockingArea = CreateDockingArea(InParentTab);
	TabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuWidget);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			MenuWidget
		]
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 2.0f))
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(0)
			.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
			[
				UToolMenus::Get()->GenerateWidget("ModelViewViewModel.Debug.Toolbar", MenuContext)
			]
			]
			+ SVerticalBox::Slot()
			.Padding(4.0f, 2.f, 4.f, 2.f)
			.FillHeight(1.0f)
			[
				DockingArea
			]
	];
}


void SMainDebug::HandlePullDownWindowMenu(FMenuBuilder& MenuBuilder)
{
	if (!TabManager.IsValid())
	{
		return;
	}

	TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);
}


void SMainDebug::BuildToolMenu()
{
	UToolMenu* ModesToolbar = UToolMenus::Get()->ExtendMenu("ModelViewViewModel.Debug.Toolbar");
	check(ModesToolbar);

	{
		FToolMenuSection& Section = ModesToolbar->AddSection("Snapshot"); 

		FToolMenuEntry TakeSnapshotButtonEntry = FToolMenuEntry::InitToolBarButton(
			"TakeSnapshot",
			FUIAction(
				FExecuteAction::CreateSP(this, &SMainDebug::HandleTakeSnapshot)
			),
			LOCTEXT("TakesnapshotLabel", "Take snapshot"),
			LOCTEXT("TakesnapshotTooltip", "Take snapshot"),
			FSlateIcon("MVVMDebuggerEditorStyle", "Viewmodel.TabIcon")
		);
		Section.AddEntry(TakeSnapshotButtonEntry);

		FToolMenuEntry ConfigureSnapshotMenuEntry = FToolMenuEntry::InitComboButton(
			"ConfigureSnapshot",
			FUIAction(),
			FOnGetContent::CreateSP(this, &SMainDebug::HandleSnapshotMenuContent),
			LOCTEXT("ConfigureSnapshotLabel", "Configure snapshot"),
			LOCTEXT("ConfigureSnapshotTooltip", "Configure snapshot"),
			FSlateIcon(),
			true //bInSimpleComboBox
		);
		ConfigureSnapshotMenuEntry.StyleNameOverride = "CalloutToolbar";
		Section.AddEntry(ConfigureSnapshotMenuEntry);

		//Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		//	"LoadSnapshot",
		//	FUIAction(
		//		FExecuteAction::CreateSP(this, &SMainDebug::HandleLoadSnapshot)
		//		),
		//	LOCTEXT("LoadSnapshotLabel", "Load Snapshot"),
		//	LOCTEXT("LoadSnapshotTooltip", "Load Snapshot"),
		//	FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Import")
		//));

		//Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		//	"SaveSnapshot",
		//	FUIAction(
		//		FExecuteAction::CreateSP(this, &SMainDebug::HandleSaveSnapshot),
		//		FCanExecuteAction::CreateSP(this, &SMainDebug::HasValidSnapshot)
		//	),
		//	LOCTEXT("SaveSnapshotLabel", "Save Snapshot"),
		//	LOCTEXT("SaveSnapshotTooltip", "Save Snapshot"),
		//	FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		//));
	}
}


TSharedRef<SWidget> SMainDebug::HandleSnapshotMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.BeginSection("Profile", LOCTEXT("SnapshotContextMenuSectionName", "Snapshot"));
	{
		//MenuBuilder.AddMenuEntry(
		//	LOCTEXT("CreateMenuLabel", "New Empty Media Profile"),
		//	LOCTEXT("CreateMenuTooltip", "Create a new Media Profile asset."),
		//	FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName(), TEXT("ClassIcon.MediaProfile")),
		//	FUIAction(FExecuteAction::CreateRaw(this, &FMediaProfileMenuEntryImpl::CreateNewProfile))
		//);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


void SMainDebug::HandleTakeSnapshot()
{
	CurrentSelection = ESelection::None;
	Selection();

	Snapshot = FDebugSnapshot::CreateSnapshot();
	if (TSharedPtr<SViewSelection> SelectionViewPtr = ViewSelection.Pin())
	{
		SelectionViewPtr->SetSnapshot(Snapshot);
	}
	if (TSharedPtr<SViewModelSelection> SelectionViewModelPtr = ViewModelSelection.Pin())
	{
		SelectionViewModelPtr->SetSnapshot(Snapshot);
	}
}


void SMainDebug::HandleLoadSnapshot()
{
	//FNotificationInfo Info(LOCTEXT("LoadFail", "Failed to load snapshot from disk."));
	//Info.ExpireDuration = 2.0f;
	//FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
}


void SMainDebug::HandleSaveSnapshot()
{}


bool SMainDebug::HasValidSnapshot() const
{
	return Snapshot.IsValid();
}


void SMainDebug::HandleViewSelectionChanged()
{
	CurrentSelection = ESelection::View;
	Selection();
	if (TSharedPtr<SViewModelSelection> ViewModelSelectionPtr = ViewModelSelection.Pin())
	{
		ViewModelSelectionPtr->SetSelection(FGuid());
	}
}


void SMainDebug::HandleViewModleSelectionChanged()
{
	CurrentSelection = ESelection::ViewModel;
	Selection();
	if (TSharedPtr<SViewSelection> ViewSelectionPtr = ViewSelection.Pin())
	{
		ViewSelectionPtr->SetSelection(FGuid());
	}
}


void SMainDebug::Selection()
{
	if (Snapshot == nullptr)
	{
		if (TSharedPtr<SDetailsTab> DetailViewPtr = LiveDetailView.Pin())
		{
			DetailViewPtr->SetObjects(TArray<UObject*>());
		}
		if (TSharedPtr<SDetailsTab> DetailViewPtr = EntryDetailView.Pin())
		{
			DetailViewPtr->SetStruct(TSharedPtr<FStructOnScope>());
		}
		if (TSharedPtr<SViewModelBindingDetail> ViewModelBindingDetailPtr = ViewModelBindingDetail.Pin())
		{
			ViewModelBindingDetailPtr->SetViewModels(TArray<TSharedPtr<FMVVMViewModelDebugEntry>>());
		}
	}
	else
	{
		TArray<UObject*> SelectedObjects;
		TSharedPtr<FStructOnScope> SelectedStruct;
		if (CurrentSelection == ESelection::View)
		{
			if (TSharedPtr<SViewSelection> ViewSelectionPtr = ViewSelection.Pin())
			{
				for (TSharedPtr<FMVVMViewDebugEntry>& Selection : ViewSelectionPtr->GetSelectedItems())
				{
					SelectedObjects.Add(Selection->LiveView.Get());
					if (!SelectedStruct.IsValid())
					{
						SelectedStruct = MakeShared<FStructOnScope>(FMVVMViewDebugEntry::StaticStruct(), reinterpret_cast<uint8*>(Selection.Get()));
					}
				}
			}
		}
		else if (CurrentSelection == ESelection::ViewModel)
		{
			if (TSharedPtr<SViewModelSelection> ViewModelSelectionPtr = ViewModelSelection.Pin())
			{
				for (TSharedPtr<FMVVMViewModelDebugEntry>& Selection : ViewModelSelectionPtr->GetSelectedItems())
				{
					SelectedObjects.Add(Selection->LiveViewModel.Get());
					if (!SelectedStruct.IsValid())
					{
						SelectedStruct = MakeShared<FStructOnScope>(FMVVMViewModelDebugEntry::StaticStruct(), reinterpret_cast<uint8*>(Selection.Get()));
					}
				}
			}
		}

		TArray<FDebugItemId> SelectedItems;
		if (CurrentSelection == ESelection::View)
		{
			if (TSharedPtr<SViewSelection> ViewSelectionPtr = ViewSelection.Pin())
			{
				for (const TSharedPtr<FMVVMViewDebugEntry>& Selection : ViewSelectionPtr->GetSelectedItems())
				{
					SelectedItems.Add(FDebugItemId(FDebugItemId::EType::View, Selection->ViewInstanceDebugId));
				}
			}
		}
		else if (CurrentSelection == ESelection::ViewModel)
		{
			if (TSharedPtr<SViewModelSelection> ViewModelSelectionPtr = ViewModelSelection.Pin())
			{
				for (const TSharedPtr<FMVVMViewModelDebugEntry>& Selection : ViewModelSelectionPtr->GetSelectedItems())
				{
					SelectedItems.Add(FDebugItemId(FDebugItemId::EType::ViewModel, Selection->ViewModelDebugId));
				}
			}
		}

		if (TSharedPtr<SDetailsTab> DetailViewPtr = LiveDetailView.Pin())
		{
			DetailViewPtr->SetObjects(SelectedObjects);
		}
		if (TSharedPtr<SDetailsTab> DetailViewPtr = EntryDetailView.Pin())
		{
			DetailViewPtr->SetStruct(SelectedStruct);
		}

		if (TSharedPtr<SViewModelBindingDetail> ViewModelBindingDetailPtr = ViewModelBindingDetail.Pin())
		{
			TArray<TSharedPtr<FMVVMViewModelDebugEntry>> ViewModels;
			for (FDebugItemId DebugItem : SelectedItems)
			{
				if (DebugItem.Type == FDebugItemId::EType::ViewModel)
				{
					if (TSharedPtr<FMVVMViewModelDebugEntry> Found = Snapshot->FindViewModel(DebugItem.Id))
					{
						ViewModels.Add(Found);
					}
				}
			}
			ViewModelBindingDetailPtr->SetViewModels(ViewModels);
		}
	}
}


TSharedRef<SWidget> SMainDebug::CreateDockingArea(const TSharedRef<SDockTab>& InParentTab)
{
	const FName LayoutName = TEXT("MVVMDebugger_Layout_v1.0");
	TSharedRef<FTabManager::FLayout> DefaultLayer = FTabManager::NewLayout(LayoutName)
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.25f)
						->SetHideTabWell(true)
						->AddTab(Private::Stack_ViewSelection, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.25f)
						->SetHideTabWell(true)
						->AddTab(Private::Stack_ViewmodelSelection, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->SetHideTabWell(true)
					->AddTab(Private::Stack_Binding, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.25f)
						->SetHideTabWell(true)
						->AddTab(Private::Stack_LiveDetail, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.25f)
						->SetHideTabWell(true)
						->AddTab(Private::Stack_EntryDetail, ETabState::OpenedTab)
					)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(1.0f)
				->SetHideTabWell(true)
				->AddTab(Private::Stack_Messages, ETabState::OpenedTab)
			)
		);

	TSharedRef<FTabManager::FLayout> Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, DefaultLayer);
	FLayoutExtender LayoutExtender;
	Layout->ProcessExtensions(LayoutExtender);

	TabManager = FGlobalTabmanager::Get()->NewTabManager(InParentTab);

	TabManager->RegisterTabSpawner(Private::Stack_ViewSelection, FOnSpawnTab::CreateSP(this, &SMainDebug::SpawnViewSelectionTab))
		.SetDisplayName(LOCTEXT("SelectionTab", "Selection"));
	TabManager->RegisterTabSpawner(Private::Stack_ViewmodelSelection, FOnSpawnTab::CreateSP(this, &SMainDebug::SpawnViewModelSelectionTab))
		.SetDisplayName(LOCTEXT("SelectionTab", "Selection"));
	TabManager->RegisterTabSpawner(Private::Stack_Binding, FOnSpawnTab::CreateSP(this, &SMainDebug::SpawnBindingTab))
		.SetDisplayName(LOCTEXT("BindingTab", "Bindings"));
	TabManager->RegisterTabSpawner(Private::Stack_LiveDetail, FOnSpawnTab::CreateSP(this, &SMainDebug::SpawnLiveDetailTab))
		.SetDisplayName(LOCTEXT("LiveDetailTab", "Live Object Details"));
	TabManager->RegisterTabSpawner(Private::Stack_EntryDetail, FOnSpawnTab::CreateSP(this, &SMainDebug::SpawnEntryDetailTab))
		.SetDisplayName(LOCTEXT("EntryDetailTab", "Entry Details"));
	TabManager->RegisterTabSpawner(Private::Stack_Messages, FOnSpawnTab::CreateSP(this, &SMainDebug::SpawnMessagesTab))
		.SetDisplayName(LOCTEXT("MessageLogTab", "Messages Log"));

	return TabManager->RestoreFrom(Layout, nullptr).ToSharedRef();
}


TSharedRef<SDockTab> SMainDebug::SpawnViewSelectionTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SViewSelection> LocalSelectionView = SNew(SViewSelection)
		.OnSelectionChanged(this, &SMainDebug::HandleViewSelectionChanged);
	ViewSelection = LocalSelectionView;
	LocalSelectionView->SetSnapshot(Snapshot);
	return SNew(SDockTab)
		.Label(LOCTEXT("ViewTab", "View"))
		[
			LocalSelectionView
		];
}

TSharedRef<SDockTab> SMainDebug::SpawnViewModelSelectionTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SViewModelSelection> LocalSelectionView = SNew(SViewModelSelection)
		.OnSelectionChanged(this, &SMainDebug::HandleViewModleSelectionChanged);
	ViewModelSelection = LocalSelectionView;
	LocalSelectionView->SetSnapshot(Snapshot);
	return SNew(SDockTab)
		.Label(LOCTEXT("ViewmodelTab", "Viewmodel"))
		[
			LocalSelectionView
		];
}


TSharedRef<SDockTab> SMainDebug::SpawnBindingTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("BindingTab", "Bindings"))
		[
			SAssignNew(ViewModelBindingDetail, SViewModelBindingDetail)
		];
}


TSharedRef<SDockTab> SMainDebug::SpawnLiveDetailTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("LiveDetailTab", "Live Object Details"))
		[
			SAssignNew(LiveDetailView, SDetailsTab)
			.UseStructDetailView(false)
		];
}


TSharedRef<SDockTab> SMainDebug::SpawnEntryDetailTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("EntryDetailTab", "Entry Details"))
		[
			SAssignNew(EntryDetailView, SDetailsTab)
			.UseStructDetailView(true)
		];
}


TSharedRef<SDockTab> SMainDebug::SpawnMessagesTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("MessageLogTab", "Messages Log"))
		.ShouldAutosize(true)
		[
			SAssignNew(MessageLog, SMessagesLog)
		];
}

} //namespace

#undef LOCTEXT_NAMESPACE