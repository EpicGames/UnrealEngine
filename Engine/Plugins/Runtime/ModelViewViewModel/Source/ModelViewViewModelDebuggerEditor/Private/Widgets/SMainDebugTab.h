// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"


class FMenuBuilder;
class FSpawnTabArgs;
class FTabManager;
class SDockTab;
class UToolMenu;

namespace UE::MVVM
{

class FDebugSnapshot;

class SDetailsTab;
class SMessagesLog;
class SViewModelSelection;
class SViewModelBindingDetail;
class SViewSelection;

class SMainDebug : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMainDebug) { }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& InParentTab);

private:
	void BuildToolMenu();
	void HandlePullDownWindowMenu(FMenuBuilder& MenuBuilder);
	TSharedRef<SWidget> HandleSnapshotMenuContent();

	void HandleTakeSnapshot();
	void HandleLoadSnapshot();
	void HandleSaveSnapshot();
	bool HasValidSnapshot() const;

	void HandleViewSelectionChanged();
	void HandleViewModleSelectionChanged();
	void Selection();

	TSharedRef<SWidget> CreateDockingArea(const TSharedRef<SDockTab>& InParentTab);
	TSharedRef<SDockTab> SpawnViewSelectionTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnViewModelSelectionTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnBindingTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnLiveDetailTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnEntryDetailTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnMessagesTab(const FSpawnTabArgs& Args);

private:
	TSharedPtr<FTabManager> TabManager;

	TWeakPtr<SDetailsTab> LiveDetailView;
	TWeakPtr<SDetailsTab> EntryDetailView;
	TWeakPtr<SMessagesLog> MessageLog;
	TWeakPtr<SViewSelection> ViewSelection;
	TWeakPtr<SViewModelSelection> ViewModelSelection;
	TWeakPtr<SViewModelBindingDetail> ViewModelBindingDetail;

	TSharedPtr<FDebugSnapshot> Snapshot;

	enum class ESelection
	{
		None,
		View,
		ViewModel,
	};
	ESelection CurrentSelection = ESelection::None;
};

} //namespace
