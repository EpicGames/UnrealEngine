// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMessagesLog.h"

#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#include "MVVMMessageLog.h"
#include "Modules/ModuleManager.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MVVMDebuggerMessagesLog"

namespace UE::MVVM
{

void SMessagesLog::Construct(const FArguments& InArgs)
{
	const FName LogName = UE::MVVM::FMessageLog::LogName;
	TSharedPtr<IMessageLogListing> MessageLogListing;

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	if (MessageLogModule.IsRegisteredLogListing(LogName))
	{
		MessageLogListing = MessageLogModule.GetLogListing(LogName);
	}
	TSharedRef<class SWidget> MessageLogListingWidget = MessageLogListing.IsValid() ? MessageLogModule.CreateLogListingWidget(MessageLogListing.ToSharedRef()) : SNullWidget::NullWidget;

	ChildSlot
	[
		//SNew(SOverlay)
		//+SOverlay::Slot()
		//[
			MessageLogListingWidget
		//]
		//+SOverlay::Slot()
		//.HAlign(HAlign_Left)
		//.VAlign(VAlign_Bottom)
		//[
		//	SNew(SHorizontalBox)
		//	+SHorizontalBox::Slot()
		//	.AutoWidth()
		//	.Padding(10, 4, 4, 10)
		//	[
		//		SNew(STextBlock)
		//		.Text(this, &SMessagesLog::GetMessageCountText)
		//	]
		//]
	];
}

FText SMessagesLog::GetMessageCountText() const
{
	int32 ErrorCount = 0;
	int32 WarningCount = 0;
	int32 InfoCount = 0;
	//FMessageLog::GetLogCount(ErrorCount, WarningCount, InfoCount);
	return FText::Format(LOCTEXT("MessageCountText", "{0} Error(s)  {1} Warning(s)"), FText::AsNumber(ErrorCount), FText::AsNumber(WarningCount));
}

} //namespace

#undef LOCTEXT_NAMESPACE