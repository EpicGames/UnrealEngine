// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"


namespace UE::MVVM
{

class SMessagesLog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMessagesLog) { }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

private:
	FText GetMessageCountText() const;
};

} //namespace
