// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSelectionBase.h"

#include "MVVMDebugSnapshot.h"
#include "MVVMDebugView.h"
#include "MVVMDebugViewClass.h"

#include "Filters/SFilterSearchBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "MVVMDebuggerViewSelection"

namespace UE::MVVM::Private
{

void SSelectionBase::Construct(const FArguments& InArgs)
{
	//VerticalBox->AddSlot()
	//.AutoHeight()
	//[
	//	SNew(SVerticalBox)
	//	+ SVerticalBox::Slot()
	//	.AutoHeight()
	//	[
	//		SNew(SMultiLineEditableTextBox)
	//		.IsReadOnly(true)
	//		.Visibility_Lambda([this]() { return Mode->HasErrors() ? EVisibility::Visible : EVisibility::Collapsed; })
	//		.Font(IDetailLayoutBuilder::GetDetailFontBold())
	//		.BackgroundColor(FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor"))
	//		.Text(Mode->GetErrorsText())
	//		.AutoWrapText(true)
	//	]
	//	+ SVerticalBox::Slot()
	//	.AutoHeight()
	//	[
	//		SNew(SButton)
	//		.OnClicked_Lambda([this] { Mode->RepairErrors(); return FReply::Handled(); })
	//		.Visibility_Lambda([this]() { return Mode->HasErrors() ? EVisibility::Visible : EVisibility::Collapsed; })
	//		.HAlign(HAlign_Center)
	//		.Text(LOCTEXT("SceneOutlinerRepairErrors", "Repair Errors"))
	//	]
	//];

	TSharedRef<SHorizontalBox> Toolbar = SNew(SHorizontalBox)
	//+SHorizontalBox::Slot().VAlign(VAlign_Center)
	//.Padding(0.0f, 0.0f, 2.0f, 0.0f)
	//.AutoWidth()
	//[
	//	SSceneOutlinerFilterBar::MakeAddFilterButton(FilterBar.ToSharedRef())
	//]
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	[
		SAssignNew(SearchBox, SFilterSearchBox)
		.HintText(LOCTEXT("FilterSearch", "Search..."))
		.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search (pressing enter selects the results)"))
		.OnTextChanged(this, &SSelectionBase::HandleFilterTextChanged)
	]
	+SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		SNew(SComboButton)
		.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon") // Use the tool bar item style for this button
		.OnGetMenuContent(this, &SSelectionBase::GetViewButtonContent)
		.HasDownArrow(false)
		.ButtonContent()
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
		]
	];
	//+SHorizontalBox::Slot()
	//.VAlign(VAlign_Center)
	//.AutoWidth()
	//[
	//	SNew(SButton)
	//	.ButtonStyle(FAppStyle::Get(), "SimpleButton")
	//	.OnClicked(this, &SSelectionBase::HandleLockView)
	//	.ContentPadding(FMargin(4, 2))
	//	.HAlign(HAlign_Center)
	//	.VAlign(VAlign_Center)
	//	.ToolTipText(LOCTEXT("LockSelectionButton_ToolTip", "Locks the current selection into the Details panel"))
	//	[
	//		SNew(SImage)
	//		.ColorAndOpacity(FSlateColor::UseForeground())
	//		.Image(this, &SDetailNameArea::OnGetLockButtonImageResource)
	//	]
	//];


	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox)
	+SVerticalBox::Slot()
	.AutoHeight()
	.Padding(8.0f, 8.0f, 8.0f, 4.0f)
	[
		Toolbar
	];

	//	VerticalBox->AddSlot()
	//		.AutoHeight()
	//		.Padding(0.0f, 0.0f, 0.0f, 4.0f)
	//		[
	//			FilterBar.ToSharedRef()
	//		];
	//}

	VerticalBox->AddSlot()
	.FillHeight(1.0)
	[
		SNew(SOverlay)
		//+ SOverlay::Slot()
		//.HAlign(HAlign_Center)
		//[
		//	SNew(STextBlock)
		//	.Visibility(this, &SSceneOutliner::GetEmptyLabelVisibility)
		//	.Text(LOCTEXT("EmptyLabel", "Empty"))
		//	.ColorAndOpacity(FLinearColor(0.4f, 1.0f, 0.4f))
		//]
		+ SOverlay::Slot()
		[
			SNew(SBorder).BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		]
		+ SOverlay::Slot()
		[
			BuildListImpl()
		]
	];


	//// Bottom panel status bar, if enabled by the mode
	//VerticalBox->AddSlot()
	//.AutoHeight()
	//[
	//	SNew(SBorder)
	//	.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
	//	.VAlign(VAlign_Center)
	//	.HAlign(HAlign_Left)
	//	.Padding(FMargin(14, 9))
	//	[
	//		SNew(STextBlock)
	//		.Text(this, &SSelectionBase::GetFilterStatusText)
	//		.ColorAndOpacity(this, &SSelectionBase::GetFilterStatusTextColor)
	//	]
	//];

	ChildSlot
	[
		VerticalBox
	];
}


void SSelectionBase::SetSnapshot(const TSharedPtr<FDebugSnapshot> InSnapshot)
{
	Snapshot = InSnapshot;
	UpdateSource();
}


void SSelectionBase::HandleFilterTextChanged(const FText& InFilterText)
{
	//for (const TSharedPtr<FTreeNode>& Node : FilteredTreeSource)
	//{
	//	SetHighlightTextRecursive(Node, InFilterText);
	//}

	FText ErrorText = FilterTextChangedImpl(InFilterText);
	if (SearchBox)
	{
		SearchBox->SetError(ErrorText);
	}
	UpdateSource();
}


TSharedRef<SWidget> SSelectionBase::GetViewButtonContent() const
{
	// Menu should stay open on selection if filters are not being shown
	FMenuBuilder MenuBuilder(true, nullptr);

	//MenuBuilder.BeginSection("OutlinerSettings", LOCTEXT("HierarchyHeading", "Hierarchy"));
	//{
	//	MenuBuilder.AddMenuEntry(
	//		LOCTEXT("ExpandAll", "Refresh"),
	//		LOCTEXT("ExpandAllToolTip", "Expand All Items in the Hierarchy"),
	//		FSlateIcon(),
	//		FUIAction(FExecuteAction::CreateSP(this, &SSelectionBase::ExpandAll)));

	//	MenuBuilder.AddMenuEntry(
	//		LOCTEXT("CollapseAll", "Collapse All"),
	//		LOCTEXT("CollapseAllToolTip", "Collapse All Items in the Hierarchy"),
	//		FSlateIcon(),
	//		FUIAction(FExecuteAction::CreateSP(this, &SSceneOutliner::CollapseAll)));

	//	MenuBuilder.AddMenuEntry(
	//		LOCTEXT("ShowHierarchy", "Stack Hierarchy Headers"),
	//		LOCTEXT("ShowHierarchyToolTip", "Toggle pinning of the hierarchy of items at the top of the outliner"),
	//		FSlateIcon(),
	//		FUIAction(
	//			FExecuteAction::CreateRaw(this, &SSceneOutliner::ToggleStackHierarchyHeaders),
	//			FCanExecuteAction(),
	//			FIsActionChecked::CreateRaw(this, &SSceneOutliner::ShouldStackHierarchyHeaders)
	//		),
	//		NAME_None,
	//		EUserInterfaceActionType::ToggleButton
	//	);
	//}
	//MenuBuilder.EndSection();

	//if (bShowFilters)
	//{
	//	MenuBuilder.BeginSection("AssetThumbnails", LOCTEXT("ShowHeading", "Show"));
	//	{
	//		// Add mode filters
	//		for (auto& ModeFilterInfo : Mode->GetFilterInfos())
	//		{
	//			ModeFilterInfo.Value.AddMenu(MenuBuilder);
	//		}
	//	}
	//	MenuBuilder.EndSection();
	//}
	//Mode->CreateViewContent(MenuBuilder);

	return MenuBuilder.MakeWidget();
}


FText SSelectionBase::GetFilterStatusText() const
{
	return FText::GetEmpty();
}


FSlateColor SSelectionBase::GetFilterStatusTextColor() const
{
	return FSlateColor();
}

} //namespace

#undef LOCTEXT_NAMESPACE