// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/TextFilter.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class SFilterSearchBox;
class ITableRow;

class SHeader;
class SHeaderRow;
class STableViewBase;

namespace UE::MVVM
{

class FDebugSnapshot;

namespace Private
{

class SSelectionBase : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSelectionBase) { }
	SLATE_EVENT(FSimpleDelegate, OnSelectionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetSnapshot(const TSharedPtr<FDebugSnapshot> Snapshot);
	TSharedPtr<FDebugSnapshot> GetSnapshot() const
	{
		return Snapshot;
	}

protected:
	TSharedPtr<SFilterSearchBox> SearchBox;
	TSharedPtr<FDebugSnapshot> Snapshot;
	FSimpleDelegate OnSelectionChanged;
	bool bIsSelectionReentrant = false;

protected:
	virtual void UpdateSource() = 0;
	virtual TSharedRef<STableViewBase> BuildListImpl() = 0;
	virtual FText FilterTextChangedImpl(const FText& InFilterText) = 0;

private:
	void HandleFilterTextChanged(const FText& InFilterText);
	TSharedRef<SWidget> GetViewButtonContent() const;
	FText GetFilterStatusText() const;
	FSlateColor GetFilterStatusTextColor() const;
};
}

template <typename ItemType, typename ItemWidgetType>
class SSelectionBase : public Private::SSelectionBase
{
public:
	SLATE_BEGIN_ARGS(SSelectionBase) { }
	SLATE_EVENT(FSimpleDelegate, OnSelectionChanged)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs)
	{
		OnSelectionChanged = InArgs._OnSelectionChanged;
		SearchFilter = MakeShared<FTextFilter>(FTextFilter::FItemToStringArray::CreateSP(this, &SSelectionBase::HandleGetFilterStrings));
		Private::SSelectionBase::Construct(Private::SSelectionBase::FArguments());
	}

	TArray<ItemType> GetSelectedItems() const
	{
		TArray<ItemType> Result;
		ListView->GetSelectedItems(Result);
		return Result;
	}

protected:
	virtual void GatherFilterStringsImpl(ItemType Item, TArray<FString>& OutStrings) const = 0;
	virtual TSharedRef<SHeaderRow> BuildHeaderRowImpl() const = 0;
	virtual TArray<ItemType> UpdateSourceImpl(TSharedPtr<TTextFilter<ItemType>> TextFilter) = 0;

	virtual FText FilterTextChangedImpl(const FText& InFilterText) override
	{
		SearchFilter->SetRawFilterText(InFilterText);
		return SearchFilter->GetFilterErrorText();
	}

private:
	virtual void UpdateSource() override
	{
		FilteredSource = UpdateSourceImpl(SearchFilter);
		//FilteredItems.Sort(&FSortPlaceableItems::SortItemsByName);
		ListView->RequestListRefresh();
	}

	virtual TSharedRef<STableViewBase> BuildListImpl() override
	{
		return SAssignNew(ListView, SListView<ItemType>)
			.ListItemsSource(&FilteredSource)
			//.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow(this, &SSelectionBase::HandleGenerateWidgetForItem)
			.OnSelectionChanged(this, &SSelectionBase::HandleListSelectionChanged)
			//.OnMouseButtonDoubleClick(this, &SSceneOutliner::OnOutlinerTreeDoubleClick)
			//.OnContextMenuOpening(this, &SLiveLinkClientPanel::OnOpenVirtualSubjectContextMenu)
			.HeaderRow(BuildHeaderRowImpl());
	}

	void HandleGetFilterStrings(ItemType Item, TArray<FString>& OutStrings)
	{
		GatherFilterStringsImpl(Item, OutStrings);
	}

	void HandleListSelectionChanged(ItemType TreeItem, ESelectInfo::Type SelectInfo)
	{
		if (SelectInfo == ESelectInfo::Direct)
		{
			return;
		}

		if (!bIsSelectionReentrant)
		{
			TGuardValue<bool> ReentrantGuard(bIsSelectionReentrant, true);
			OnSelectionChanged.ExecuteIfBound();
		}
	}

	TSharedRef<ITableRow> HandleGenerateWidgetForItem(ItemType Entry, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		return SNew(ItemWidgetType, OwnerTable)
			.Entry(Entry);
	}


protected:
	TArray<ItemType> FilteredSource;
	TSharedPtr<SListView<ItemType>> ListView;

	using FTextFilter = TTextFilter<ItemType>;
	TSharedPtr<FTextFilter> SearchFilter;
};

} //namespace
