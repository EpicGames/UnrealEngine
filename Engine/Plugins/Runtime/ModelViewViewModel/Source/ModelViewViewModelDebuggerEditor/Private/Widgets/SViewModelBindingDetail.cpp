// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SViewModelBindingDetail.h"

#include "MVVMDebugSnapshot.h"
#include "MVVMDebugView.h"
#include "MVVMDebugViewClass.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

//ToolWidgets

#define LOCTEXT_NAMESPACE "MVVMViewModelBindingDetail"


namespace UE::MVVM
{
namespace Private
{

namespace ViewColumnName
{
FLazyName KeyFieldId = "FieldId";
FLazyName KeyObjectName = "KeyObjectName";
FLazyName BindingObjectName = "BindingObjectName";
FLazyName BindingFunctionName = "BindingFunction";
}

class SViewModelBindingDetailItem : public SMultiColumnTableRow<SViewModelBindingDetail::FViewModelBindingId>
{
	SLATE_BEGIN_ARGS(SViewModelBindingDetailItem) {}
	SLATE_ARGUMENT(SViewModelBindingDetail::FViewModelBindingId, BindingId)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		BindingId = Args._BindingId;

		SMultiColumnTableRow<SViewModelBindingDetail::FViewModelBindingId>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			OwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (BindingId.ViewModel && BindingId.ViewModel->FieldBound.IsValidIndex(BindingId.Index))
		{
			const FMVVMViewModelFieldBoundDebugEntry& Entry = BindingId.ViewModel->FieldBound[BindingId.Index];
			if (ColumnName == ViewColumnName::KeyObjectName)
			{
				return SNew(STextBlock)
					.Text(FText::FromName(Entry.KeyObjectName));
			}
			if (ColumnName == ViewColumnName::KeyFieldId)
			{
				return SNew(STextBlock)
					.Text(FText::FromName(Entry.KeyFieldId.GetFieldName()));
			}
			if (ColumnName == ViewColumnName::BindingObjectName)
			{
				return SNew(STextBlock)
					.Text(FText::FromString(Entry.BindingObjectPathName));
			}
			if (ColumnName == ViewColumnName::BindingFunctionName)
			{
				return SNew(STextBlock)
					.Text(FText::FromName(Entry.BindingFunctionName));
			}
		}

		check(false);
		return SNew(STextBlock);
	}

private:
	SViewModelBindingDetail::FViewModelBindingId BindingId;
};
} //namespace Private


void SViewModelBindingDetail::Construct(const FArguments& InArgs)
{
	TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow)
		+ SHeaderRow::Column(Private::ViewColumnName::KeyObjectName)
		.FillWidth(0.25f)
		.DefaultLabel(LOCTEXT("KeyObjectdColumnHeaderName", "Key Object"))
		+ SHeaderRow::Column(Private::ViewColumnName::KeyFieldId)
		.FillWidth(0.25f)
		.DefaultLabel(LOCTEXT("FieldIdColumnHeaderName", "Field"))
		+ SHeaderRow::Column(Private::ViewColumnName::BindingObjectName)
		.FillWidth(0.50f)
		.DefaultLabel(LOCTEXT("BindingObjectColumnHeaderName", "Delegate Object"))
		+SHeaderRow::Column(Private::ViewColumnName::BindingFunctionName)
		.FillWidth(0.50f)
		.DefaultLabel(LOCTEXT("BindingFunctionColumnHeaderName", "Delegate Function"));

	SAssignNew(ListView, SListView<FViewModelBindingId>)
		.ListItemsSource(&Entries)
		//.SelectionMode(ESelectionMode::Single)
		.OnGenerateRow(this, &SViewModelBindingDetail::HandleGenerateWidgetForItem)
		.OnSelectionChanged(this, &SViewModelBindingDetail::HandleListSelectionChanged)
		//.OnMouseButtonDoubleClick(this, &SViewModelBindingDetail::OnOutlinerTreeDoubleClick)
		//.OnContextMenuOpening(this, &SViewModelBindingDetail::OnOpenVirtualSubjectContextMenu)
		.HeaderRow(HeaderRow);

	ChildSlot
	[
		ListView.ToSharedRef()
	];
}


void SViewModelBindingDetail::SetViewModels(TArray<TSharedPtr<FMVVMViewModelDebugEntry>> InViewModel)
{
	if (ViewModels != InViewModel)
	{
		ViewModels = InViewModel;
		Entries.Reset();

		FViewModelBindingId NewItem;
		for (TSharedPtr<FMVVMViewModelDebugEntry> Entry : ViewModels)
		{
			NewItem.ViewModel = Entry;
			for (int32 Index = 0; Index < Entry->FieldBound.Num(); ++Index)
			{
				NewItem.Index = Index;
				Entries.Add(NewItem);
			}
		}

		ListView->RebuildList(); //RequestListRefresh
	}
}


TSharedRef<ITableRow> SViewModelBindingDetail::HandleGenerateWidgetForItem(FViewModelBindingId Entry, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(Private::SViewModelBindingDetailItem, OwnerTable)
		.BindingId(Entry);
}


void SViewModelBindingDetail::HandleListSelectionChanged(FViewModelBindingId TreeItem, ESelectInfo::Type SelectInfo)
{}

} //namespace

#undef LOCTEXT_NAMESPACE