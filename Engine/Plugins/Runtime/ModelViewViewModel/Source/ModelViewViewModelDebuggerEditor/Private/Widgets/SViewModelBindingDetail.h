// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

#include "Framework/Views/TableViewTypeTraits.h"

#include "MVVMDebugView.h"
#include "MVVMDebugViewModel.h"

template<typename ItemType>
class SListView;
class ITableRow;
class STableViewBase;

namespace UE::MVVM
{

class SViewModelBindingDetail : public SCompoundWidget
{
public:
	struct FViewModelBindingId
	{
		TSharedPtr<FMVVMViewModelDebugEntry> ViewModel;
		int32 Index = INDEX_NONE;

		bool operator==(const FViewModelBindingId& Other) const
		{
			return Index == Other.Index && ViewModel == Other.ViewModel;
		}

		friend uint32 GetTypeHash(const FViewModelBindingId& Value)
		{
			return HashCombine(GetTypeHash(Value.ViewModel), GetTypeHash(Value.Index));
		}
	};

public:
	SLATE_BEGIN_ARGS(SViewModelBindingDetail) { }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	void SetViewModels(TArray<TSharedPtr<FMVVMViewModelDebugEntry>> ViewModel);

private:
	TSharedRef<ITableRow> HandleGenerateWidgetForItem(FViewModelBindingId Entry, const TSharedRef<STableViewBase>& OwnerTable);
	void HandleListSelectionChanged(FViewModelBindingId TreeItem, ESelectInfo::Type SelectInfo);

private:
	TArray<TSharedPtr<FMVVMViewModelDebugEntry>> ViewModels;
	TArray<FViewModelBindingId> Entries;
	TSharedPtr<SListView<FViewModelBindingId>> ListView;
};

} //namespace


template <>
struct TListTypeTraits<UE::MVVM::SViewModelBindingDetail::SViewModelBindingDetail::FViewModelBindingId>
{
public:
	typedef UE::MVVM::SViewModelBindingDetail::FViewModelBindingId NullableType;

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<UE::MVVM::SViewModelBindingDetail::SViewModelBindingDetail::FViewModelBindingId, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<UE::MVVM::SViewModelBindingDetail::SViewModelBindingDetail::FViewModelBindingId, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<UE::MVVM::SViewModelBindingDetail::SViewModelBindingDetail::FViewModelBindingId>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector& Collector
		, TArray<UE::MVVM::SViewModelBindingDetail::FViewModelBindingId>& ItemsWithGeneratedWidgets
		, TSet<UE::MVVM::SViewModelBindingDetail::FViewModelBindingId>& SelectedItems
		, TMap<const U*, UE::MVVM::SViewModelBindingDetail::FViewModelBindingId>& WidgetToItemMap)
	{
	}

	static bool IsPtrValid(const UE::MVVM::SViewModelBindingDetail::FViewModelBindingId& InValue)
	{
		return InValue.ViewModel.IsValid() && InValue.Index != INDEX_NONE;
	}

	static void ResetPtr(UE::MVVM::SViewModelBindingDetail::FViewModelBindingId& InValue)
	{
		InValue.ViewModel.Reset();
		InValue.Index = INDEX_NONE;
	}

	static UE::MVVM::SViewModelBindingDetail::FViewModelBindingId MakeNullPtr()
	{
		return UE::MVVM::SViewModelBindingDetail::FViewModelBindingId();
	}

	static UE::MVVM::SViewModelBindingDetail::FViewModelBindingId NullableItemTypeConvertToItemType(const UE::MVVM::SViewModelBindingDetail::FViewModelBindingId& InValue)
	{
		return InValue;
	}

	static FString DebugDump(UE::MVVM::SViewModelBindingDetail::FViewModelBindingId InValue)
	{
		return FString();
	}

	class SerializerType {};
};

template <>
struct TIsValidListItem<UE::MVVM::SViewModelBindingDetail::SViewModelBindingDetail::FViewModelBindingId>
{
	enum
	{
		Value = true
	};
};
