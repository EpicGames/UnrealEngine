// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SViewModelSelection.h"

#include "MVVMDebugSnapshot.h"
#include "MVVMDebugView.h"
#include "MVVMDebugViewClass.h"

#include "Misc/TextFilter.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "MVVMDebuggerViewModelSelection"

namespace UE::MVVM
{
namespace Private
{
namespace ViewModelColumnName
{
FLazyName Name = "Name";
FLazyName Class = "Class";
}


void SViewModelSelectionItem::Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
{
	Entry = Args._Entry;

	SMultiColumnTableRow<TSharedPtr<FMVVMViewModelDebugEntry>>::Construct(
		FSuperRowType::FArguments()
		.Padding(1.0f),
		OwnerTableView
	);
}

TSharedRef<SWidget> SViewModelSelectionItem::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == ViewModelColumnName::Name)
	{
		return SNew(STextBlock)
			.Text(FText::FromName(Entry->Name));
	}
	if (ColumnName == ViewModelColumnName::Class)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(Entry->ViewModelAsset.GetObjectPathString()));
	}

	check(false);
	return SNew(STextBlock);
}

} //namespace Private


void SViewModelSelection::Construct(const FArguments& InArgs)
{
	Super::Construct(Super::FArguments()
		.OnSelectionChanged(InArgs._OnSelectionChanged)
	);
}


void SViewModelSelection::SetSelection(FGuid Id)
{
	bool bClear = true;
	if (GetSnapshot())
	{
		if (TSharedPtr<FMVVMViewModelDebugEntry> Found = GetSnapshot()->FindViewModel(Id))
		{
			ListView->SetSelection(Found);
			bClear = false;
		}
	}
	if (bClear)
	{
		ListView->ClearSelection();
	}
}


TArray<TSharedPtr<FMVVMViewModelDebugEntry>> SViewModelSelection::UpdateSourceImpl(TSharedPtr<TTextFilter<TSharedPtr<FMVVMViewModelDebugEntry>>> InTextFilter)
{
	TArray<TSharedPtr<FMVVMViewModelDebugEntry>> Result;
	if (GetSnapshot())
	{
		if (!InTextFilter->GetRawFilterText().IsEmpty())
		{
			Result.Reserve(Snapshot->GetViewModels().Num());
			for (const TSharedPtr<FMVVMViewModelDebugEntry>& Item : GetSnapshot()->GetViewModels())
			{
				if (InTextFilter->PassesFilter(Item))
				{
					Result.Add(Item);
				}
			}
		}
		else
		{
			Result = Snapshot->GetViewModels();
		}
	}
	return Result;
}


TSharedRef<SHeaderRow> SViewModelSelection::BuildHeaderRowImpl() const
{
	return SNew(SHeaderRow)
		+ SHeaderRow::Column(Private::ViewModelColumnName::Name)
		.FillWidth(0.75f)
		.DefaultLabel(LOCTEXT("ViewmodelColumnHeaderName", "Name"))
		+ SHeaderRow::Column(Private::ViewModelColumnName::Class)
		.FillWidth(0.25f)
		.DefaultLabel(LOCTEXT("ClassColumnHeaderName", "Class"));
}


void SViewModelSelection::GatherFilterStringsImpl(TSharedPtr<FMVVMViewModelDebugEntry> Item, TArray<FString>& OutStrings) const
{
	OutStrings.Add(Item->PathName);
	OutStrings.Add(Item->ViewModelAsset.GetObjectPathString());
}

} //namespace

#undef LOCTEXT_NAMESPACE