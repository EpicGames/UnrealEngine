// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SSelectionBase.h"
#include "Widgets/Views/STableRow.h"

#include "MVVMDebugViewModel.h"


namespace UE::MVVM
{

namespace Private
{
class SViewModelSelectionItem : public SMultiColumnTableRow<TSharedPtr<FMVVMViewModelDebugEntry>>
{
	SLATE_BEGIN_ARGS(SViewModelSelectionItem) {}
	SLATE_ARGUMENT(TSharedPtr<FMVVMViewModelDebugEntry>, Entry)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	TSharedPtr<FMVVMViewModelDebugEntry> Entry;
};
}

class SViewModelSelection : public SSelectionBase<TSharedPtr<FMVVMViewModelDebugEntry>, Private::SViewModelSelectionItem>
{
private:
	using Super = SSelectionBase<TSharedPtr<FMVVMViewModelDebugEntry>, Private::SViewModelSelectionItem>;

public:
	SLATE_BEGIN_ARGS(SViewModelSelection) { }
	SLATE_EVENT(FSimpleDelegate, OnSelectionChanged)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	void SetSelection(FGuid);

protected:
	virtual void GatherFilterStringsImpl(TSharedPtr<FMVVMViewModelDebugEntry> Item, TArray<FString>& OutStrings) const override;
	virtual TSharedRef<SHeaderRow> BuildHeaderRowImpl() const override;
	virtual TArray<TSharedPtr<FMVVMViewModelDebugEntry>> UpdateSourceImpl(TSharedPtr<TTextFilter<TSharedPtr<FMVVMViewModelDebugEntry>>> TextFilter) override;
};

} //namespace
