// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SViewSelection.h"

#include "MVVMDebugSnapshot.h"
#include "MVVMDebugView.h"
#include "MVVMDebugViewClass.h"

#include "Misc/TextFilter.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

//ToolWidgets

#define LOCTEXT_NAMESPACE "MVVMDebuggerViewSelection"

namespace UE::MVVM
{
namespace Private
{
namespace ViewColumnName
{
FLazyName UserWidget = "UserWidget";
FLazyName LocalPlayer = "LocalPlayer";
FLazyName WorldName = "WorldName";
}

void SViewSelectionItem::Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
{
	Entry = Args._Entry;

	SMultiColumnTableRow<TSharedPtr<FMVVMViewDebugEntry>>::Construct(
		FSuperRowType::FArguments()
		.Padding(1.0f),
		OwnerTableView
	);
}

TSharedRef<SWidget> SViewSelectionItem::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == ViewColumnName::UserWidget)
	{
		return SNew(STextBlock)
			.Text(FText::FromName(Entry->UserWidgetInstanceName));
	}
	if (ColumnName == ViewColumnName::LocalPlayer)
	{
		return SNew(STextBlock)
			.Text(FText::FromName(Entry->LocalPlayerName));
	}
	if (ColumnName == ViewColumnName::WorldName)
	{
		return SNew(STextBlock)
			.Text(FText::FromName(Entry->WorldName));
	}

	check(false);
	return SNew(STextBlock);
}

} //namespace Private


void SViewSelection::Construct(const FArguments& InArgs)
{
	Super::Construct(Super::FArguments()
		.OnSelectionChanged(InArgs._OnSelectionChanged)
		);
}


void SViewSelection::SetSelection(FGuid Id)
{
	bool bClear = true;
	if (GetSnapshot())
	{
		if (TSharedPtr<FMVVMViewDebugEntry> Found = GetSnapshot()->FindView(Id))
		{
			ListView->SetSelection(Found);
			bClear= false;
		}
	}
	if (bClear)
	{
		ListView->ClearSelection();
	}
}


TArray<TSharedPtr<FMVVMViewDebugEntry>> SViewSelection::UpdateSourceImpl(TSharedPtr<TTextFilter<TSharedPtr<FMVVMViewDebugEntry>>> InTextFilter)
{
	TArray<TSharedPtr<FMVVMViewDebugEntry>> Result;
	if (GetSnapshot())
	{
		if (!InTextFilter->GetRawFilterText().IsEmpty())
		{
			Result.Reserve(Snapshot->GetViews().Num());
			for (const TSharedPtr<FMVVMViewDebugEntry>& Item : GetSnapshot()->GetViews())
			{
				if (InTextFilter->PassesFilter(Item))
				{
					Result.Add(Item);
				}
			}
		}
		else
		{
			Result = Snapshot->GetViews();
		}
	}
	return Result;
}


TSharedRef<SHeaderRow> SViewSelection::BuildHeaderRowImpl() const
{
	return SNew(SHeaderRow)
		+ SHeaderRow::Column(Private::ViewColumnName::UserWidget)
		.FillWidth(0.5f)
		.DefaultLabel(LOCTEXT("UserWidgetColumnHeaderName", "User Widget"))
		+ SHeaderRow::Column(Private::ViewColumnName::LocalPlayer)
		.FillWidth(0.25f)
		.DefaultLabel(LOCTEXT("LocalPlayerColumnHeaderName", "Player"))
		+ SHeaderRow::Column(Private::ViewColumnName::WorldName)
		.FillWidth(0.25f)
		.DefaultLabel(LOCTEXT("WorldColumnHeaderName", "World"));
}


void SViewSelection::GatherFilterStringsImpl(TSharedPtr<FMVVMViewDebugEntry> Item, TArray<FString>& OutStrings) const
{
	OutStrings.Add(Item->UserWidgetInstanceName.ToString());
	OutStrings.Add(Item->LocalPlayerName.ToString());
	//OutStrings.Add(Item->UserWidgetAsset.ObjectPath.ToString());
	OutStrings.Add(Item->UserWidgetAsset.AssetClassPath.ToString());
	OutStrings.Add(Item->ViewClassDebugId.ToString());
}

} //namespace

#undef LOCTEXT_NAMESPACE