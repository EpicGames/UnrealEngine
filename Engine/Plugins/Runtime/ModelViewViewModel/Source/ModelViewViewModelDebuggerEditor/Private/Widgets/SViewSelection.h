// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SSelectionBase.h"
#include "Widgets/Views/STableRow.h"

#include "MVVMDebugView.h"

namespace UE::MVVM
{

namespace Private
{
class SViewSelectionItem : public SMultiColumnTableRow<TSharedPtr<FMVVMViewDebugEntry>>
{
	SLATE_BEGIN_ARGS(SViewSelectionItem) {}
	SLATE_ARGUMENT(TSharedPtr<FMVVMViewDebugEntry>, Entry)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView);
	TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName);

private:
	TSharedPtr<FMVVMViewDebugEntry> Entry;
};
}

class SViewSelection : public SSelectionBase<TSharedPtr<FMVVMViewDebugEntry>, Private::SViewSelectionItem>
{
private:
	using Super = SSelectionBase<TSharedPtr<FMVVMViewDebugEntry>, Private::SViewSelectionItem>;

public:
	SLATE_BEGIN_ARGS(SViewSelection) { }
	SLATE_EVENT(FSimpleDelegate, OnSelectionChanged)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	void SetSelection(FGuid Id);

protected:
	virtual void GatherFilterStringsImpl(TSharedPtr<FMVVMViewDebugEntry> Item, TArray<FString>& OutStrings) const override;
	virtual TSharedRef<SHeaderRow> BuildHeaderRowImpl() const override;
	virtual TArray<TSharedPtr<FMVVMViewDebugEntry>> UpdateSourceImpl(TSharedPtr<TTextFilter<TSharedPtr<FMVVMViewDebugEntry>>> TextFilter) override;
};

} //namespace
