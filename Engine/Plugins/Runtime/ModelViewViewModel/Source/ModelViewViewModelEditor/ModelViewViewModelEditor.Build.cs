// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using UnrealBuildTool;

public class ModelViewViewModelEditor : ModuleRules 
{
	public ModelViewViewModelEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"EnhancedInput",
				"BlueprintGraph",
				"Core",
				"CoreUObject",
				"EditorSubsystem",
				"FieldNotification",
				"KismetCompiler",
				"ModelViewViewModel",
				"ModelViewViewModelBlueprint",
				"PropertyPath",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AdvancedWidgets",
				"ApplicationCore",
				"AssetTools",
				"EditorWidgets",
				"GraphEditor",
				"InputCore",
				"Kismet",
				"MessageLog",
				"Projects",
				"PropertyEditor",
				"StatusBar",
				"SlateCore",
				"Slate",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
				"UMG",
				"UMGEditor",
			});
	}
}
