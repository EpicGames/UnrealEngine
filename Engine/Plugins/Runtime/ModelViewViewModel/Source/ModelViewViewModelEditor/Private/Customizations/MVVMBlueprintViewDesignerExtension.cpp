// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MVVMBlueprintViewDesignerExtension.h"

#include "Extensions/MVVMBlueprintViewExtension.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "WidgetBlueprintExtension.h"

namespace UE::MVVM
{
TSharedRef<FDesignerExtension> FBlueprintViewDesignerExtensionFactory::CreateDesignerExtension() const
{
	return StaticCastSharedRef<FBlueprintViewDesignerExtension>(MakeShared<FBlueprintViewDesignerExtension>());
}

void FBlueprintViewDesignerExtension::PreviewContentChanged(TSharedRef<SWidget> NewContent)
{
	const UWidgetBlueprint* WidgetBlueprint = Blueprint.Get();

	if (UMVVMWidgetBlueprintExtension_View* ExtensionView = WidgetBlueprint ? UWidgetBlueprintExtension::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint) : nullptr)
	{
		for (UMVVMBlueprintViewExtension* BlueprintViewExtension : ExtensionView->GetAllBlueprintExtensions())
		{
			if (ensure(BlueprintViewExtension))
			{
				BlueprintViewExtension->OnPreviewContentChanged(NewContent);
			}
		}
	}
}
}
