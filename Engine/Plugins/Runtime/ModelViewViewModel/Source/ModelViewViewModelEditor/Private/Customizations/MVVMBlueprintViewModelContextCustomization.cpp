// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBlueprintViewModelContextCustomization.h"

#include "Bindings/MVVMBindingHelper.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Dialogs/Dialogs.h"
#include "Features/IModularFeatures.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyAccessEditor.h"
#include "IPropertyTypeCustomization.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintInstancedViewModel.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMEditorSubsystem.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "View/MVVMViewModelContextResolver.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "WidgetBlueprintEditor.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SMVVMSelectViewModel.h"
#include "Widgets/SMVVMViewModelPanel.h"

#define LOCTEXT_NAMESPACE "BlueprintViewModelContextDetailCustomization"

namespace UE::MVVM
{
namespace Private
{
	FText BindingWidgetForVM_GetName()
	{
		return FText::GetEmpty();
	}

	bool BindingWidgetForVM_CanBindProperty(const FProperty* Property, const UClass* ClassToLookFor)
	{
		const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property);
		return (ObjectProperty != nullptr && ObjectProperty->PropertyClass->IsChildOf(ClassToLookFor));
	}
}

/**
 *
 */
bool FViewModelPropertyAccessEditor::CanBindProperty(FProperty* Property) const
{
	// Property == GeneratePureBindingsProperty is only to start the algo
	return ViewModelProperty != Property && (Private::BindingWidgetForVM_CanBindProperty(Property, ClassToLookFor.Get()) || Property == GeneratePureBindingsProperty);
}

bool FViewModelPropertyAccessEditor::CanBindFunction(UFunction* Function) const
{
	return Private::BindingWidgetForVM_CanBindProperty(BindingHelper::GetReturnProperty(Function), ClassToLookFor.Get());
}

bool FViewModelPropertyAccessEditor::CanBindToClass(UClass* Class) const
{
	return true;
}

void FViewModelPropertyAccessEditor::AddBinding(FName, const TArray<FBindingChainElement>& BindingChain)
{
	TStringBuilder<256> Path;
	for (const FBindingChainElement& Binding : BindingChain)
	{
		if (Path.Len() != 0)
		{
			Path << TEXT('.');
		}
		Path << Binding.Field.GetFName();
	}

	AssignToProperty->SetValue(Path.ToString());
}

bool FViewModelPropertyAccessEditor::HasValidClassToLookFor() const
{
	return ClassToLookFor.Get() != nullptr;
}

TSharedRef<SWidget> FViewModelPropertyAccessEditor::MakePropertyBindingWidget(TSharedRef<FWidgetBlueprintEditor> WidgetBlueprintEditor, FProperty* PropertyToMatch, TSharedRef<IPropertyHandle> InAssignToProperty, FName ViewModelPropertyName)
{
	UClass* SkeletonClass = WidgetBlueprintEditor->GetBlueprintObj()->SkeletonGeneratedClass.Get();
	if (!SkeletonClass)
	{
		return SNullWidget::NullWidget;
	}
	ViewModelProperty = SkeletonClass->FindPropertyByName(ViewModelPropertyName);
	AssignToProperty = InAssignToProperty;

	if (!IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
	{
		return SNullWidget::NullWidget;
	}

	GeneratePureBindingsProperty = PropertyToMatch;
	FPropertyBindingWidgetArgs Args;
	Args.Property = GeneratePureBindingsProperty;
	Args.bAllowArrayElementBindings = false;
	Args.bAllowStructMemberBindings = true;
	Args.bAllowUObjectFunctions = true;
	Args.bAllowStructFunctions = true;
	Args.bAllowNewBindings = true;
	Args.bGeneratePureBindings = true;

	Args.CurrentBindingText.BindStatic(&Private::BindingWidgetForVM_GetName);
	Args.OnCanBindPropertyWithBindingChain = FOnCanBindPropertyWithBindingChain::CreateLambda([this](FProperty* InProperty, TConstArrayView<FBindingChainElement> InBindingChain)
	{
		return CanBindProperty(InProperty);
	});
	Args.OnCanBindFunction.BindRaw(this, &FViewModelPropertyAccessEditor::CanBindFunction);
	Args.OnCanBindToClass.BindRaw(this, &FViewModelPropertyAccessEditor::CanBindToClass);
	Args.OnAddBinding.BindRaw(this, &FViewModelPropertyAccessEditor::AddBinding);

	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
	TSharedRef<SWidget> Result = PropertyAccessEditor.MakePropertyBindingWidget(WidgetBlueprintEditor->GetBlueprintObj(), Args);
	Result->SetEnabled(MakeAttributeRaw(this, &FViewModelPropertyAccessEditor::HasValidClassToLookFor));
	return Result;
}

/**
 * 
 */
namespace Private
{
	FMVVMBlueprintViewModelContext* GetViewModelContext(TSharedRef<IPropertyHandle> PropertyHandle)
	{
		ensure(CastField<FStructProperty>(PropertyHandle->GetProperty()) && CastField<FStructProperty>(PropertyHandle->GetProperty())->Struct == FMVVMBlueprintViewModelContext::StaticStruct());
		void* Buffer = nullptr;
		if (PropertyHandle->GetValueData(Buffer) == FPropertyAccess::Success)
		{
			return reinterpret_cast<FMVVMBlueprintViewModelContext*>(Buffer);
		}
		return nullptr;
	}

	class FResolverClassFilter : public IClassViewerFilter
	{
	public:
		const UClass* ViewModelClass = nullptr;

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			if (ViewModelClass && InClass->IsChildOf(UMVVMViewModelContextResolver::StaticClass()))
			{
				return InClass->GetDefaultObject<UMVVMViewModelContextResolver>()->DoesSupportViewModelClass(ViewModelClass);
			}

			return false;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InBlueprint, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			if (ViewModelClass && InBlueprint->IsChildOf(UMVVMViewModelContextResolver::StaticClass()))
			{
				// Load the Blueprint
				FSoftObjectPath BlueprintPath = FSoftObjectPath(InBlueprint->GetClassPathName());
				if (UClass* LoadedClass = Cast<UClass>(BlueprintPath.TryLoad()))
				{
					return LoadedClass->GetDefaultObject<UMVVMViewModelContextResolver>()->DoesSupportViewModelClass(ViewModelClass);
				}
			}
			return false;
		}
	};

}

FBlueprintViewModelContextDetailCustomization::FBlueprintViewModelContextDetailCustomization(TWeakPtr<FWidgetBlueprintEditor> InEditor)
	: WidgetBlueprintEditor(InEditor)
{}


void FBlueprintViewModelContextDetailCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);

	ContextHandle = PropertyHandle;
	FMVVMBlueprintViewModelContext* ContextPtr = Private::GetViewModelContext(PropertyHandle);
	if (ContextPtr == nullptr)
	{
		return;
	}

	UClass* ViewModelClass = nullptr;
	FName ViewModelPropertyName = ContextPtr->GetViewModelName();
	bool bCanEdit = ContextPtr->bCanEdit;

	// Reset the value to what the user expect to see. It is not used in by the compiler.
	if (!ContextPtr->bOverrideForceExecuteBindingsOnSetSource)
	{
		ContextPtr->bForceExecuteBindingsOnSetSource = GetDefault<UMVVMDeveloperProjectSettings>()->bForceExecuteBindingsOnSetSource;
	}
	
	NotifyFieldValueClassHandle = PropertyHandle->GetChildHandle(TEXT("NotifyFieldValueClass"), false);
	PropertyPathHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, ViewModelPropertyPath), false);
	CreationTypeHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, CreationType), false);
	ViewModelNameHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, ViewModelName), false);
	OptionalHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, bOptional), false);
	UseAsInterfaceHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, bUseAsInterface), false);
	CreateSetterFunctionHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, bCreateSetterFunction), false);
	ForceExecuteBindingsOnSetSourceHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, bForceExecuteBindingsOnSetSource), false);
	ResolverHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, Resolver), false);

	if (ensure(NotifyFieldValueClassHandle))
	{
		UObject* Object = nullptr;
		if (NotifyFieldValueClassHandle->GetValue(Object) == FPropertyAccess::Success)
		{
			ViewModelClass = Cast<UClass>(Object);
			if (ViewModelClass)
			{
				AllowedCreationTypes = GetAllowedContextCreationType(ViewModelClass);
			}
			PropertyAccessEditor.ClassToLookFor = ViewModelClass;
		}
		NotifyFieldValueClassHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FBlueprintViewModelContextDetailCustomization::HandleClassChanged));
	}

	if (ensure(CreationTypeHandle))
	{
		CreationTypeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FBlueprintViewModelContextDetailCustomization::HandleCreationTypeChanged));
	}

	if (ensure(CreateSetterFunctionHandle))
	{
		CreateSetterFunctionHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FBlueprintViewModelContextDetailCustomization::HandleCreateSetterFunctionChanged));
	}

	if (ensure(ViewModelNameHandle))
	{
		IDetailPropertyRow& PropertyRow = ChildBuilder.AddProperty(ViewModelNameHandle.ToSharedRef());
		TSharedPtr<SWidget> NameWidget, ValueWidget;
		PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget);
		PropertyRow.CustomWidget()
			.NameContent()
			[
				NameWidget.ToSharedRef()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SEditableTextBox)
					.Text(this, &FBlueprintViewModelContextDetailCustomization::GetViewModelNameValueAsText)
					.Font(CustomizationUtils.GetRegularFont())
					.SelectAllTextWhenFocused(true)
					.ClearKeyboardFocusOnCommit(false)
					.OnTextCommitted(this, &FBlueprintViewModelContextDetailCustomization::HandleNameTextCommitted)
					.OnVerifyTextChanged(this, &FBlueprintViewModelContextDetailCustomization::HandleNameVerifyTextChanged)
					.SelectAllTextOnCommit(true)
					.IsEnabled(this, &FBlueprintViewModelContextDetailCustomization::CanRenameViewModel)
				]
			];
	}

	if (ensure(NotifyFieldValueClassHandle))
	{
		IDetailPropertyRow& PropertyRow = ChildBuilder.AddProperty(NotifyFieldValueClassHandle.ToSharedRef())
			.IsEnabled(bCanEdit)
			.Visibility(MakeAttributeLambda([ContextPtr]() { return ContextPtr->InstancedViewModel != nullptr ? EVisibility::Collapsed : EVisibility::Visible; }));

		TSharedPtr<SWidget> NameWidget, ValueWidget;
		PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget);
		PropertyRow.CustomWidget()
			.NameContent()
			[
				NameWidget.ToSharedRef()
			]
			.ValueContent()
			[
				SAssignNew(NotifyFieldValueClassComboButton, SComboButton)
					.IsEnabled(bCanEdit)
					.OnGetMenuContent(this, &FBlueprintViewModelContextDetailCustomization::HandleClassGetMenuContent)
					.ButtonContent()
					[
						SNew(STextBlock)
							.Text(this, &FBlueprintViewModelContextDetailCustomization::GetClassName)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
	}

	if (ContextPtr->InstancedViewModel == nullptr)
	{
		if (ensure(CreationTypeHandle))
		{
			IDetailPropertyRow& PropertyRow = ChildBuilder.AddProperty(CreationTypeHandle.ToSharedRef())
				.IsEnabled(bCanEdit);

			TSharedPtr<SWidget> NameWidget, ValueWidget;
			PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget);
			PropertyRow.CustomWidget()
				.NameContent()
				[
					NameWidget.ToSharedRef()
				]
				.ValueContent()
				[
					SNew(SComboButton)
					.ContentPadding(FMargin(4.f, 0.f))
					.OnGetMenuContent(this, &FBlueprintViewModelContextDetailCustomization::CreateExecutionTypeMenuContent)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &FBlueprintViewModelContextDetailCustomization::GetCreationTypeValue)
						.ToolTipText(this, &FBlueprintViewModelContextDetailCustomization::GetExecutionTypeValueToolTip)
					]
				];
		}

		TSharedPtr<IPropertyHandle> GlobalViewModelIdentifierlHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, GlobalViewModelIdentifier), false);
		if (ensure(GlobalViewModelIdentifierlHandle))
		{
			ChildBuilder.AddProperty(GlobalViewModelIdentifierlHandle.ToSharedRef())
				.IsEnabled(bCanEdit)
				.Visibility(MakeAttributeLambda([ContextPtr]()
					{
						bool bResult = ContextPtr->CreationType == EMVVMBlueprintViewModelContextCreationType::GlobalViewModelCollection;
						return bResult ? EVisibility::Visible : EVisibility::Collapsed;
					}));
		}
		
		if (ensure(PropertyPathHandle))
		{
			if (TSharedPtr<FWidgetBlueprintEditor> SharedWidgetBlueprintEditor = WidgetBlueprintEditor.Pin())
			{
				IDetailPropertyRow& PropertyRow = ChildBuilder.AddProperty(PropertyPathHandle.ToSharedRef())
					.IsEnabled(bCanEdit)
					.Visibility(MakeAttributeLambda([ContextPtr]()
						{
							bool bResult = ContextPtr->CreationType == EMVVMBlueprintViewModelContextCreationType::PropertyPath;
							return bResult ? EVisibility::Visible : EVisibility::Collapsed;
						}));

				TSharedPtr<SWidget> NameWidget, ValueWidget;
				PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget);
				PropertyRow.CustomWidget()
					.NameContent()
					[
						NameWidget.ToSharedRef()
					]
					.ValueContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(1.f)
						[
							ValueWidget.ToSharedRef()
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							PropertyAccessEditor.MakePropertyBindingWidget(SharedWidgetBlueprintEditor.ToSharedRef(), NotifyFieldValueClassHandle->GetProperty(), PropertyPathHandle.ToSharedRef(), ViewModelPropertyName)
						]
					];
			}
		}

		if (ensure(ResolverHandle))
		{
			TSharedRef<Private::FResolverClassFilter> ClassFilter = MakeShared<Private::FResolverClassFilter>();
			ClassFilter->ViewModelClass = ViewModelClass;
			TSharedRef<FPropertyRestriction> Restriction = MakeShared<FPropertyRestriction>(LOCTEXT("ResolverPropertyRestriction", "Resolver Property Restriction"));
			Restriction->AddClassFilter(ClassFilter);

			ResolverHandle->AddRestriction(Restriction);

			ChildBuilder.AddProperty(ResolverHandle.ToSharedRef())
				.IsEnabled(bCanEdit)
				.Visibility(MakeAttributeLambda([ContextPtr]()
					{
						bool bResult = ContextPtr->CreationType == EMVVMBlueprintViewModelContextCreationType::Resolver;
						return bResult ? EVisibility::Visible : EVisibility::Collapsed;
					}));
		}

		if (GetDefault<UMVVMDeveloperProjectSettings>()->bAllowGeneratedViewModelSetter)
		{
			if (ensure(CreateSetterFunctionHandle))
			{
				ChildBuilder.AddProperty(CreateSetterFunctionHandle.ToSharedRef())
					.IsEnabled(MakeAttributeLambda([bCanEdit, ContextPtr]()
						{
							bool bResult = ContextPtr->CreationType != EMVVMBlueprintViewModelContextCreationType::Manual && ContextPtr->CreationType != EMVVMBlueprintViewModelContextCreationType::Context;
							return bResult && bCanEdit;
						}));
			}
		}

		TSharedPtr<IPropertyHandle> CreateGetterFunctionHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, bCreateGetterFunction), false);
		if (ensure(CreateGetterFunctionHandle))
		{
			ChildBuilder.AddProperty(CreateGetterFunctionHandle.ToSharedRef())
				.IsEnabled(bCanEdit);
		}

		TSharedPtr<IPropertyHandle> ExposeInstanceInEditorHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, bExposeInstanceInEditor), false);
		if (ensure(ExposeInstanceInEditorHandle))
		{
			ChildBuilder.AddProperty(ExposeInstanceInEditorHandle.ToSharedRef())
				.IsEnabled(bCanEdit)
				.Visibility(MakeAttributeLambda([ContextPtr]()
					{
						bool bResult = ContextPtr->CreationType == EMVVMBlueprintViewModelContextCreationType::CreateInstance;
						return bResult ? EVisibility::Visible : EVisibility::Collapsed;
					}));
		}

		TSharedPtr<IPropertyHandle> GlobalViewModelCollectionUpdateHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, bGlobalViewModelCollectionUpdate), false);
		if (ensure(GlobalViewModelCollectionUpdateHandle))
		{
			ChildBuilder.AddProperty(GlobalViewModelCollectionUpdateHandle.ToSharedRef())
				.IsEnabled(bCanEdit)
				.Visibility(MakeAttributeLambda([ContextPtr]()
					{
						bool bResult = ContextPtr->CreationType == EMVVMBlueprintViewModelContextCreationType::GlobalViewModelCollection;
						return bResult ? EVisibility::Visible : EVisibility::Collapsed;
					}));
		}

		if (ensure(OptionalHandle))
		{
			ChildBuilder.AddProperty(OptionalHandle.ToSharedRef())
				.IsEnabled(MakeAttributeLambda([bCanEdit, ContextPtr]()
					{
						bool bResult = ContextPtr->CreationType == EMVVMBlueprintViewModelContextCreationType::GlobalViewModelCollection
						 || ContextPtr->CreationType == EMVVMBlueprintViewModelContextCreationType::PropertyPath
						 || ContextPtr->CreationType == EMVVMBlueprintViewModelContextCreationType::Resolver;
						return bResult && bCanEdit;
					}));
		}

		if (ensure(UseAsInterfaceHandle))
		{
			ChildBuilder.AddProperty(UseAsInterfaceHandle.ToSharedRef())
				.IsEnabled(TAttribute<bool>(this, &FBlueprintViewModelContextDetailCustomization::IsUseAsInterfaceAvailable))
				.ToolTip(LOCTEXT("UseAsInterfaceToolTip", "True to use as interface, False to use as property.\nIt can only be enabled if there is no other interface of the same type, and if the default name for the viewmodel is available.\nIt is not possible to rename the viewmodel if the option is enabled."))
				.Visibility(MakeAttributeLambda([]()
					{
						IConsoleVariable* CVarSupportUseAsInterfaceSetting = IConsoleManager::Get().FindConsoleVariable(TEXT("MVVM.SupportUseAsInterfaceSetting")); ensure(CVarSupportUseAsInterfaceSetting);
						if (CVarSupportUseAsInterfaceSetting)
						{
							return CVarSupportUseAsInterfaceSetting->GetBool() ? EVisibility::Visible : EVisibility::Collapsed;
						}
						return EVisibility::Collapsed;
					}));

			UseAsInterfaceHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FBlueprintViewModelContextDetailCustomization::HandleUseAsInterfaceChanged));
		}

		if (ensure(ForceExecuteBindingsOnSetSourceHandle))
		{
			ChildBuilder.AddProperty(ForceExecuteBindingsOnSetSourceHandle.ToSharedRef())
				.IsEnabled(bCanEdit)
				.Visibility(MakeAttributeLambda([ContextPtr]()
					{
						bool bResult = ContextPtr->bCreateSetterFunction;
						return bResult ? EVisibility::Visible : EVisibility::Collapsed;
					}));
		}
	}
	else
	{
		TSharedPtr<IPropertyHandle> InstancedViewModelHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewModelContext, InstancedViewModel), false);
		if (ensure(InstancedViewModelHandle))
		{
			ChildBuilder.AddProperty(InstancedViewModelHandle.ToSharedRef())
				.IsEnabled(bCanEdit);
		}
	}
}

void FBlueprintViewModelContextDetailCustomization::HandleClassChanged()
{
	UObject* Object = nullptr;
	AllowedCreationTypes.Reset();
	PropertyAccessEditor.ClassToLookFor.Reset();
	if (NotifyFieldValueClassHandle->GetValue(Object) == FPropertyAccess::Success)
	{
		if (UClass* ViewModelClass = Cast<UClass>(Object))
		{
			PropertyAccessEditor.ClassToLookFor = ViewModelClass;
			AllowedCreationTypes = GetAllowedContextCreationType(ViewModelClass);
		}
	}
}

FText FBlueprintViewModelContextDetailCustomization::GetClassName() const
{
	UObject* Object = nullptr;
	FPropertyAccess::Result ValueResult = NotifyFieldValueClassHandle->GetValue(Object);
	if (ValueResult == FPropertyAccess::Success)
	{
		UClass* ViewModelClass = Cast<UClass>(Object);
		if (ViewModelClass)
		{
			return ViewModelClass->GetDisplayNameText();
		}
		if (Object)
		{
			return FText::FromName(Object->GetFName());
		}
	}
	else if (ValueResult == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}
	return LOCTEXT("None", "None");
}

void FBlueprintViewModelContextDetailCustomization::HandleCreationTypeChanged()
{
	uint8 NewValue = 0;
	if (CreationTypeHandle->GetValue(NewValue) == FPropertyAccess::Success)
	{
		if (FMVVMBlueprintViewModelContext* ContextPtr = Private::GetViewModelContext(ContextHandle.ToSharedRef()))
		{
			const EMVVMBlueprintViewModelContextCreationType CreationType = (EMVVMBlueprintViewModelContextCreationType)NewValue;
			const bool bIsManual = CreationType == EMVVMBlueprintViewModelContextCreationType::Manual;
			const bool bIsAlwaysOptional = CreationType == EMVVMBlueprintViewModelContextCreationType::Manual || CreationType == EMVVMBlueprintViewModelContextCreationType::Context;
			if (ContextPtr->bOptional != bIsAlwaysOptional)
			{
				OptionalHandle->SetValue(bIsAlwaysOptional);
			}
			if (ContextPtr->bCreateSetterFunction != bIsManual)
			{
				CreateSetterFunctionHandle->SetValue(bIsManual);
			}

			// Set default resolver only if not already set to a valid value
			if (CreationType == EMVVMBlueprintViewModelContextCreationType::Resolver)
			{
				UObject* ExistingResolver = nullptr;
				if (ResolverHandle->GetValue(ExistingResolver) == FPropertyAccess::Fail || ExistingResolver == nullptr)
				{
					TObjectPtr<UMVVMViewModelContextResolver> NewResolver = ContextPtr->CreateDefaultResolver(GetTransientPackage());

					// Bypass SetValue, Resolver is set to Instanced which will block it
					FString PropertyText = NewResolver ? NewResolver->GetPathName() : TEXT("None");
					ResolverHandle->SetValueFromFormattedString(PropertyText);
				}
			}
		}
	}
}

void FBlueprintViewModelContextDetailCustomization::HandleCreateSetterFunctionChanged()
{
	bool bNewCreateSetterFunctionHandle = false;
	if (CreateSetterFunctionHandle->GetValue(bNewCreateSetterFunctionHandle) == FPropertyAccess::Success)
	{
		if (FMVVMBlueprintViewModelContext* ContextPtr = Private::GetViewModelContext(ContextHandle.ToSharedRef()))
		{
			if (ContextPtr->bOverrideForceExecuteBindingsOnSetSource && !bNewCreateSetterFunctionHandle)
			{
				ContextPtr->bOverrideForceExecuteBindingsOnSetSource = false;
				ForceExecuteBindingsOnSetSourceHandle->SetValue(GetDefault<UMVVMDeveloperProjectSettings>()->bForceExecuteBindingsOnSetSource);
			}
		}
	}
}

TSharedRef<SWidget> FBlueprintViewModelContextDetailCustomization::CreateExecutionTypeMenuContent()
{
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr, nullptr, true);

	const UEnum* EnumCreationType = StaticEnum<EMVVMBlueprintViewModelContextCreationType>();
	for (EMVVMBlueprintViewModelContextCreationType Type : AllowedCreationTypes)
	{
		int32 Index = EnumCreationType->GetIndexByValue((int64)Type);
		MenuBuilder.AddMenuEntry(
			EnumCreationType->GetDisplayNameTextByIndex(Index),
			EnumCreationType->GetToolTipTextByIndex(Index),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateLambda([this, Type]()
					{
						uint8 Value = static_cast<uint8>(Type);
						CreationTypeHandle->SetValue(Value);
					})
			)
		);
	}

	return MenuBuilder.MakeWidget();
}

FText FBlueprintViewModelContextDetailCustomization::GetCreationTypeValue() const
{
	uint8 Value = 0;
	if (CreationTypeHandle->GetValue(Value) == FPropertyAccess::Result::Success)
	{
		return StaticEnum<EMVVMBlueprintViewModelContextCreationType>()->GetDisplayNameTextByValue((int64)Value);
	}
	return FText::GetEmpty();
}

FText FBlueprintViewModelContextDetailCustomization::GetExecutionTypeValueToolTip() const
{
	uint8 Value = 0;
	if (CreationTypeHandle->GetValue(Value) == FPropertyAccess::Result::Success)
	{
		UEnum* EnumCreationType = StaticEnum<EMVVMBlueprintViewModelContextCreationType>();
		return EnumCreationType->GetToolTipTextByIndex(EnumCreationType->GetIndexByValue((int64)Value));
	}
	return FText::GetEmpty();
}

bool FBlueprintViewModelContextDetailCustomization::IsUseAsInterfaceAvailable() const
{
	UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor.Pin()->GetWidgetBlueprintObj();
	FMVVMBlueprintViewModelContext* ContextPtr = Private::GetViewModelContext(ContextHandle.ToSharedRef());
	if (ContextPtr == nullptr)
	{
		return false;
	}
	const UClass* ViewModelClass = ContextPtr->GetViewModelClass();
	if (ViewModelClass == nullptr)
	{
		return false;
	}

	UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	check(EditorSubsystem);
	UMVVMBlueprintView* View = EditorSubsystem->GetView(WidgetBlueprint);
	if (View == nullptr)
	{
		return false;
	}

	const FGuid ViewModelContextId = ContextPtr->GetViewModelId();
	const TArrayView<const FMVVMBlueprintViewModelContext> ViewModels = View->GetViewModels();

	bool bHasInterfaceOfSameType = false;
	if (ViewModels.FindByPredicate([&ViewModelClass, &ViewModelContextId](FMVVMBlueprintViewModelContext ViewModel) { return ViewModel.GetViewModelClass() == ViewModelClass && ViewModel.bUseAsInterface && ViewModel.GetViewModelId() != ViewModelContextId; }))
	{
		bHasInterfaceOfSameType = true;
	}

	bool bHasAnotherViewModelUsingTheName = false;
	const FString DefaultViewModelName = UMVVMEditorSubsystem::GetDefaultViewModelName(ViewModelClass);
	if (ViewModels.FindByPredicate([&DefaultViewModelName, &ViewModelContextId](FMVVMBlueprintViewModelContext ViewModel) { return ViewModel.GetViewModelName() == DefaultViewModelName && ViewModel.GetViewModelId() != ViewModelContextId; }))
	{
		bHasAnotherViewModelUsingTheName = true;
	}

	return !bHasInterfaceOfSameType && !bHasAnotherViewModelUsingTheName;
}

FText FBlueprintViewModelContextDetailCustomization::GetViewModelNameValueAsText() const
{
	check(ViewModelNameHandle.IsValid());
	FText Result;
	ViewModelNameHandle->GetValueAsFormattedText(Result);
	return Result;
}

bool FBlueprintViewModelContextDetailCustomization::CanRenameViewModel() const
{
	if (FMVVMBlueprintViewModelContext* ContextPtr = Private::GetViewModelContext(ContextHandle.ToSharedRef()))
	{
		return ContextPtr->CanRename() && ContextPtr->GetViewModelId().IsValid();
	}

	return false;
}

TSharedRef<SWidget> FBlueprintViewModelContextDetailCustomization::HandleClassGetMenuContent()
{
	return SNew(SBox)
		.WidthOverride(600)
		.HeightOverride(500)
		[
			SNew(SMVVMSelectViewModel, WidgetBlueprintEditor.Pin()->GetWidgetBlueprintObj())
			.OnCancel(this, &FBlueprintViewModelContextDetailCustomization::HandleClassCancelMenu)
			.OnViewModelCommitted(this, &FBlueprintViewModelContextDetailCustomization::HandleClassCommitted)
			.DisallowedClassFlags(CLASS_HideDropDown | CLASS_Hidden | CLASS_Deprecated | CLASS_NotPlaceable)
		];
}

void FBlueprintViewModelContextDetailCustomization::HandleClassCancelMenu()
{
	if (NotifyFieldValueClassComboButton)
	{
		NotifyFieldValueClassComboButton->SetIsOpen(false, false);
	}
}

void FBlueprintViewModelContextDetailCustomization::HandleClassCommitted(const UClass* SelectedClass)
{
	if (NotifyFieldValueClassComboButton)
	{
		NotifyFieldValueClassComboButton->SetIsOpen(false, false);
	}
	bool bReparent = false;
	FName ViewModelName;
	{
		UObject* Object = nullptr;
		FPropertyAccess::Result ClassValueResult = NotifyFieldValueClassHandle->GetValue(Object);
		UClass* PreviousClass = Cast<UClass>(Object);
		FPropertyAccess::Result NameValueResult = ViewModelNameHandle->GetValue(ViewModelName);
		if (ClassValueResult == FPropertyAccess::Success && SelectedClass && SelectedClass != PreviousClass
			&& NameValueResult == FPropertyAccess::Success && !ViewModelName.IsNone())
		{
			const FText Title = LOCTEXT("ReparentTitle", "Reparent Viewmodel");
			const FText Message = LOCTEXT("ReparentWarning", "Reparenting the viewmodel may cause data loss. Continue reparenting?");

			// Warn the user that this may result in data loss
			FSuppressableWarningDialog::FSetupInfo Info(Message, Title, "Warning_ReparentTitle");
			Info.ConfirmText = LOCTEXT("ReparentYesButton", "Reparent");
			Info.CancelText = LOCTEXT("ReparentNoButton", "Cancel");
			Info.CheckBoxText = FText::GetEmpty();	// not suppressible

			FSuppressableWarningDialog ReparentBlueprintDlg(Info);
			if (ReparentBlueprintDlg.ShowModal() == FSuppressableWarningDialog::Confirm)
			{
				bReparent = true;
			}
		}
	}

	if (bReparent)
	{
		UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		check(EditorSubsystem);
		FText ErrorMessage;
		EditorSubsystem->ReparentViewModel(WidgetBlueprintEditor.Pin()->GetWidgetBlueprintObj(), ViewModelName, SelectedClass, ErrorMessage);
	}
}

namespace Private
{
bool VerifyViewModelName(TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor, TSharedPtr<IPropertyHandle> ViewModelNameHandle, const FText& RenameTo, bool bCommit, FText& OutErrorMessage)
{
	if (!WidgetBlueprintEditor || !ViewModelNameHandle)
	{
		return false;
	}

	if (RenameTo.IsEmptyOrWhitespace())
	{
		OutErrorMessage = LOCTEXT("EmptyViewModelName", "Empty viewmodel name.");
		return false;
	}

	const FString& NewNameString = RenameTo.ToString();
	if (NewNameString.Len() >= NAME_SIZE)
	{
		OutErrorMessage = LOCTEXT("ViewModelNameTooLong", "Viewmodel name is too long.");
		return false;
	}

	FString GeneratedName = SlugStringForValidName(NewNameString);
	if (NewNameString != GeneratedName)
	{
		OutErrorMessage = LOCTEXT("ViewModelHasInvalidChar", "ViewModel name has an invalid character.");
		return false;
	}

	FName CurrentViewModelName;
	if (ViewModelNameHandle->GetValue(CurrentViewModelName) != FPropertyAccess::Success)
	{
		OutErrorMessage = LOCTEXT("MultipleViewModel", "Can't edit multiple viewmodel name.");
		return false;
	}

	const FName GeneratedFName(*GeneratedName);
	check(GeneratedFName.IsValidXName(INVALID_OBJECTNAME_CHARACTERS));

	if (UWidgetBlueprint* WidgetBP = WidgetBlueprintEditor->GetWidgetBlueprintObj())
	{
		if (bCommit)
		{
			return GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->RenameViewModel(WidgetBP, CurrentViewModelName, *NewNameString, OutErrorMessage);
		}
		else
		{
			return GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->VerifyViewModelRename(WidgetBP, CurrentViewModelName, *NewNameString, OutErrorMessage);
		}
	}
	return false;
}
}

void FBlueprintViewModelContextDetailCustomization::HandleNameTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		FText OutErrorMessage;
		Private::VerifyViewModelName(WidgetBlueprintEditor.Pin(), ViewModelNameHandle, NewText, true, OutErrorMessage);
	}
}

bool FBlueprintViewModelContextDetailCustomization::HandleNameVerifyTextChanged(const FText& NewText, FText& OutError) const
{
	return Private::VerifyViewModelName(WidgetBlueprintEditor.Pin(), ViewModelNameHandle, NewText, false, OutError);
}

void FBlueprintViewModelContextDetailCustomization::HandleUseAsInterfaceChanged()
{
	bool bUseAsInterfaceHandle = false;
	if (UseAsInterfaceHandle->GetValue(bUseAsInterfaceHandle) == FPropertyAccess::Success)
	{
		if (FMVVMBlueprintViewModelContext* ContextPtr = Private::GetViewModelContext(ContextHandle.ToSharedRef()))
		{
			const FString DefaultViewModelName = UMVVMEditorSubsystem::GetDefaultViewModelName(ContextPtr->GetViewModelClass());
			if (ContextPtr->GetViewModelName() != DefaultViewModelName && bUseAsInterfaceHandle)
			{
				FText OutErrorMessage;
				FText NewName = FText::FromString(DefaultViewModelName);
				if (!Private::VerifyViewModelName(WidgetBlueprintEditor.Pin(), ViewModelNameHandle, NewName, true, OutErrorMessage))
				{
					//Unable to set the name as required, revert the change.
					UseAsInterfaceHandle->SetValue(false);
				}
			}
		}
	}
}

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
