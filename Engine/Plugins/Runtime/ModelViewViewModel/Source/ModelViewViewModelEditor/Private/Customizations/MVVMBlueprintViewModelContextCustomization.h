// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"

class FWidgetBlueprintEditor;
struct FBindingChainElement;
class SComboButton;
class SWidget;
enum class EMVVMBlueprintViewModelContextCreationType : uint8;

namespace UE::MVVM
{

class FViewModelPropertyAccessEditor
{
public:
	TWeakObjectPtr<UClass> ClassToLookFor;

	TSharedRef<SWidget> MakePropertyBindingWidget(TSharedRef<FWidgetBlueprintEditor> WidgetBlueprintEditor, FProperty* PropertyToMatch, TSharedRef<IPropertyHandle> AssignToProperty, FName ViewModelPropertyName);

private:
	bool CanBindProperty(FProperty* Property) const;
	bool CanBindFunction(UFunction* Function) const;
	bool CanBindToClass(UClass* Class) const;
	bool HasValidClassToLookFor() const;
	void AddBinding(FName, const TArray<FBindingChainElement>& BindingChain);

	FProperty* GeneratePureBindingsProperty = nullptr;
	FProperty* ViewModelProperty = nullptr;
	TSharedPtr<IPropertyHandle> AssignToProperty = nullptr;
};

class FBlueprintViewModelContextDetailCustomization : public IPropertyTypeCustomization
{
public:
	FBlueprintViewModelContextDetailCustomization(TWeakPtr<FWidgetBlueprintEditor> InEditor);

	//~ IDetailCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TWeakPtr<FWidgetBlueprintEditor> InEditor)
	{
		return MakeShared<FBlueprintViewModelContextDetailCustomization>(InEditor);
	}

private:
	TSharedRef<SWidget> CreateExecutionTypeMenuContent();
	FText GetCreationTypeValue() const;
	FText GetExecutionTypeValueToolTip() const;
	FText GetClassName() const;
	void HandleClassChanged();
	void HandleCreationTypeChanged();
	void HandleCreateSetterFunctionChanged();
	void HandleUseAsInterfaceChanged();
	TSharedRef<SWidget> HandleClassGetMenuContent();
	void HandleClassCancelMenu();
	void HandleClassCommitted(const UClass* SelectedClass);
	FText GetViewModelNameValueAsText() const;
	bool CanRenameViewModel() const;
	void HandleNameTextCommitted(const FText& NewText, ETextCommit::Type /*CommitInfo*/);
	bool HandleNameVerifyTextChanged(const FText& Text, FText& OutError) const;
	bool IsUseAsInterfaceAvailable() const;

	FViewModelPropertyAccessEditor PropertyAccessEditor;
	TWeakPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor;
	TSharedPtr<SComboButton> NotifyFieldValueClassComboButton;
	TSharedPtr<IPropertyHandle> ContextHandle;
	TSharedPtr<IPropertyHandle> NotifyFieldValueClassHandle;
	TSharedPtr<IPropertyHandle> PropertyPathHandle;
	TSharedPtr<IPropertyHandle> CreationTypeHandle;
	TSharedPtr<IPropertyHandle> ViewModelNameHandle;
	TSharedPtr<IPropertyHandle> OptionalHandle;
	TSharedPtr<IPropertyHandle> UseAsInterfaceHandle;
	TSharedPtr<IPropertyHandle> CreateSetterFunctionHandle;
	TSharedPtr<IPropertyHandle> ForceExecuteBindingsOnSetSourceHandle;
	TSharedPtr<IPropertyHandle> ResolverHandle;
	TArray<EMVVMBlueprintViewModelContextCreationType> AllowedCreationTypes;
};

} // namespace UE::MVVM
