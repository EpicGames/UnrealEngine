// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MVVMClipboardExtension.h"

#include "Exporters/Exporter.h"
#include "Extensions/MVVMBlueprintViewExtension.h"
#include "Extensions/MVVMViewBlueprintPanelWidgetExtension.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "WidgetBlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "MVVMClipboardExtension"

namespace UE::MVVM
{

void FClipboardExtension::AppendToClipboard(const UWidget* Widget, const FExportArgs& ExportArgs)
{
	check(ExportArgs.Out);

	if (UWidgetBlueprint* WidgetBlueprint = FWidgetBlueprintEditorUtils::GetWidgetBlueprintFromWidget(Widget))
	{
		if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionViewPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint))
		{
			for (UMVVMBlueprintViewExtension* Extension : MVVMExtensionViewPtr->GetBlueprintExtensionsForWidget(Widget->GetFName()))
			{
				UExporter::ExportToOutputDevice(ExportArgs.Context, Extension, ExportArgs.Exporter, *ExportArgs.Out, ExportArgs.FileType, ExportArgs.Indent, ExportArgs.PortFlags, ExportArgs.bSelectedOnly, ExportArgs.ExportRootScope);
			}
		}
	}
}

bool FClipboardExtension::CanAppendToClipboard(const UWidget* Widget) const
{
	if (UWidgetBlueprint* WidgetBlueprint = FWidgetBlueprintEditorUtils::GetWidgetBlueprintFromWidget(Widget))
	{
		UMVVMWidgetBlueprintExtension_View* MVVMExtensionViewPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
		return MVVMExtensionViewPtr && MVVMExtensionViewPtr->GetBlueprintExtensionsForWidget(Widget->GetFName()).Num() > 0;
	}
	return false;
}

void FClipboardExtension::ProcessImportedText(const UWidgetBlueprint* WidgetBlueprint, const FString& TextToImport, UPackage*& TempPackage)
{
	Factory.NewExtensions.Empty();
	if (UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint))
	{
		Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);
	}
}

bool FClipboardExtension::CanImportFromClipboard(const UWidget* Widget) const
{
	return Factory.NewExtensions.Num() > 0;
}

void FClipboardExtension::ImportDataToWidget(const UWidget* Widget, FName OldWidgetName)
{
	if (UWidgetBlueprint* WidgetBlueprint = FWidgetBlueprintEditorUtils::GetWidgetBlueprintFromWidget(Widget))
	{
		if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionViewPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint))
		{
			for (int32 Index = Factory.NewExtensions.Num() - 1; Index >= 0; Index--)
			{
				if (UMVVMBlueprintViewExtension* NewExtension = Factory.NewExtensions[Index].Get())
				{
					if (NewExtension->WidgetRenamed(OldWidgetName, Widget->GetFName()))
					{
						NewExtension->Rename(nullptr, MVVMExtensionViewPtr, NewExtension->GetFlags());
						MVVMExtensionViewPtr->Modify();
						NewExtension->Modify();
						FMVVMExtensionItem CopiedExtension;
						CopiedExtension.ExtensionObj = NewExtension;
						CopiedExtension.WidgetName = Widget->GetFName();
						MVVMExtensionViewPtr->BlueprintExtensions.Add(CopiedExtension);
						Factory.NewExtensions.RemoveAtSwap(Index);
					}
				}
			}
		}
	}
}

bool FClipboardExtension::CanWidgetAcceptPaste(const UWidget* Widget) const
{
	bool bCanWidgetAcceptPaste = true;

	if (const UWidgetBlueprint* WidgetBlueprint = FWidgetBlueprintEditorUtils::GetWidgetBlueprintFromWidget(Widget))
	{
		if (const UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint))
		{
			const TArray<UMVVMBlueprintViewExtension*> BlueprintExtensions = ExtensionView->GetBlueprintExtensionsForWidget(Widget->GetFName());
			bCanWidgetAcceptPaste = !BlueprintExtensions.ContainsByPredicate([](const UMVVMBlueprintViewExtension* BlueprintExtension) -> bool
			{
				return BlueprintExtension->IsA<UMVVMBlueprintViewExtension_PanelWidget>();
			});
		}
	}

	return bCanWidgetAcceptPaste;
}

FClipboardExtension::FExtensionTextFactory::FExtensionTextFactory()
	: FCustomizableTextObjectFactory(GWarn)
{
}

bool FClipboardExtension::FExtensionTextFactory::CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const
{
	return ObjectClass && ObjectClass->IsChildOf(UMVVMBlueprintViewExtension::StaticClass());
}

void FClipboardExtension::FExtensionTextFactory::ProcessConstructedObject(UObject* NewObject)
{
	check(NewObject);

	if (UMVVMBlueprintViewExtension* Extension = Cast<UMVVMBlueprintViewExtension>(NewObject))
	{
		NewExtensions.Add(Extension);
	}
}

}
#undef LOCTEXT_NAMESPACE
