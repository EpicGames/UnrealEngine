// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories.h"
#include "IHasClipboardExtensibility.h"

class UMVVMBlueprintViewExtension;
class UWidget;
class UWidgetBlueprint;

namespace UE::MVVM
{

class FClipboardExtension
	: public IClipboardExtension
{
	//~ Begin IClipboardExtension overrides
	virtual void AppendToClipboard(const UWidget* Widget, const FExportArgs& ExportArgs) override;
	virtual bool CanAppendToClipboard(const UWidget* Widget) const override;
	virtual void ProcessImportedText(const UWidgetBlueprint* WidgetBlueprint, const FString& TextToImport, UPackage*& TempPackage) override;
	virtual bool CanImportFromClipboard(const UWidget* Widget) const override;
	virtual void ImportDataToWidget(const UWidget* Widget, FName OldWidgetName) override;
	virtual bool CanWidgetAcceptPaste(const UWidget* Widget) const override;
	//~ End IClipboardExtension overrides

private:

	class FExtensionTextFactory : public FCustomizableTextObjectFactory
	{
	public:
		FExtensionTextFactory();

		//~ Begin FCustomizableTextObjectFactory overrides
		virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override;
		virtual void ProcessConstructedObject(UObject* NewObject) override;
		//~ End FCustomizableTextObjectFactory overrides

	public:
		TArray<TWeakObjectPtr<UMVVMBlueprintViewExtension>> NewExtensions;
	};

	FExtensionTextFactory Factory;
};

}
