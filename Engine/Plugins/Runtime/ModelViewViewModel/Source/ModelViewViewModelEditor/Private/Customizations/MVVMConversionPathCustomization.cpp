// Copyright Epic Games, Inc. All Rights Reserved.
#include "MVVMConversionPathCustomization.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "MVVMBlueprintFunctionReference.h"
#include "MVVMBlueprintViewBinding.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "WidgetBlueprint.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMVVMFunctionParameter.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MVVMConversionPath"

namespace UE::MVVM
{
FConversionPathCustomization::FConversionPathCustomization(UWidgetBlueprint* InWidgetBlueprint)
{
	check(InWidgetBlueprint);
	WidgetBlueprint = InWidgetBlueprint;
}

void FConversionPathCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	ParentHandle = InPropertyHandle->GetParentHandle();
	BindingModeHandle = ParentHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, BindingType));
	HeaderRow
		.ShouldAutoExpand()
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			InPropertyHandle->CreatePropertyValueWidget()
		];
}

void FConversionPathCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	AddRowForProperty(ChildBuilder, InPropertyHandle, true);
	AddRowForProperty(ChildBuilder, InPropertyHandle, false);
}

FText FConversionPathCustomization::GetFunctionPathText(UMVVMBlueprintViewConversionFunction* BlueprintViewConversionFunction) const
{
	const FText& NoneText = LOCTEXT("None", "None");
	if (!BlueprintViewConversionFunction)
	{
		return NoneText;
	}
	
	if (UWidgetBlueprint* WidgetBP = WidgetBlueprint.Get())
	{
		if (const UFunction* Function = BlueprintViewConversionFunction->GetConversionFunction().GetFunction(WidgetBP))
		{
			return FText::FromString(Function->GetPathName());
		}
	}
	return NoneText;
}

void FConversionPathCustomization::AddRowForProperty(IDetailChildrenBuilder& ChildBuilder, const TSharedPtr<IPropertyHandle>& InPropertyHandle, bool bSourceToDestination)
{
	check(InPropertyHandle->GetProperty()->GetOwnerStruct() == FMVVMBlueprintViewBinding::StaticStruct());
	 
	TArray<FMVVMBlueprintViewBinding*> ViewBindings;
	TArray<void*> RawData;
	ParentHandle->AccessRawData(RawData);
	for (void* Data : RawData)
	{
		FMVVMBlueprintViewBinding* Binding = reinterpret_cast<FMVVMBlueprintViewBinding*>(Data);
		if (Binding)
		{
			ViewBindings.Add(Binding);
		}
	}

	// Multi-selection is not currently supported in the binding panel.
	if (RawData.Num() != 1)
	{
		return;
	}

	const FName FunctionPropertyName = bSourceToDestination ? GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewConversionPath, SourceToDestinationConversion) : GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewConversionPath, DestinationToSourceConversion);
	TSharedRef<IPropertyHandle> FunctionProperty = InPropertyHandle->GetChildHandle(FunctionPropertyName).ToSharedRef();
	UMVVMBlueprintViewConversionFunction* ConversionFunction = ViewBindings[0]->Conversion.GetConversionFunction(bSourceToDestination);

	ChildBuilder.AddCustomRow(FText::FromName(FunctionPropertyName))
		.NameContent()
		[
			FunctionProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SEditableTextBox)
				.Text(this, &FConversionPathCustomization::GetFunctionPathText, ConversionFunction)
			]
		];
			
		const EMVVMBindingMode BindingMode = GetBindingMode();

		if (ConversionFunction == nullptr || !IsForwardBinding(GetBindingMode()))
		{
			return;
		}

		const TArrayView<const FMVVMBlueprintPin>& Pins = ConversionFunction->GetPins();
		for (const FMVVMBlueprintPin& Pin : Pins)
		{
			const FText& ArgumentName = FText::FromString(Pin.GetId().ToString());
			ChildBuilder.AddCustomRow(ArgumentName)
				.NameContent()
				[
					SNew(SBox)
					.Padding(FMargin(16, 0, 0, 0))
					[
						SNew(STextBlock)
						.Text(ArgumentName)
					]
				]
				.ValueContent()
				[
					SNew(UE::MVVM::SFunctionParameter, WidgetBlueprint.Get())
					.BindingId(ViewBindings[0]->BindingId)
					.ParameterId(Pin.GetId())
					.SourceToDestination(bSourceToDestination)
					.AllowDefault(true)
					.IsEnabled(false)
				];
		}
}

EMVVMBindingMode FConversionPathCustomization::GetBindingMode() const
{
	uint8 EnumValue = 0;
	if (BindingModeHandle->GetValue(EnumValue) == FPropertyAccess::Success)
	{
		return static_cast<EMVVMBindingMode>(EnumValue);
	}
	return EMVVMBindingMode::OneWayToDestination;
}
}
#undef LOCTEXT_NAMESPACE 