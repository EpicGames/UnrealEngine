// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IPropertyTypeCustomization.h"

enum class EMVVMBindingMode : uint8;
struct EVisibility;
class UWidgetBlueprint;
class UMVVMBlueprintViewConversionFunction;

namespace UE::MVVM
{
class FConversionPathCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance(UWidgetBlueprint* InWidgetBlueprint)
	{
		return MakeShared<FConversionPathCustomization>(InWidgetBlueprint);
	}
	FConversionPathCustomization(UWidgetBlueprint* InWidgetBlueprint);
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	FText GetFunctionPathText(UMVVMBlueprintViewConversionFunction* BlueprintViewConversionFunction) const;
	void AddRowForProperty(IDetailChildrenBuilder& ChildBuilder, const TSharedPtr<IPropertyHandle>& Property, bool bSourceToDestination);
	EMVVMBindingMode GetBindingMode() const;

private:
	TWeakObjectPtr<UWidgetBlueprint> WidgetBlueprint;
	TSharedPtr<IPropertyHandle> ParentHandle;
	TSharedPtr<IPropertyHandle> BindingModeHandle;
};
} // namespace UE::MVVM