// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MVVMDragDropExtension.h"

#include "Components/PanelWidget.h"
#include "Extensions/MVVMViewBlueprintPanelWidgetExtension.h"
#include "Internationalization/Internationalization.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditorUtils.h"

namespace UE::MVVM
{
bool FWidgetDragDropExtension::ShouldPreventDropOnTarget(const UWidget* Target, const TSharedPtr<FDragDropOperation>& DragDropOp) const
{
	bool bShouldPreventDropWidgetOnTarget = false;

	if (Target && Target->IsA<UPanelWidget>())
	{
		if (const UWidgetBlueprint* WidgetBlueprint = FWidgetBlueprintEditorUtils::GetWidgetBlueprintFromWidget(Target))
		{
			if (const UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint))
			{
				const TArray<UMVVMBlueprintViewExtension*> BlueprintExtensions = ExtensionView->GetBlueprintExtensionsForWidget(Target->GetFName());
				bShouldPreventDropWidgetOnTarget = BlueprintExtensions.ContainsByPredicate([](const UMVVMBlueprintViewExtension* BlueprintExtension) -> bool
				{
					return BlueprintExtension->IsA<UMVVMBlueprintViewExtension_PanelWidget>();
				});
			}
		}
	}

	return bShouldPreventDropWidgetOnTarget;
}

FText FWidgetDragDropExtension::GetDropFailureText(const UWidget* Target, const TSharedPtr<FDragDropOperation>& DragDropOp) const
{
	return NSLOCTEXT("MVVMDragDropExtension", "UnableToAddChildWidget", "Cannot add children to a panel widget with an MVVM extension.");
}
}
