// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IHasWidgetDragDropExtensibility.h"

namespace UE::MVVM
{
class FWidgetDragDropExtension : public IWidgetDragDropExtension
{
	//~ Begin IWidgetDragDropExtension overrides
	virtual bool ShouldPreventDropOnTarget(const UWidget* Target, const TSharedPtr<FDragDropOperation>& DragDropOp) const override;
	virtual FText GetDropFailureText(const UWidget* Target, const TSharedPtr<FDragDropOperation>& DragDropOp) const override;
	//~ End IWidgetDragDropExtension overrides
};
}
