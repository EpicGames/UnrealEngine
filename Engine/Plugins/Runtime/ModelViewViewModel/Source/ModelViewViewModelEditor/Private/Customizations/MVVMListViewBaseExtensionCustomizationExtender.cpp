// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMListViewBaseExtensionCustomizationExtender.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/ListViewBase.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Extensions/MVVMViewBlueprintListViewBaseExtension.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewModel.h"
#include "MVVMDeveloperProjectSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MVVMListViewBaseExtensionCustomizationExtender"

namespace UE::MVVM
{

TSharedPtr<FMVVMListViewBaseExtensionCustomizationExtender> FMVVMListViewBaseExtensionCustomizationExtender::MakeInstance()
{
	return MakeShared<FMVVMListViewBaseExtensionCustomizationExtender>();
}

void FMVVMListViewBaseExtensionCustomizationExtender::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout, const TArrayView<UWidget*> InWidgets, const TSharedRef<FWidgetBlueprintEditor>& InWidgetBlueprintEditor)
{
	// multi-selection not supported for the data
	if (InWidgets.Num() == 1)
	{
		if (UListViewBase* ListView = Cast<UListViewBase>(InWidgets[0]))
		{
			if (GetDefault<UMVVMDeveloperProjectSettings>()->IsExtensionSupportedForListViewBaseClass(ListView->GetClass()))
			{
				Widget = ListView;
				WidgetBlueprintEditor = InWidgetBlueprintEditor;

				// Only do a customization if we have a MVVM blueprint view class on this blueprint.
				if (GetExtensionViewForSelectedWidgetBlueprint())
				{
					IDetailCategoryBuilder& MVVMCategory = InDetailLayout.EditCategory("ListEntries");
					FName NAME_ViewmodelExtension = "ViewmodelExtension";

					// Fetch the entry widget class handle.
					TArray<TSharedRef<IPropertyHandle>> ListEntryProperties;
					MVVMCategory.GetDefaultProperties(ListEntryProperties);
					TSharedRef<IPropertyHandle>* EntryClassPtr = ListEntryProperties.FindByPredicate( [](const TSharedRef<IPropertyHandle> Property) 
					{ 
						if (Property->GetProperty())
						{
      						return Property->GetProperty()->GetName() == TEXT("EntryWidgetClass");
      					}
						return false;
					});

					check(EntryClassPtr);
					EntryClassHandle = *EntryClassPtr;
					EntryClassHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FMVVMListViewBaseExtensionCustomizationExtender::HandleEntryClassChanged, false));
					bIsExtensionAdded = GetListBaseViewExtension() != nullptr;
					HandleEntryClassChanged(true);

					// Add a button that controls adding/removing the extension on the ListViewBase widget
					MVVMCategory.AddCustomRow(FText::FromString(TEXT("Viewmodel")))
					.RowTag(NAME_ViewmodelExtension)
					.NameContent()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("VMSupport", "Viewmodel Extension"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.ToolTipText(LOCTEXT("VMExtensionToolTip", "Add or remove a Viewmodel Extension. This extension adds a widget to this list view for each entry that is provided to the Set List Items function via a binding."))
					]
					.ValueContent()
					.HAlign(HAlign_Fill)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SButton)
							.OnClicked(this, &FMVVMListViewBaseExtensionCustomizationExtender::ModifyExtension)
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.AutoWidth()
								[
									SNew(SImage)
									.Image(this, &FMVVMListViewBaseExtensionCustomizationExtender::GetExtensionButtonIcon)
								]
								+ SHorizontalBox::Slot()
								.Padding(FMargin(3.0f, 0.0f, 0.0f, 0.0f))
								.VAlign(VAlign_Center)
								.AutoWidth()
								[
									SNew(STextBlock)
									.TextStyle(FAppStyle::Get(), "SmallButtonText")
									.Text(this, &FMVVMListViewBaseExtensionCustomizationExtender::GetExtensionButtonText)
								]
							]
						]
					];

					// Add a combobox that allows selecting from the viewmodels in the entry widgets
					MVVMCategory.AddCustomRow(FText::FromString(TEXT("Viewmodel")))
					.RowTag(NAME_ViewmodelExtension)
					.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMVVMListViewBaseExtensionCustomizationExtender::GetEntryViewModelVisibility)))
					.NameContent()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("EntryVM", "Entry Viewmodel"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.ToolTipText(LOCTEXT("EntryVMToolTip", "Each entry created by this Viewmodel Extension will be bound to this Viewmodel on the entry widget"))
					]
					.ValueContent()
					.HAlign(HAlign_Fill)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SComboButton)
							.OnGetMenuContent(this, &FMVVMListViewBaseExtensionCustomizationExtender::OnGetViewModelsMenuContent)
							.ButtonContent()
							[
								SNew(STextBlock)
								.Text(this, &FMVVMListViewBaseExtensionCustomizationExtender::OnGetSelectedViewModel)
								.ToolTipText(this, &FMVVMListViewBaseExtensionCustomizationExtender::OnGetSelectedViewModel)
							]
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							PropertyCustomizationHelpers::MakeClearButton(
							FSimpleDelegate::CreateSP(this, &FMVVMListViewBaseExtensionCustomizationExtender::ClearEntryViewModel))
						]
					];
				}
			}
		}
	}
}

FReply FMVVMListViewBaseExtensionCustomizationExtender::ModifyExtension()
{
	if (UMVVMBlueprintViewExtension_ListViewBase* ListBaseViewExtension = GetListBaseViewExtension())
	{
		if (UListViewBase* WidgetPtr = Widget.Get())
		{
			GetExtensionViewForSelectedWidgetBlueprint()->RemoveBlueprintWidgetExtension(ListBaseViewExtension, WidgetPtr->GetFName());
			bIsExtensionAdded = false;
		}
	}
	else
	{
		CreateListBaseViewExtensionIfNotExisting();
		bIsExtensionAdded = true;
	}
	return FReply::Handled();
}

void FMVVMListViewBaseExtensionCustomizationExtender::CreateListBaseViewExtensionIfNotExisting()
{
	if (UMVVMWidgetBlueprintExtension_View* Extension = GetExtensionViewForSelectedWidgetBlueprint())
	{
		if (UListViewBase* WidgetPtr = Widget.Get())
		{
			TArray<UMVVMBlueprintViewExtension*> Extensions = Extension->GetBlueprintExtensionsForWidget(WidgetPtr->GetFName());
			bool bExists = Extensions.ContainsByPredicate([](UMVVMBlueprintViewExtension* Extension) { return Cast<UMVVMBlueprintViewExtension_ListViewBase>(Extension) != nullptr; });
			if (!bExists)
			{
				UMVVMBlueprintViewExtension* NewExtension = Extension->CreateBlueprintWidgetExtension(UMVVMBlueprintViewExtension_ListViewBase::StaticClass(), WidgetPtr->GetFName());
				UMVVMBlueprintViewExtension_ListViewBase* NewListViewExtension = CastChecked<UMVVMBlueprintViewExtension_ListViewBase>(NewExtension);
				NewListViewExtension->WidgetName = WidgetPtr->GetFName();
			}
		}
	}
}

UMVVMBlueprintViewExtension_ListViewBase* FMVVMListViewBaseExtensionCustomizationExtender::GetListBaseViewExtension() const
{
	if (UMVVMWidgetBlueprintExtension_View* ViewClass = GetExtensionViewForSelectedWidgetBlueprint())
	{
		if (UListViewBase* WidgetPtr = Widget.Get())
		{
			for (UMVVMBlueprintViewExtension* Extension : ViewClass->GetBlueprintExtensionsForWidget(WidgetPtr->GetFName()))
			{
				if (UMVVMBlueprintViewExtension_ListViewBase* ListViewBaseExtension = Cast<UMVVMBlueprintViewExtension_ListViewBase>(Extension))
				{
					return ListViewBaseExtension;
				}

			}
		}
	}

	return nullptr;
}

UMVVMWidgetBlueprintExtension_View* FMVVMListViewBaseExtensionCustomizationExtender::GetExtensionViewForSelectedWidgetBlueprint() const
{
	if (const TSharedPtr<FWidgetBlueprintEditor> BPEditor = WidgetBlueprintEditor.Pin())
	{
		if (const UWidgetBlueprint* Blueprint = BPEditor->GetWidgetBlueprintObj())
		{
			return UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(Blueprint);
		}
	}

	return nullptr;
}

void FMVVMListViewBaseExtensionCustomizationExtender::ClearEntryViewModel()
{
	SetEntryViewModel(FGuid());
}

void FMVVMListViewBaseExtensionCustomizationExtender::HandleEntryClassChanged(bool bIsInit)
{
	// Update the cached value of entry class
	void* EntryClassPtr = nullptr;
	TSubclassOf<UUserWidget>* EntryClassValue = nullptr;
	bool bEntryClassChanged = false;

	if (EntryClassHandle->IsValidHandle() && EntryClassHandle->GetValueData(EntryClassPtr) == FPropertyAccess::Success)
	{
		EntryClassValue = static_cast<TSubclassOf<UUserWidget>*>(EntryClassPtr);
	}

	if (!EntryClassValue || !EntryClass.Get() || EntryClass.Get() != *EntryClassValue)
	{
		bEntryClassChanged = true;
	}
	EntryClass = EntryClassValue ? *EntryClassValue : nullptr;

	// Update other values that depend on the entry class (only if the cached value actually changed)
	if (bEntryClassChanged && EntryClass)
	{
		if (const UUserWidget* EntryCDO = Cast<UUserWidget>(EntryClass->GetDefaultObject(false)))
		{
			EntryWidgetBlueprint = Cast<UWidgetBlueprint>(EntryCDO->GetClass()->ClassGeneratedBy);
		}

		// Clear the saved entry viewmodel if we're not calling this from CustomizeDetails (not initializing the customizer)
		if (!bIsInit)
		{
			SetEntryViewModel(FGuid(), false);
		}
	}
}

FText FMVVMListViewBaseExtensionCustomizationExtender::OnGetSelectedViewModel() const
{
	if (const UListViewBase* WidgetPtr = Widget.Get())
	{
		if (EntryClass)
		{
			if (UMVVMBlueprintViewExtension_ListViewBase* ListBaseViewExtension = GetListBaseViewExtension())
			{
				if (const UUserWidget* EntryUserWidget = Cast<UUserWidget>(EntryClass->GetDefaultObject(false)))
				{
					if (const UWidgetBlueprint* EntryBlueprint = Cast<UWidgetBlueprint>(EntryUserWidget->GetClass()->ClassGeneratedBy))
					{
						if (const UMVVMWidgetBlueprintExtension_View* EntryWidgetExtension = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(EntryBlueprint))
						{
							if (const UMVVMBlueprintView* EntryWidgetView = EntryWidgetExtension->GetBlueprintView())
							{
								if (const FMVVMBlueprintViewModelContext* ViewModelContext = EntryWidgetView->FindViewModel(ListBaseViewExtension->GetEntryViewModelId()))
								{
									return FText::FromName(ViewModelContext->GetViewModelName());
								}
							}
						}
					}
				}
			}
		}
	}

	return LOCTEXT("NoViewmodel", "No Viewmodel");
}

FText FMVVMListViewBaseExtensionCustomizationExtender::GetExtensionButtonText() const
{
	return bIsExtensionAdded ? LOCTEXT("RemoveVMExt", "Remove Viewmodel Extension") : LOCTEXT("AddVMExt", "Add Viewmodel Extension");
}

const FSlateBrush* FMVVMListViewBaseExtensionCustomizationExtender::GetExtensionButtonIcon() const
{
	return bIsExtensionAdded ? FAppStyle::Get().GetBrush("Icons.X") : FAppStyle::Get().GetBrush("Icons.Plus");
}

TSharedRef<SWidget> FMVVMListViewBaseExtensionCustomizationExtender::OnGetViewModelsMenuContent()
{
	FMenuBuilder MenuBuilder(true, NULL);

	if (EntryClass)
	{
		// Find all viewmodels in the entry widget
		if (const UWidgetBlueprint* EntryWidgetBlueprintPtr = EntryWidgetBlueprint.Get())
		{
			if (const UMVVMWidgetBlueprintExtension_View* EntryWidgetExtension = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(EntryWidgetBlueprintPtr))
			{
				if (const UMVVMBlueprintView* EntryWidgetView = EntryWidgetExtension->GetBlueprintView())
				{
					const TArrayView<const FMVVMBlueprintViewModelContext> ViewModels = EntryWidgetView->GetViewModels();
					for (const FMVVMBlueprintViewModelContext& EntryViewModel : ViewModels)
					{
						// Create the menu action for this entry viewmodel
						FUIAction ItemAction(FExecuteAction::CreateSP(this, &FMVVMListViewBaseExtensionCustomizationExtender::SetEntryViewModel, EntryViewModel.GetViewModelId(), true));
						MenuBuilder.AddMenuEntry(FText::FromName(EntryViewModel.GetViewModelName()), TAttribute<FText>(), FSlateIcon(), ItemAction);
					}
				}
			}
		}
	}
	return MenuBuilder.MakeWidget();
}

void FMVVMListViewBaseExtensionCustomizationExtender::SetEntryViewModel(FGuid InEntryViewModelId, bool bMarkModified)
{
	if (UMVVMWidgetBlueprintExtension_View* Extension = GetExtensionViewForSelectedWidgetBlueprint())
	{
		if (const UListViewBase* WidgetPtr = Widget.Get())
		{
			if (UMVVMBlueprintViewExtension_ListViewBase* ListBaseViewExtension = GetListBaseViewExtension())
			{
				if (ListBaseViewExtension->EntryViewModelId != InEntryViewModelId)
				{
					const FScopedTransaction Transaction(LOCTEXT("SetEntryViewModel", "Set Entry ViewModel"));
					ListBaseViewExtension->Modify();
					ListBaseViewExtension->EntryViewModelId = InEntryViewModelId;
					if (bMarkModified)
					{
						if (const TSharedPtr<FWidgetBlueprintEditor> BPEditor = WidgetBlueprintEditor.Pin())
						{
							if (UWidgetBlueprint* Blueprint = BPEditor->GetWidgetBlueprintObj())
							{
								FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
							}
						}
					}
				}
			}
		}
	}
}

}
#undef LOCTEXT_NAMESPACE