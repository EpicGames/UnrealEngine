// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/IBlueprintWidgetCustomizationExtender.h"
#include "Blueprint/UserWidget.h"
#include "Layout/Visibility.h"
#include "MVVMPropertyPath.h"
#include "UObject/WeakObjectPtr.h"

class FReply;
class IPropertyHandle;
class SWidget;
class UListViewBase;
class UMVVMBlueprintViewExtension_ListViewBase;
class UMVVMWidgetBlueprintExtension_View;

namespace  UE::MVVM
{

class FMVVMListViewBaseExtensionCustomizationExtender : public IBlueprintWidgetCustomizationExtender
{
public:
	static TSharedPtr<FMVVMListViewBaseExtensionCustomizationExtender> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout, const TArrayView<UWidget*> InWidgets, const TSharedRef<FWidgetBlueprintEditor>& InWidgetBlueprintEditor);

private:
	/** Set the entry viewmodel on the MVVMViewBlueprintListViewBaseExtension for this widget. */
	void SetEntryViewModel(FGuid InEntryViewModelId, bool bMarkModified = true);

	/** Get a list of all viewmodels in the entry class. */
	TSharedRef<SWidget> OnGetViewModelsMenuContent();

	/** Get the name of the currently-selected entry viewmodel from the extension. */
	FText OnGetSelectedViewModel() const;

	/** Clear the entry viewmodel on the MVVMViewBlueprintListViewBaseExtension for this widget. */
	void ClearEntryViewModel();

	/** Update the cached variables when the entry class property changes. */
	void HandleEntryClassChanged(bool bIsInit);

	/** Get the visibility of entry viewmodel row in the details panel. */
	EVisibility GetEntryViewModelVisibility() const
	{
		return bIsExtensionAdded ? EVisibility::Visible : EVisibility::Collapsed;
	}

	/** Create a new MVVMViewBlueprintListViewBaseExtension for this widget in the blueprint view class. */
	void CreateListBaseViewExtensionIfNotExisting();

	/** Get the MVVMViewBlueprintListViewBaseExtension for this widget in the blueprint view class. */
	UMVVMBlueprintViewExtension_ListViewBase* GetListBaseViewExtension() const;

	/** Get the MVVM blueprint view class of this widget blueprint. */
	UMVVMWidgetBlueprintExtension_View* GetExtensionViewForSelectedWidgetBlueprint() const;

	/** Add/Remove the MVVMViewBlueprintListViewBaseExtension for this widget on button click. */
	FReply ModifyExtension();

	/** Get + or X icon for the MVVM extension button. */
	const FSlateBrush* GetExtensionButtonIcon() const;

	/** Get display text for the MVVM extension button. */
	FText GetExtensionButtonText() const;

private:
	/** The selected ListViewBase widget in the details panel. */
	TWeakObjectPtr<UListViewBase> Widget;
	TWeakPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor;

	TSubclassOf<UUserWidget> EntryClass;
	TSharedPtr<IPropertyHandle> EntryClassHandle;
	TWeakObjectPtr<UWidgetBlueprint> EntryWidgetBlueprint;

	/** Keep track of whether we have a MVVMViewBlueprintListViewBaseExtension for this widget. */
	bool bIsExtensionAdded = false;
};
}