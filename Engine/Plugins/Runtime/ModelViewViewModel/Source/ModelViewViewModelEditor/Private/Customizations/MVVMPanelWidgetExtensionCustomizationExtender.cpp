// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMPanelWidgetExtensionCustomizationExtender.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/PanelWidget.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Dialogs/Dialogs.h"
#include "Extensions/MVVMViewBlueprintPanelWidgetExtension.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewModel.h"
#include "MVVMDeveloperProjectSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"
#include "WidgetBlueprintEditorUtils.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MVVMPanelWidgetExtensionCustomizationExtender"

namespace UE::MVVM
{

TSharedPtr<FMVVMPanelWidgetExtensionCustomizationExtender> FMVVMPanelWidgetExtensionCustomizationExtender::MakeInstance()
{
	return MakeShared<FMVVMPanelWidgetExtensionCustomizationExtender>();
}

void FMVVMPanelWidgetExtensionCustomizationExtender::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout, const TArrayView<UWidget*> InWidgets, const TSharedRef<FWidgetBlueprintEditor>& InWidgetBlueprintEditor)
{
	// multi-selection not supported for the data
	if (InWidgets.Num() == 1)
	{
		if (UPanelWidget* Panel = Cast<UPanelWidget>(InWidgets[0]))
		{
			if (Panel->CanHaveMultipleChildren())
			{
				if (GetDefault<UMVVMDeveloperProjectSettings>()->IsExtensionSupportedForPanelClass(Panel->GetClass()))
				{
					FName NAME_ViewmodelExtension = "ViewmodelExtension";
					FName NAME_ViewmodelExtensionSlot = "ViewmodelExtensionSlot";
					Widget = Panel;
					WidgetBlueprintEditor = InWidgetBlueprintEditor;

					// Only do a customization if we have a MVVM blueprint view class on this blueprint.
					if (GetExtensionViewForSelectedWidgetBlueprint())
					{
						IDetailCategoryBuilder& MVVMCategory = InDetailLayout.EditCategory("Viewmodel");

						bIsExtensionAdded = GetPanelWidgetExtension() != nullptr;

						// Add a button that controls adding/removing the extension on the panel widget
						MVVMCategory.AddCustomRow(FText::FromString(TEXT("Viewmodel")))
						.RowTag(NAME_ViewmodelExtension)
						.NameContent()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("VMExtension", "Viewmodel Extension"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.ToolTipText(LOCTEXT("VMExtensionToolTip", "Add or remove a Viewmodel Extension. This extension adds a widget to this Panel Widget for each entry that is provided to the Set Items function of the Viewmodel Extension via a binding."))
						]
						.ValueContent()
						.HAlign(HAlign_Fill)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SButton)
								.OnClicked(this, &FMVVMPanelWidgetExtensionCustomizationExtender::ModifyExtension)
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.HAlign(HAlign_Center)
									.VAlign(VAlign_Center)
									.AutoWidth()
									[
										SNew(SImage)
										.Image(this, &FMVVMPanelWidgetExtensionCustomizationExtender::GetExtensionButtonIcon)
									]
									+ SHorizontalBox::Slot()
									.Padding(FMargin(3.0f, 0.0f, 0.0f, 0.0f))
									.VAlign(VAlign_Center)
									.AutoWidth()
									[
										SNew(STextBlock)
										.TextStyle(FAppStyle::Get(), "SmallButtonText")
										.Text(this, &FMVVMPanelWidgetExtensionCustomizationExtender::GetExtensionButtonText)
									]
								]
							]
						];

						if (UMVVMBlueprintViewExtension_PanelWidget* PanelExtension = GetPanelWidgetExtension())
						{
							IDetailPropertyRow* PanelExtensionPropertyRow = MVVMCategory.AddExternalObjects({ PanelExtension});
							TSharedPtr<IPropertyHandle> PanelExtensionObjectHandle = PanelExtensionPropertyRow->GetPropertyHandle();
							PanelExtensionPropertyRow->Visibility(EVisibility::Collapsed);

							// "Entry Widget Class" property row
							EntryClassHandle = PanelExtensionObjectHandle->GetChildHandle("EntryWidgetClass");
							EntryClassHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FMVVMPanelWidgetExtensionCustomizationExtender::HandleEntryClassChanged, false));
							HandleEntryClassChanged(true);
							IDetailPropertyRow& EntryClassRow = MVVMCategory.AddProperty(EntryClassHandle);
							EntryClassHandle->SetToolTipText(LOCTEXT("EntryWidgetClassToolTip", "A custom widget that will be added to this Panel Widget for each entry provided to this Viewmodel Extension."));

							// "Entry Viewmodel" property row
							MVVMCategory.AddCustomRow(FText::FromString(TEXT("Viewmodel")))
								.RowTag(NAME_ViewmodelExtension)
								.NameContent()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("EntryVM", "Entry Viewmodel"))
									.Font(IDetailLayoutBuilder::GetDetailFont())
									.ToolTipText(LOCTEXT("EntryVMToolTip", "Each entry created by this Viewmodel Extension will be bound to this Viewmodel on the entry widget"))
								]
								.ValueContent()
								.HAlign(HAlign_Fill)
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.AutoWidth()
									[
										SNew(SComboButton)
										.OnGetMenuContent(this, &FMVVMPanelWidgetExtensionCustomizationExtender::OnGetViewModelsMenuContent)
										.ButtonContent()
										[
											SNew(STextBlock)
											.Text(this, &FMVVMPanelWidgetExtensionCustomizationExtender::OnGetSelectedViewModel)
											.ToolTipText(this, &FMVVMPanelWidgetExtensionCustomizationExtender::OnGetSelectedViewModel)
										]
									]
									+ SHorizontalBox::Slot()
									.AutoWidth()
									[
										PropertyCustomizationHelpers::MakeClearButton(
											FSimpleDelegate::CreateSP(this, &FMVVMPanelWidgetExtensionCustomizationExtender::ClearEntryViewModel))
									]
								];

							// "Slot template" property row
							IDetailPropertyRow* SlotDetailRow = MVVMCategory.AddExternalObjects({ PanelExtension->SlotObj }, EPropertyLocation::Default,
								FAddPropertyParams()
								.CreateCategoryNodes(false)
								.AllowChildren(true)
								.HideRootObjectNode(false)
								.UniqueId(NAME_ViewmodelExtensionSlot)
							);

							TSharedPtr<IPropertyHandle> SlotPropertyHandle = SlotDetailRow->GetPropertyHandle();
							SlotPropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FMVVMPanelWidgetExtensionCustomizationExtender::HandleSlotChildPropertyChanged));

							SlotDetailRow->CustomWidget(true)
								.NameContent()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("SlotTemplate", "Slot Template"))
									.Font(IDetailLayoutBuilder::GetDetailFont())
									.ToolTipText(LOCTEXT("SlotTemplateToolTip", "Customize the slot properties used to insert widgets into this Panel Widget."))
								]
								.ValueContent()
								[
									SlotPropertyHandle->CreatePropertyValueWidget()
								];

							// Because AddExternalObjects was used the property system will not add a reset to default widget by default 
							SlotDetailRow->OverrideResetToDefault(
								FResetToDefaultOverride::Create(
									FIsResetToDefaultVisible::CreateLambda([SlotPropertyHandle](TSharedPtr<IPropertyHandle> Handle)
										{
											return SlotPropertyHandle->CanResetToDefault();
										}),
									FResetToDefaultHandler::CreateLambda([SlotPropertyHandle](TSharedPtr<IPropertyHandle> Handle)
										{
											return SlotPropertyHandle->ResetToDefault();
										})
									)
							);

							// "Num Designer Preview Entries" property row
							TSharedPtr<IPropertyHandle> NumDesignerPreviewEntriesHandle = PanelExtensionObjectHandle->GetChildHandle("NumDesignerPreviewEntries");
							NumDesignerPreviewEntriesHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FMVVMPanelWidgetExtensionCustomizationExtender::HandleNumDesignerPreviewEntriesChanged));
							MVVMCategory.AddProperty(NumDesignerPreviewEntriesHandle);
							NumDesignerPreviewEntriesHandle->SetToolTipText(LOCTEXT("NumDesignerPreviewEntriesToolTip", "Set the number of dummy widgets to show in the editor preview of the Panel Widget, to check your layout settings."));
						}
					}
				}
			}
		}
	}
}

FReply FMVVMPanelWidgetExtensionCustomizationExtender::ModifyExtension()
{
	UPanelWidget* WidgetPtr = Widget.Get();

	if (UMVVMBlueprintViewExtension_PanelWidget* PanelExtension = GetPanelWidgetExtension())
	{
		if (WidgetPtr)
		{
			GetExtensionViewForSelectedWidgetBlueprint()->RemoveBlueprintWidgetExtension(PanelExtension, WidgetPtr->GetFName());
			bIsExtensionAdded = false;
			WidgetPtr->ClearChildren();
		}
	}
	else
	{
		// Warn the user that this may result in data loss
		if (WidgetPtr && WidgetPtr->GetChildrenCount() > 0)
		{
			FString ChildNames;

			for (UWidget* ChildWidget : WidgetPtr->GetAllChildren())
			{
				if (ensure(ChildWidget))
				{
					ChildNames += FString::Printf(TEXT("%s"), *ChildWidget->GetFName().ToString());
					ChildNames += ChildWidget == WidgetPtr->GetAllChildren().Last() ? TEXT(".") : TEXT(", ");
				}
			}

			FFormatNamedArguments Args;
			Args.Add(TEXT("WidgetName"), FText::FromName(WidgetPtr->GetFName()));
			Args.Add(TEXT("NumChildren"), FText::AsNumber(WidgetPtr->GetChildrenCount()));
			Args.Add(TEXT("ChildNames"), FText::FromString(ChildNames));
			const FText ConfirmDelete = FText::Format(LOCTEXT("ConfirmReplaceWidgetWithVariableInUse", "Adding a viewmodel extension to {WidgetName} will erase its {NumChildren}|plural(one=child, other=children):\n\n{ChildNames}\n\nDo you wish to continue?"), Args);

			FSuppressableWarningDialog::FSetupInfo Info(ConfirmDelete, LOCTEXT("DeletePanelWidgetChildren", "Delete children"), "DeletePanelWidgetChildren_Warning");
			Info.ConfirmText = LOCTEXT("DeleteChildren_Continue", "Continue");
			Info.CancelText = LOCTEXT("DeleteChildren_Cancel", "Cancel");

			const FSuppressableWarningDialog DeletePanelWidgetChildren(Info);

			if (DeletePanelWidgetChildren.ShowModal() == FSuppressableWarningDialog::Cancel)
			{
				return FReply::Handled();
			}

			if (const TSharedPtr<FWidgetBlueprintEditor> BPEditor = WidgetBlueprintEditor.Pin())
			{
				if (UWidgetBlueprint* Blueprint = BPEditor->GetWidgetBlueprintObj())
				{
					TSet<UWidget*> ChildWidgets;

					for (UWidget* Child : WidgetPtr->GetAllChildren())
					{
						ChildWidgets.Add(BPEditor->GetReferenceFromPreview(Child).GetTemplate());
					}

					FWidgetBlueprintEditorUtils::DeleteWidgets(Blueprint, ChildWidgets, FWidgetBlueprintEditorUtils::EDeleteWidgetWarningType::WarnAndAskUser);
				}
			}
		}

		CreatePanelWidgetViewExtensionIfNotExisting();
		bIsExtensionAdded = true;
	}

	return FReply::Handled();
}

void FMVVMPanelWidgetExtensionCustomizationExtender::CreatePanelWidgetViewExtensionIfNotExisting()
{
	if (UMVVMWidgetBlueprintExtension_View* Extension = GetExtensionViewForSelectedWidgetBlueprint())
	{
		if (UPanelWidget* WidgetPtr = Widget.Get())
		{
			TArray<UMVVMBlueprintViewExtension*> Extensions = Extension->GetBlueprintExtensionsForWidget(WidgetPtr->GetFName());
			bool bExists = Extensions.ContainsByPredicate([](UMVVMBlueprintViewExtension* Extension){ return Cast<UMVVMBlueprintViewExtension_PanelWidget>(Extension) != nullptr; });
			if (!bExists)
			{
				UMVVMBlueprintViewExtension* NewExtension = Extension->CreateBlueprintWidgetExtension(UMVVMBlueprintViewExtension_PanelWidget::StaticClass(), WidgetPtr->GetFName());
				UMVVMBlueprintViewExtension_PanelWidget* NewPanelWidgetExtension = CastChecked<UMVVMBlueprintViewExtension_PanelWidget>(NewExtension);
				NewPanelWidgetExtension->WidgetName = WidgetPtr->GetFName();

				UPanelSlot* SlotObj = NewObject<UPanelSlot>(NewPanelWidgetExtension, WidgetPtr->GetSlotClass(), NAME_None, RF_Transactional);
				NewPanelWidgetExtension->SlotObj = SlotObj;
			}
		}
	}
}

void FMVVMPanelWidgetExtensionCustomizationExtender::RefreshDesignerPreviewEntries(bool bFullRebuild)
{
	if (UPanelWidget* PanelWidget = Widget.Get())
	{
		UMVVMBlueprintViewExtension_PanelWidget* PanelWidgetExtension = GetPanelWidgetExtension();
		UPanelSlot* SlotTemplate = PanelWidgetExtension ? PanelWidgetExtension->SlotObj.Get() : nullptr;
		const int32 NumDesignerPreviewEntries = PanelWidgetExtension ? PanelWidgetExtension->NumDesignerPreviewEntries : 0;
		UMVVMBlueprintViewExtension_PanelWidget::RefreshDesignerPreviewEntries(PanelWidget, EntryClass, SlotTemplate, NumDesignerPreviewEntries, bFullRebuild);
	}
}

UMVVMBlueprintViewExtension_PanelWidget* FMVVMPanelWidgetExtensionCustomizationExtender::GetPanelWidgetExtension() const
{
	if (UMVVMWidgetBlueprintExtension_View* ViewClass = GetExtensionViewForSelectedWidgetBlueprint())
	{
		if (UPanelWidget* WidgetPtr = Widget.Get())
		{
			for (UMVVMBlueprintViewExtension* Extension : ViewClass->GetBlueprintExtensionsForWidget(WidgetPtr->GetFName()))
			{
				if (UMVVMBlueprintViewExtension_PanelWidget* PanelWidgetExtension = Cast<UMVVMBlueprintViewExtension_PanelWidget>(Extension))
				{
					return PanelWidgetExtension;
				}
			}
		}
	}

	return nullptr;
}

UMVVMWidgetBlueprintExtension_View* FMVVMPanelWidgetExtensionCustomizationExtender::GetExtensionViewForSelectedWidgetBlueprint() const
{
	if (const TSharedPtr<FWidgetBlueprintEditor> BPEditor = WidgetBlueprintEditor.Pin())
	{
		if (const UWidgetBlueprint* Blueprint = BPEditor->GetWidgetBlueprintObj())
		{
			return UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(Blueprint);
		}
	}

	return nullptr;
}

void FMVVMPanelWidgetExtensionCustomizationExtender::ClearEntryViewModel()
{
	SetEntryViewModel(FGuid());
}

void FMVVMPanelWidgetExtensionCustomizationExtender::HandleEntryClassChanged(bool bIsInit)
{
	// Update the cached value of entry class
	void* EntryClassPtr = nullptr;
	TSubclassOf<UUserWidget>* EntryClassValue = nullptr;
	bool bEntryClassChanged = false;

	if (EntryClassHandle->IsValidHandle() && EntryClassHandle->GetValueData(EntryClassPtr) == FPropertyAccess::Success)
	{
		EntryClassValue = reinterpret_cast<TSubclassOf<UUserWidget>*>(EntryClassPtr);
	}

	if (!EntryClassValue || !EntryClass.Get() || EntryClass.Get() != *EntryClassValue)
	{
		bEntryClassChanged = true;
	}
	EntryClass = EntryClassValue ? *EntryClassValue : nullptr;

	// Update other values that depend on the entry class (only if the cached value actually changed)
	if (bEntryClassChanged)
	{
		if (const UUserWidget* EntryCDO = EntryClass ? Cast<UUserWidget>(EntryClass->GetDefaultObject(false)) : nullptr)
		{
			EntryWidgetBlueprint = Cast<UWidgetBlueprint>(EntryCDO->GetClass()->ClassGeneratedBy);
		}

		// Clear the saved entry viewmodel if we're not calling this from CustomizeDetails (not initializing the customizer)
		if (!bIsInit)
		{
			SetEntryViewModel(FGuid(), false);
			constexpr bool bFullRebuild = true;
			RefreshDesignerPreviewEntries(bFullRebuild);
		}
	}
}

void FMVVMPanelWidgetExtensionCustomizationExtender::HandleSlotChildPropertyChanged()
{
	constexpr bool bFullRebuild = true;
	RefreshDesignerPreviewEntries(bFullRebuild);
}

void FMVVMPanelWidgetExtensionCustomizationExtender::HandleNumDesignerPreviewEntriesChanged()
{
	constexpr bool bFullRebuild = false;
	RefreshDesignerPreviewEntries(bFullRebuild);
}

FText FMVVMPanelWidgetExtensionCustomizationExtender::OnGetSelectedViewModel() const
{
	if (const UPanelWidget* WidgetPtr = Widget.Get())
	{
		if (EntryClass)
		{
			if (UMVVMBlueprintViewExtension_PanelWidget* PanelWidgetExtension = GetPanelWidgetExtension())
			{
				if (const UUserWidget* EntryUserWidget = Cast<UUserWidget>(EntryClass->GetDefaultObject(false)))
				{
					if (const UWidgetBlueprint* EntryBlueprint = Cast<UWidgetBlueprint>(EntryUserWidget->GetClass()->ClassGeneratedBy))
					{
						if (const UMVVMWidgetBlueprintExtension_View* EntryWidgetExtension = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(EntryBlueprint))
						{
							if (const UMVVMBlueprintView* EntryWidgetView = EntryWidgetExtension->GetBlueprintView())
							{
								if (const FMVVMBlueprintViewModelContext* ViewModelContext = EntryWidgetView->FindViewModel(PanelWidgetExtension->GetEntryViewModelId()))
								{
									return FText::FromName(ViewModelContext->GetViewModelName());
								}
							}
						}
					}
				}
			}
		}
	}

	return LOCTEXT("NoViewmodel", "No Viewmodel");
}

FText FMVVMPanelWidgetExtensionCustomizationExtender::GetExtensionButtonText() const
{
	return bIsExtensionAdded ? LOCTEXT("RemoveVMExt", "Remove Viewmodel Extension") : LOCTEXT("AddVMExt", "Add Viewmodel Extension");
}

const FSlateBrush* FMVVMPanelWidgetExtensionCustomizationExtender::GetExtensionButtonIcon() const
{
	return bIsExtensionAdded ? FAppStyle::Get().GetBrush("Icons.X") : FAppStyle::Get().GetBrush("Icons.Plus");
}

TSharedRef<SWidget> FMVVMPanelWidgetExtensionCustomizationExtender::OnGetViewModelsMenuContent()
{
	FMenuBuilder MenuBuilder(true, NULL);

	if (EntryClass)
	{
		// Find all viewmodels in the entry widget
		if (const UWidgetBlueprint* EntryWidgetBlueprintPtr = EntryWidgetBlueprint.Get())
		{
			if (const UMVVMWidgetBlueprintExtension_View* EntryWidgetExtension = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(EntryWidgetBlueprintPtr))
			{
				if (const UMVVMBlueprintView* EntryWidgetView = EntryWidgetExtension->GetBlueprintView())
				{
					const TArrayView<const FMVVMBlueprintViewModelContext> ViewModels = EntryWidgetView->GetViewModels();
					for (const FMVVMBlueprintViewModelContext& EntryViewModel : ViewModels)
					{
						// Create the menu action for this entry viewmodel
						FUIAction ItemAction(FExecuteAction::CreateSP(this, &FMVVMPanelWidgetExtensionCustomizationExtender::SetEntryViewModel, EntryViewModel.GetViewModelId(), true));
						MenuBuilder.AddMenuEntry(FText::FromName(EntryViewModel.GetViewModelName()), TAttribute<FText>(), FSlateIcon(), ItemAction);
					}
				}
			}
		}
	}
	return MenuBuilder.MakeWidget();
}

void FMVVMPanelWidgetExtensionCustomizationExtender::SetEntryViewModel(FGuid InEntryViewModelId, bool bMarkModified)
{
	if (UMVVMWidgetBlueprintExtension_View* Extension = GetExtensionViewForSelectedWidgetBlueprint())
	{
		if (const UPanelWidget* WidgetPtr = Widget.Get())
		{
			if (UMVVMBlueprintViewExtension_PanelWidget* PanelWidgetExtension = GetPanelWidgetExtension())
			{
				if (PanelWidgetExtension->EntryViewModelId != InEntryViewModelId)
				{
					const FScopedTransaction Transaction(LOCTEXT("SetEntryViewModel", "Set Entry ViewModel"));
					PanelWidgetExtension->Modify();
					PanelWidgetExtension->EntryViewModelId = InEntryViewModelId;
					if (bMarkModified)
					{
						if (const TSharedPtr<FWidgetBlueprintEditor> BPEditor = WidgetBlueprintEditor.Pin())
						{
							if (UWidgetBlueprint* Blueprint = BPEditor->GetWidgetBlueprintObj())
							{
								FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
							}
						}
					}
				}
			}
		}
	}
}

}
#undef LOCTEXT_NAMESPACE