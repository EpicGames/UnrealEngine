// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/IBlueprintWidgetCustomizationExtender.h"
#include "Blueprint/UserWidget.h"
#include "Layout/Visibility.h"
#include "MVVMPropertyPath.h"
#include "UObject/WeakObjectPtr.h"

class FReply;
class IPropertyHandle;
class SWidget;
class UPanelWidget;
class UMVVMBlueprintViewExtension_PanelWidget;
class UMVVMWidgetBlueprintExtension_View;

namespace  UE::MVVM
{

class FMVVMPanelWidgetExtensionCustomizationExtender : public IBlueprintWidgetCustomizationExtender
{
public:
	static TSharedPtr<FMVVMPanelWidgetExtensionCustomizationExtender> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout, const TArrayView<UWidget*> InWidgets, const TSharedRef<FWidgetBlueprintEditor>& InWidgetBlueprintEditor);

private:
	/** Set the entry viewmodel on the MVVMViewBlueprintPanelWidgetExtension for this widget. */
	void SetEntryViewModel(FGuid InEntryViewModelId, bool bMarkModified = true);

	/** Get a list of all viewmodels in the entry class. */
	TSharedRef<SWidget> OnGetViewModelsMenuContent();

	/** Get the name of the currently-selected entry viewmodel from the extension. */
	FText OnGetSelectedViewModel() const;

	/** Clear the entry viewmodel on the MVVMViewBlueprintPanelWidgetExtension for this widget. */
	void ClearEntryViewModel();

	/** Update the cached variables when the entry class property changes. */
	void HandleEntryClassChanged(bool bIsInit);

	/** Called when a child property on the slot property changes. */
	void HandleSlotChildPropertyChanged();

	/** Called when the "Num Designer Preview Entries" property changes. */
	void HandleNumDesignerPreviewEntriesChanged();

	/** Update the cached variables when the number of grid columns changes. */
	void HandleNumGridColumns();

	/** Create a new MVVMViewBlueprintPanelWidgetExtension for this widget in the blueprint view class. */
	void CreatePanelWidgetViewExtensionIfNotExisting();

	/** Create preview entries for the selected panel widget. */
	void RefreshDesignerPreviewEntries(bool bFullRebuild);

	/** Get the MVVMViewBlueprintPanelWidgetExtension for this widget in the blueprint view class. */
	UMVVMBlueprintViewExtension_PanelWidget* GetPanelWidgetExtension() const;

	/** Get the MVVM blueprint view class of this widget blueprint. */
	UMVVMWidgetBlueprintExtension_View* GetExtensionViewForSelectedWidgetBlueprint() const;

	/** Add/Remove the MVVMViewBlueprintPanelWidgetExtension for this widget on button click. */
	FReply ModifyExtension();

	/** Get + or X icon for the MVVM extension button. */
	const FSlateBrush* GetExtensionButtonIcon() const;

	/** Get display text for the MVVM extension button. */
	FText GetExtensionButtonText() const;

private:
	/** The selected panel widget in the details panel. */
	TWeakObjectPtr<UPanelWidget> Widget;
	TWeakPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor; 

	TSubclassOf<UUserWidget> EntryClass;
	TSharedPtr<IPropertyHandle> EntryClassHandle;
	TWeakObjectPtr<UWidgetBlueprint> EntryWidgetBlueprint;

	/** Keep track of whether we have a MVVMViewBlueprintPanelWidgetExtension for this widget. */
	bool bIsExtensionAdded = false;
};
}