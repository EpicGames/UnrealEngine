// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MVVMPropertyBindingExtension.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Bindings/MVVMConversionFunctionHelper.h"
#include "BlueprintEditor.h"
#include "Components/Widget.h"
#include "DetailLayoutBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMEditorSubsystem.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "PropertyHandle.h"
#include "PropertyPathHelpers.h"
#include "ScopedTransaction.h"
#include "SPrimaryButton.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"
#include "UMGEditorProjectSettings.h"
#include "Widgets/ViewModelFieldDragDropOp.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SMVVMFieldSelectorMenu.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MVVMPropertyBindingExtension"

namespace UE::MVVM
{
namespace Private
{

struct FCreatePropertyBindingParams
{
	TWeakObjectPtr<const UWidgetBlueprint> WidgetBlueprint;
	TWeakObjectPtr<UMVVMBlueprintView> MVVMBlueprintView;
	TWeakObjectPtr<UWidget> Widget;
	TSharedPtr<IPropertyHandle> PropertyHandle;
};

void CreateBindingFromProperty(const FCreatePropertyBindingParams& Params, FMVVMLinkedPinValue LinkedValue, const FMVVMFieldConversion& Conversion)
{
	UWidgetBlueprint* WidgetBlueprint = const_cast<UWidgetBlueprint*>(Params.WidgetBlueprint.Get());
	UMVVMBlueprintView* MVVMBlueprintView = Params.MVVMBlueprintView.Get();
	UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	UWidget* Widget = Params.Widget.Get();
	if (WidgetBlueprint == nullptr || MVVMBlueprintView == nullptr || Widget == nullptr || Subsystem == nullptr)
	{
		return;
	}

	//Search for existing binding by property target, and create it only if missing.
	//Not perfect as if there are many bindings, only the first one will be used, but this is the limitation of the design.
	FMVVMBlueprintViewBinding* Binding = MVVMBlueprintView->FindBinding(Widget, Params.PropertyHandle.Get()->GetProperty());
	if (Binding == nullptr)
	{
		Binding = &MVVMBlueprintView->AddDefaultBinding();
	}

	if (LinkedValue.IsValid() && LinkedValue.IsPropertyPath())
	{
		const FMVVMBlueprintPropertyPath& SourcePath = LinkedValue.GetPropertyPath();
		Binding->SourcePath = SourcePath;

		// Generate the destination path from the widget property that we are dropping on.
		FCachedPropertyPath CachedPropertyPath(Params.PropertyHandle->GeneratePathToProperty());
		CachedPropertyPath.Resolve(Widget);

		// Set the destination path.
		FMVVMBlueprintPropertyPath DestinationPropertyPath;
		DestinationPropertyPath.ResetPropertyPath();

		for (int32 SegNum = 0; SegNum < CachedPropertyPath.GetNumSegments(); SegNum++)
		{
			FFieldVariant Field = CachedPropertyPath.GetSegment(SegNum).GetField();
			DestinationPropertyPath.AppendPropertyPath(WidgetBlueprint, UE::MVVM::FMVVMConstFieldVariant(Field));
		}

		if (Widget->GetFName() == WidgetBlueprint->GetFName())
		{
			DestinationPropertyPath.SetSelfContext();
		}
		else
		{
			DestinationPropertyPath.SetWidgetName(Widget->GetFName());
		}
		Binding->DestinationPath = DestinationPropertyPath;
		Binding->BindingType = EMVVMBindingMode::OneWayToDestination;

		if (Conversion.IsValid())
		{
			Subsystem->SetSourceToDestinationConversionFunction(WidgetBlueprint, *Binding, Conversion.ToBlueprintFunctionReference(WidgetBlueprint));
			Subsystem->SetPathForConversionFunctionArgument(WidgetBlueprint, *Binding, Conversion.ParameterId, SourcePath, true);
		}
	}
	else
	{
		MVVMBlueprintView->RemoveBinding(Binding);
	}

	//Refresh after change.
	MVVMBlueprintView->OnBindingsUpdated.Broadcast();
}
}

void FMVVMPropertyBindingExtension::ExtendBindingsMenu(FMenuBuilder& MenuBuilder, TSharedRef<FMVVMPropertyBindingExtension> MVVMPropertyBindingExtension, const UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, TSharedPtr<IPropertyHandle> WidgetPropertyHandle)
{
	const UUMGEditorProjectSettings* UMGEditorProjectSettings = GetMutableDefault<UUMGEditorProjectSettings>();
	if (UMGEditorProjectSettings->bUseNewBindingFromDetailsView)
	{
		return;
	}

	UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	if (MVVMExtensionPtr == nullptr)
	{
		return;
	}
	UMVVMBlueprintView* MVVMBlueprintView = MVVMExtensionPtr->GetBlueprintView();
	if (MVVMBlueprintView == nullptr)
	{
		return;
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	auto CreatePropertyWidget = [Schema, WidgetPropertyHandle, WidgetBlueprint, Widget, MVVMPropertyBindingExtension](const FProperty* Property, FGuid OwningViewModelId ,bool bRequiresConversion)
		-> TSharedRef<SWidget>
	{
		FEdGraphPinType PinType;
		Schema->ConvertPropertyToPinType(Property, PinType);
		const FSlateBrush* SlateBrush = FBlueprintEditorUtils::GetIconFromPin(PinType, true);

		TSharedRef<SHorizontalBox> HorizontalBox =
			SNew(SHorizontalBox)
			.ToolTipText(Property->GetDisplayNameText())
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 5.0f, 0.0f)
			[
				SNew(SImage)
				.Image(FCoreStyle::Get().GetBrush("Icons.Check"))
				.Visibility(MVVMPropertyBindingExtension, &UE::MVVM::FMVVMPropertyBindingExtension::GetCheckmarkVisibility, WidgetBlueprint, Widget, Property, OwningViewModelId, WidgetPropertyHandle)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 5.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(SlateBrush)
				.ColorAndOpacity(Schema->GetPinTypeColor(PinType))
			]
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 5.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Property->GetDisplayNameText())
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			];

		if (bRequiresConversion)
		{
			HorizontalBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Sequencer.CreateEventBinding"))
					.ColorAndOpacity(FSlateColor(EStyleColor::AccentGreen))
				];
		}

		return HorizontalBox;
	};

	auto CreateBinding = [WidgetBlueprint, MVVMBlueprintView](UWidget* Widget, TSharedPtr<IPropertyHandle> WidgetPropertyHandle, FGuid ViewModelId, const FProperty* ViewModelProperty)
	{
		FMVVMBlueprintViewBinding& NewBinding = MVVMBlueprintView->AddDefaultBinding();

		NewBinding.SourcePath.SetViewModelId(ViewModelId);
		NewBinding.SourcePath.SetPropertyPath(WidgetBlueprint, UE::MVVM::FMVVMConstFieldVariant(ViewModelProperty));

		// Generate the destination path from the widget property that we are dropping on.
		FCachedPropertyPath CachedPropertyPath(WidgetPropertyHandle->GeneratePathToProperty());
		CachedPropertyPath.Resolve(Widget);

		// Set the destination path.
		FMVVMBlueprintPropertyPath DestinationPropertyPath;
		DestinationPropertyPath.ResetPropertyPath();

		for (int32 SegNum = 0; SegNum < CachedPropertyPath.GetNumSegments(); SegNum++)
		{
			FFieldVariant Field = CachedPropertyPath.GetSegment(SegNum).GetField();
			DestinationPropertyPath.AppendPropertyPath(WidgetBlueprint, UE::MVVM::FMVVMConstFieldVariant(Field));
		}

		if (Widget->GetFName() == WidgetBlueprint->GetFName())
		{
			DestinationPropertyPath.SetSelfContext();
		}
		else
		{
			DestinationPropertyPath.SetWidgetName(Widget->GetFName());
		}
		NewBinding.DestinationPath = DestinationPropertyPath;

		NewBinding.BindingType = EMVVMBindingMode::OneWayToDestination;

		MVVMBlueprintView->OnBindingsUpdated.Broadcast();
	};

	bool bHasValidViewModel = false;

	for (const FMVVMBlueprintViewModelContext& ViewModel : MVVMBlueprintView->GetViewModels())
	{
		if (ViewModel.GetViewModelClass() == nullptr)
		{
			// invalid viewmodel, possibly just created by the user but not filled in, skip it for now
			continue;
		}

		if (!bHasValidViewModel)
		{
			MenuBuilder.BeginSection("ViewModels", LOCTEXT("ViewModels", "View Models"));
			bHasValidViewModel = true;
		}
		MenuBuilder.AddSubMenu(ViewModel.GetDisplayName(), ViewModel.GetDisplayName(),
			FNewMenuDelegate::CreateLambda([ViewModel, Widget, WidgetPropertyHandle, Schema, CreatePropertyWidget, CreateBinding](FMenuBuilder& MenuBuilder)
				{
					const UClass* ViewModelClass = ViewModel.GetViewModelClass();
					const FProperty* Property = WidgetPropertyHandle->GetProperty();
					auto IsPropertyVisible = [](const FProperty* VMProperty) -> bool
						{
							static FName NAME_BlueprintPrivate = "BlueprintPrivate";
							if (VMProperty->HasMetaData(NAME_BlueprintPrivate))
							{
								return false;
							}
							static FName NAME_BlueprintProtected = "BlueprintProtected";
							if (VMProperty->HasMetaData(NAME_BlueprintProtected))
							{
								return false;
							}
							if (!VMProperty->HasAnyPropertyFlags(CPF_BlueprintVisible | CPF_BlueprintAssignable))
							{
								return false;
							}

							return true;
						};

					MenuBuilder.BeginSection("ValidDataTypes", LOCTEXT("ValidDataTypes", "Valid Data Types"));

					for (TFieldIterator<FProperty> It(ViewModelClass); It; ++It)
					{
						const FProperty* VMProperty = *It;
						if (!IsPropertyVisible(VMProperty))
						{
							continue;
						}

						if (VMProperty->GetClass() != Property->GetClass())
						{
							continue;
						}

						FUIAction UIAction;
						UIAction.ExecuteAction = FExecuteAction::CreateLambda(CreateBinding, Widget, WidgetPropertyHandle, ViewModel.GetViewModelId(), VMProperty);
						MenuBuilder.AddMenuEntry(
							UIAction,
							CreatePropertyWidget(VMProperty, ViewModel.GetViewModelId(), false)
						);
					}

					MenuBuilder.EndSection();

					MenuBuilder.BeginSection("InvalidDataTypes", LOCTEXT("InvalidDataTypes", "Invalid Data Types"));

					for (TFieldIterator<FProperty> It(ViewModelClass); It; ++It)
					{
						const FProperty* VMProperty = *It;
						if (!IsPropertyVisible(VMProperty))
						{
							continue;
						}

						if (VMProperty->GetClass() == Property->GetClass())
						{
							continue;
						}

						FUIAction UIAction;
						UIAction.ExecuteAction = FExecuteAction::CreateLambda(CreateBinding, Widget, WidgetPropertyHandle, ViewModel.GetViewModelId(), VMProperty);
						MenuBuilder.AddMenuEntry(
							UIAction,
							CreatePropertyWidget(VMProperty, ViewModel.GetViewModelId(), true)
						);
					}
				}));
	}

	if (bHasValidViewModel)
	{
		MenuBuilder.EndSection();
	}
}

EVisibility FMVVMPropertyBindingExtension::GetCheckmarkVisibility(const UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, const FProperty* Property, FGuid OwningViewModelId, TSharedPtr<IPropertyHandle> WidgetPropertyHandle) const
{
	const UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	if (MVVMExtensionPtr == nullptr)
	{
		return EVisibility::Hidden;
	}

	const UMVVMBlueprintView* MVVMBlueprintView = MVVMExtensionPtr->GetBlueprintView();
	const FMVVMBlueprintViewBinding* Binding = MVVMBlueprintView ? MVVMBlueprintView->FindBinding(Widget, WidgetPropertyHandle->GetProperty()) : nullptr;
	if (Binding == nullptr)
	{
		return EVisibility::Hidden;
	}

	TArray<UE::MVVM::FMVVMConstFieldVariant> Fields = Binding->SourcePath.GetFields(WidgetBlueprint->GeneratedClass);

	// Currently the bind menu only supports top-level properties in viewmodels, so we can only check the first field to find a match.
	// This should be updated once we are able to expand the full tree of properties in the bind menu.
	if (Fields.Num() > 0 && Fields[0].IsProperty() && Property == Fields[0].GetProperty() && Binding->SourcePath.GetViewModelId() == OwningViewModelId)
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}

TOptional<FName> FMVVMPropertyBindingExtension::GetCurrentValue(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	if (!PropertyHandle.IsValid())
	{
		return TOptional<FName>();
	}

	const UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	if (MVVMExtensionPtr == nullptr)
	{
		return TOptional<FName>();
	}
	const UMVVMBlueprintView* MVVMBlueprintView = MVVMExtensionPtr->GetBlueprintView();

	const FMVVMBlueprintViewBinding* Binding = MVVMBlueprintView ? MVVMBlueprintView->FindBinding(Widget, PropertyHandle->GetProperty()) : nullptr;
	if (Binding == nullptr)
	{
		return TOptional<FName>();
	}

	TArray<FName> Names = Binding->SourcePath.GetFieldNames(WidgetBlueprint->SkeletonGeneratedClass);
	if (Names.Num() > 0)
	{
		return Names.Last();
	}
	if (UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding->Conversion.GetConversionFunction(true))
	{
		FMVVMBlueprintFunctionReference FunctionRef = ConversionFunction->GetConversionFunction();
		if (FunctionRef.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
		{
			const UFunction* FoundFunction = FunctionRef.GetFunction(WidgetBlueprint);
			return FoundFunction ? FName(FoundFunction->GetDisplayNameText().ToString()) : TOptional<FName>();
		}
		else if (FunctionRef.GetType() == EMVVMBlueprintFunctionReferenceType::Node)
		{
			TSubclassOf<UK2Node> FoundFunction = FunctionRef.GetNode();
			if (FoundFunction && FoundFunction->GetDefaultObject<UK2Node>())
			{
				return FName(FoundFunction->GetDefaultObject<UK2Node>()->GetNodeTitle(ENodeTitleType::ListView).ToString());
			}
			return FoundFunction.Get() ? FName(FoundFunction->GetDisplayNameText().ToString()) : TOptional<FName>();
		}
	}
	return TOptional<FName>();
}

const FSlateBrush* FMVVMPropertyBindingExtension::GetCurrentIcon(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	if (!PropertyHandle.IsValid())
	{
		return nullptr;
	}

	const UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	if (MVVMExtensionPtr == nullptr)
	{
		return nullptr;
	}
	const UMVVMBlueprintView* MVVMBlueprintView = MVVMExtensionPtr->GetBlueprintView();

	const FMVVMBlueprintViewBinding* Binding = MVVMBlueprintView ? MVVMBlueprintView->FindBinding(Widget, PropertyHandle->GetProperty()) : nullptr;
	if (Binding == nullptr)
	{
		return nullptr;
	}

	TArray<UE::MVVM::FMVVMConstFieldVariant> Fields = Binding->SourcePath.GetFields(WidgetBlueprint->SkeletonGeneratedClass);
	if (Fields.IsEmpty())
	{
		return nullptr;
	}

	UE::MVVM::FMVVMConstFieldVariant Field = Fields.Last();
	if (Field.IsFunction() && Field.GetFunction() != nullptr)
	{
		return FAppStyle::Get().GetBrush("GraphEditor.Function_16x");
	}
	else if (Field.IsProperty() && Field.GetProperty() != nullptr)
	{
		FSlateColor PrimaryColor, SecondaryColor;
		const FSlateBrush* SecondaryBrush = nullptr;
		return FBlueprintEditor::GetVarIconAndColorFromProperty(Field.GetProperty(), PrimaryColor, SecondaryBrush, SecondaryColor);
	}

	return nullptr;
}

TOptional<FLinearColor> FMVVMPropertyBindingExtension::GetCurrentIconColor(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	if (!PropertyHandle.IsValid())
	{
		return TOptional<FLinearColor>();
	}

	const UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	if (MVVMExtensionPtr == nullptr)
	{
		return TOptional<FLinearColor>();
	}
	const UMVVMBlueprintView* MVVMBlueprintView = MVVMExtensionPtr->GetBlueprintView();

	const FMVVMBlueprintViewBinding* Binding = MVVMBlueprintView ? MVVMBlueprintView->FindBinding(Widget, PropertyHandle->GetProperty()) : nullptr;
	if (Binding == nullptr)
	{
		return TOptional<FLinearColor>();
	}

	TArray<UE::MVVM::FMVVMConstFieldVariant> Fields = Binding->SourcePath.GetFields(WidgetBlueprint->GeneratedClass);
	if (Fields.IsEmpty())
	{
		return TOptional<FLinearColor>();
	}

	UE::MVVM::FMVVMConstFieldVariant Field = Fields.Last();
	const FProperty* IconProperty = nullptr;

	if (Field.IsProperty())
	{
		IconProperty = Field.GetProperty();
	}
	else if (Field.IsFunction())
	{
		const UFunction* Function = Field.GetFunction();
		const FProperty* ReturnProperty = BindingHelper::GetReturnProperty(Function);
		if (ReturnProperty != nullptr)
		{
			IconProperty = ReturnProperty;
		}
		else
		{
			IconProperty = BindingHelper::GetFirstArgumentProperty(Function);
		}
	}
	if (IconProperty != nullptr)
	{
		FSlateColor PrimaryColor, SecondaryColor;
		const FSlateBrush* SecondaryBrush = nullptr;
		const FSlateBrush* PrimaryBrush = FBlueprintEditor::GetVarIconAndColorFromProperty(IconProperty, PrimaryColor, SecondaryBrush, SecondaryColor);
		return PrimaryColor.GetSpecifiedColor();
	}

	return TOptional<FLinearColor>();
}

bool FMVVMPropertyBindingExtension::ShouldAllowEditingValue(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	const UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	if (MVVMExtensionPtr == nullptr)
	{
		return false;
	}
	const UMVVMBlueprintView* MVVMBlueprintView = MVVMExtensionPtr->GetBlueprintView();

	const FMVVMBlueprintViewBinding* Binding = MVVMBlueprintView ? MVVMBlueprintView->FindBinding(Widget, PropertyHandle->GetProperty()) : nullptr;
	if (Binding == nullptr)
	{
		return false;
	}

	return Binding->bCompile;
}

void FMVVMPropertyBindingExtension::ClearCurrentValue(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if (PropertyHandle.IsValid())
	{
		if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint))
		{
			if (UMVVMBlueprintView* MVVMBlueprintView = MVVMExtensionPtr->GetBlueprintView())
			{
				if (FMVVMBlueprintViewBinding* Binding = MVVMBlueprintView->FindBinding(Widget, PropertyHandle->GetProperty()))
				{
					FScopedTransaction Transaction(LOCTEXT("DeleteBindingsTransaction", "Delete Binding"));
					MVVMBlueprintView->Modify();
					MVVMBlueprintView->RemoveBinding(Binding);
				}
			}
		}
	}
}

TSharedPtr<FExtender> FMVVMPropertyBindingExtension::CreateMenuExtender(const UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, TSharedPtr<IPropertyHandle> WidgetPropertyHandle)
{
	TSharedPtr<FExtender> Extender = MakeShared<FExtender>();
	Extender->AddMenuExtension("BindingActions", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateStatic(&ExtendBindingsMenu, MakeShared<FMVVMPropertyBindingExtension>(*this), WidgetBlueprint, Widget, WidgetPropertyHandle));
	return Extender;
}

bool FMVVMPropertyBindingExtension::CanExtend(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<const IPropertyHandle> PropertyHandle) const
{
	const UMVVMDeveloperProjectSettings* MVVMSettings = GetDefault<UMVVMDeveloperProjectSettings>();
	if (!MVVMSettings->bAllowBindingFromDetailView)
	{
		return false;
	}

	const UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	if (MVVMExtensionPtr == nullptr)
	{
		return false;
	}

	const bool NeedToRequestView = MVVMSettings->bAllowBindingEditingWithoutViewModel;
	if (NeedToRequestView)
	{	//Make a view available from start without waiting for a view model to be added.
		GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->RequestView(const_cast<UWidgetBlueprint*>(WidgetBlueprint));
	}

	const UMVVMBlueprintView* MVVMBlueprintView = MVVMExtensionPtr->GetBlueprintView();
	if (MVVMBlueprintView != nullptr)
	{
		if (MVVMBlueprintView->GetViewModels().Num() > 0 || MVVMBlueprintView->GetNumBindings() > 0 || MVVMSettings->bAllowBindingEditingWithoutViewModel)
		{
			return true;
		}
	}

	return false;
}

IPropertyBindingExtension::EDropResult FMVVMPropertyBindingExtension::OnDrop(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent, UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, TSharedPtr<IPropertyHandle> WidgetPropertyHandle)
{
	UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	if (MVVMExtensionPtr == nullptr)
	{
		return EDropResult::Unhandled;
	}
	UMVVMBlueprintView* MVVMBlueprintView = MVVMExtensionPtr->GetBlueprintView();
	if (MVVMBlueprintView == nullptr)
	{
		return EDropResult::Unhandled;
	}

	if (TSharedPtr<UE::MVVM::FViewModelFieldDragDropOp> ViewModelFieldDragDropOp = DragDropEvent.GetOperationAs<UE::MVVM::FViewModelFieldDragDropOp>())
	{
		UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		FMVVMBlueprintViewBinding& NewBinding = EditorSubsystem->AddBinding(WidgetBlueprint);

		TArray<FFieldVariant> SourceFieldPath = ViewModelFieldDragDropOp->DraggedField;

		// Set the source path (view model property from the drop event).
		FMVVMBlueprintPropertyPath SourcePropertyPath;
		SourcePropertyPath.ResetPropertyPath();
		for (const FFieldVariant& Field : SourceFieldPath)
		{
			SourcePropertyPath.AppendPropertyPath(WidgetBlueprint, UE::MVVM::FMVVMConstFieldVariant(Field));
		}
		if (ViewModelFieldDragDropOp->ViewModelId.IsValid())
		{
			SourcePropertyPath.SetViewModelId(ViewModelFieldDragDropOp->ViewModelId);
		}

		NewBinding.SourcePath = SourcePropertyPath;
		EditorSubsystem->SetSourcePathForBinding(WidgetBlueprint, NewBinding, SourcePropertyPath);

		// Generate the destination path from the widget property that we are dropping on.
		FCachedPropertyPath CachedPropertyPath(WidgetPropertyHandle->GeneratePathToProperty());
		CachedPropertyPath.Resolve(Widget);

		// Set the destination path.
		FMVVMBlueprintPropertyPath DestinationPropertyPath;
		DestinationPropertyPath.ResetPropertyPath();

		for (int32 SegNum = 0; SegNum < CachedPropertyPath.GetNumSegments(); SegNum++)
		{
			FFieldVariant Field = CachedPropertyPath.GetSegment(SegNum).GetField();
			DestinationPropertyPath.AppendPropertyPath(WidgetBlueprint, UE::MVVM::FMVVMConstFieldVariant(Field));
		}

		if (Widget->GetFName() == WidgetBlueprint->GetFName())
		{
			DestinationPropertyPath.SetSelfContext();
		}
		else
		{
			DestinationPropertyPath.SetWidgetName(Widget->GetFName());
		}
		EditorSubsystem->SetDestinationPathForBinding(WidgetBlueprint, NewBinding, DestinationPropertyPath, false);

		return EDropResult::HandledContinue;
	}
	return EDropResult::Unhandled;
}

TSharedRef<SWidget> FMVVMPropertyBindingExtension::OnGenerateMenu(const UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle, TWeakPtr<IDetailsView> DetailsView)
{
	UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	if (MVVMExtensionPtr == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	UMVVMBlueprintView* MVVMBlueprintView = MVVMExtensionPtr->GetBlueprintView();
	if (MVVMBlueprintView == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	Private::FCreatePropertyBindingParams CreateBindingParams;
	CreateBindingParams.WidgetBlueprint = TWeakObjectPtr<const UWidgetBlueprint>(WidgetBlueprint);
	CreateBindingParams.MVVMBlueprintView = TWeakObjectPtr<UMVVMBlueprintView>(MVVMBlueprintView);
	CreateBindingParams.Widget = TWeakObjectPtr<UWidget>(Widget);
	CreateBindingParams.PropertyHandle = PropertyHandle;

	SFieldSelectorMenu::FOnLinkedValueConversionSelected HandleFieldConversionSelectionChanged = SFieldSelectorMenu::FOnLinkedValueConversionSelected::CreateLambda(
		[CreateBindingParams, DetailsView]
		(FMVVMLinkedPinValue LinkedValue, const FMVVMFieldConversion& Conversion, SFieldSelectorMenu::ESelectionType SelectionType)
		{
			FSlateApplication::Get().DismissAllMenus();

			Private::CreateBindingFromProperty(CreateBindingParams, LinkedValue, Conversion);

			IDetailsView* DetailsViewPtr = DetailsView.Pin().Get();
			if (DetailsViewPtr != nullptr)
			{
				DetailsViewPtr->ForceRefresh();
			}
		});

	SFieldSelectorMenu::FOnLinkedValueSelected HandleFieldSelectionChanged = SFieldSelectorMenu::FOnLinkedValueSelected::CreateLambda(
		[HandleFieldConversionSelectionChanged]
		(FMVVMLinkedPinValue LinkedValue, const SFieldSelectorMenu::ESelectionType SelectionType)
		{
			HandleFieldConversionSelectionChanged.ExecuteIfBound(LinkedValue, FMVVMFieldConversion(), SelectionType);
		});

	FSimpleDelegate HandleMenuClosed = FSimpleDelegate::CreateLambda([]()
	{
		FSlateApplication::Get().DismissAllMenus();
	});

	TOptional<FMVVMLinkedPinValue> CurrentSelected;
	FMVVMBlueprintPropertyPath PropertyPath;
	PropertyPath.SetWidgetName(WidgetBlueprint->GetFName());
	CurrentSelected = FMVVMLinkedPinValue(PropertyPath);

	FFieldSelectionContext SelectionContext;
	SelectionContext.BindingMode = EMVVMBindingMode::OneWayToDestination;
	SelectionContext.AssignableTo = PropertyHandle.Get()->GetProperty();

	//Right before displaying the menu (as the value can be changed anytime from UI), ensure the list of properties is filtered with compatible types only.
	FMVVMViewBindingFilterSettings FilterSettings = MVVMExtensionPtr->GetFilterSettings();
	FilterSettings.FilterFlags = EFilterFlag::All;
	MVVMExtensionPtr->SetFilterSettings(FilterSettings);

	const bool bHasBinding = MVVMBlueprintView->FindBinding(Widget, PropertyHandle.Get()->GetProperty()) != nullptr;

	TSharedRef<SFieldSelectorMenu> Menu = SNew(SFieldSelectorMenu, WidgetBlueprint)
		.CurrentSelected(CurrentSelected)
		.OnSelected(HandleFieldSelectionChanged)
		.OnSelectedConversion(HandleFieldConversionSelectionChanged)
		.OnMenuCloseRequested(HandleMenuClosed)
		.SelectionContext(SelectionContext)
		.IsBindingToEvent(false)
		.CanCreateEvent(false)
		.IsEnableContextToggleAvailable(false)
		.IsClearEnableable(bHasBinding)
		.CanConvertField(true)
		.ShowTooltipText(true);

	return Menu;
}

}
#undef LOCTEXT_NAMESPACE
