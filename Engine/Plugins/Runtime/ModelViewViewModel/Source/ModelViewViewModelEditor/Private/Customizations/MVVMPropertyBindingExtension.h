// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IHasPropertyBindingExtensibility.h"

class UWidgetBlueprint;
class UWidget;
struct EVisibility;

namespace UE::MVVM
{

class FMVVMPropertyBindingExtension
	: public IPropertyBindingExtension
{
	static void ExtendBindingsMenu(FMenuBuilder& MenuBuilder, TSharedRef<FMVVMPropertyBindingExtension> MVVMPropertyBindingExtension, const UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, TSharedPtr<IPropertyHandle> WidgetPropertyHandle);

	virtual TOptional<FName> GetCurrentValue(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const override;
	virtual const FSlateBrush* GetCurrentIcon(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const override;
	virtual TOptional<FLinearColor> GetCurrentIconColor(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const override;

	virtual void ClearCurrentValue(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) override;
	virtual TSharedPtr<FExtender> CreateMenuExtender(const UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) override;
	virtual bool CanExtend(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<const IPropertyHandle> PropertyHandle) const override;
	virtual EDropResult OnDrop(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent, UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) override;
	virtual TSharedRef<SWidget> OnGenerateMenu(const UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle, TWeakPtr<IDetailsView> DetailsView) override;
	virtual bool ShouldAllowEditingValue(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const override;

	EVisibility GetCheckmarkVisibility(const UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, const FProperty* Property, FGuid OwningViewModelId, TSharedPtr<IPropertyHandle> PropertyHandle) const;
};

}
