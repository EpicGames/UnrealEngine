// Copyright Epic Games, Inc. All Rights Reserved.
#include "MVVMPropertyPathCustomization.h"

#include "DetailWidgetRow.h"
#include "Editor.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMEditorSubsystem.h"
#include "WidgetBlueprint.h"
#include "Widgets/SMVVMFieldDisplay.h"

#define LOCTEXT_NAMESPACE "MVVMPropertyPathCustomization"

namespace UE::MVVM
{
FPropertyPathCustomization::FPropertyPathCustomization(UWidgetBlueprint* InWidgetBlueprint) :
	WidgetBlueprint(InWidgetBlueprint)
{
	check(WidgetBlueprint.IsValid());
}

FPropertyPathCustomization::~FPropertyPathCustomization()
{
}

void FPropertyPathCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	const FName PropertyName = PropertyHandle->GetProperty()->GetFName();
	FText PropertyDisplayName;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, DestinationPath))
	{
		bIsSource = false;
		PropertyDisplayName = LOCTEXT("PropertyDisplay_Widget", "Destination");
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, SourcePath))
	{
		bIsSource = true;
		PropertyDisplayName = LOCTEXT("PropertyDisplay_Source", "Source");
	}
	else
	{
		ensureMsgf(false, TEXT("MVVMPropertyPathCustomization used in unknown context."));
	}

	FMVVMBlueprintViewBinding* OwningBinding = GetOwningBinding();
	OwningBindingId = OwningBinding ? OwningBinding->BindingId : FGuid();

	HeaderRow
		.NameWidget
		.VAlign(VAlign_Center)
		[
			PropertyHandle->CreatePropertyNameWidget(PropertyDisplayName)
		]
	.ValueWidget
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		[
			SNew(SFieldDisplay, WidgetBlueprint.Get())
			.OnGetLinkedValue(this, &FPropertyPathCustomization::GetFieldValue)
			.TextStyle(FAppStyle::Get(), "SmallText")
		];
}

FMVVMBlueprintPropertyPath FPropertyPathCustomization::GetPropertyPathValue() const
{
	check(PropertyHandle->GetProperty()->GetOwnerStruct() == FMVVMBlueprintViewBinding::StaticStruct());
	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);
	if (RawData.Num() > 1)
	{
		return FMVVMBlueprintPropertyPath();
	}

	for (void* RawPtr : RawData)
	{
		if (RawPtr == nullptr)
		{
			continue;
		}
		FMVVMBlueprintPropertyPath* PropertyPath = reinterpret_cast<FMVVMBlueprintPropertyPath*>(RawPtr);
		if (PropertyPath)
		{
			return *PropertyPath;
		}
	}
	return FMVVMBlueprintPropertyPath();
}

FMVVMBlueprintViewBinding* FPropertyPathCustomization::GetOwningBinding() const
{
	if (TSharedPtr<IPropertyHandle> ParentHandle = PropertyHandle->GetParentHandle())
	{
		TArray<void*> RawData;
		ParentHandle->AccessRawData(RawData);
		for (void* Data : RawData)
		{
			FMVVMBlueprintViewBinding* Binding = reinterpret_cast<FMVVMBlueprintViewBinding*>(Data);
			if (Binding)
			{
				return Binding;
			}
		}
	}
	return nullptr;
}

FMVVMLinkedPinValue FPropertyPathCustomization::GetFieldValue() const
{
	if (OwningBindingId.IsValid())
	{
		if (UWidgetBlueprint* WidgetBP = WidgetBlueprint.Get())
		{
			UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
			if (const UMVVMBlueprintView* View = EditorSubsystem->GetView(WidgetBlueprint.Get()))
			{
				if (const FMVVMBlueprintViewBinding* OwningBinding = View->GetBinding(OwningBindingId))
				{
					if (UMVVMBlueprintViewConversionFunction* ConversionFunction = OwningBinding->Conversion.GetConversionFunction(bIsSource))
					{
						return FMVVMLinkedPinValue(WidgetBP, ConversionFunction->GetConversionFunction());
					}
					else
					{
						return FMVVMLinkedPinValue(GetPropertyPathValue());
					}
				}
			}
		}
	}
	return FMVVMLinkedPinValue(FMVVMBlueprintPropertyPath());
}

} // namespace UE::MVVM
#undef LOCTEXT_NAMESPACE