// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "IPropertyTypeCustomization.h"
#include "Types/MVVMLinkedPinValue.h"

struct FMVVMBlueprintPropertyPath;
struct FMVVMBlueprintViewBinding;

namespace UE::MVVM
{
class FPropertyPathCustomization : public IPropertyTypeCustomization
{
public:
	FPropertyPathCustomization(UWidgetBlueprint* WidgetBlueprint);
	~FPropertyPathCustomization();
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	static TSharedRef<IPropertyTypeCustomization> MakeInstance(UWidgetBlueprint* InWidgetBlueprint)
	{
		check(InWidgetBlueprint != nullptr);
		return MakeShared<FPropertyPathCustomization>(InWidgetBlueprint);
	}

	FMVVMLinkedPinValue GetFieldValue() const;
	FMVVMBlueprintViewBinding* GetOwningBinding() const;

private:
	FMVVMBlueprintPropertyPath GetPropertyPathValue() const;

private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TWeakObjectPtr<UWidgetBlueprint> WidgetBlueprint;
	bool bIsSource = false;
	FGuid OwningBindingId;
};
} // namespace UE::MVVM