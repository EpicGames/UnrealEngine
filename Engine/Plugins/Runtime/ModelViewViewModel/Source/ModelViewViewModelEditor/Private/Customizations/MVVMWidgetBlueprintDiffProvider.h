// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "DiffResults.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewBinding.h"
#include "MVVMBlueprintViewCondition.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMWidgetBlueprintDiff.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "Widgets/SMVVMViewBindingListView.h"

class FMVVMDiffCustomObject_View : public IDiffCustomObject
{
public:
	TSharedPtr<UE::MVVM::SBindingsList> OldList;
	TSharedPtr<UE::MVVM::SBindingsList> NewList;

	FMVVMDiffCustomObject_View(const UWidgetBlueprint* NewBlueprint, const UWidgetBlueprint* OldBlueprint)
	{
		if (OldBlueprint != nullptr)
		{
			if (UMVVMWidgetBlueprintExtension_View* OldExtension = UWidgetBlueprintExtension::GetExtension<UMVVMWidgetBlueprintExtension_View>(OldBlueprint))
			{
				OldList = SNew(UE::MVVM::SBindingsList, nullptr, nullptr, OldExtension);
			}
		}

		if (NewBlueprint != nullptr)
		{
			if (UMVVMWidgetBlueprintExtension_View* NewExtension = UWidgetBlueprintExtension::GetExtension<UMVVMWidgetBlueprintExtension_View>(NewBlueprint))
			{
				NewList = SNew(UE::MVVM::SBindingsList, nullptr, nullptr, NewExtension);
			}
		}
	}

	TSharedPtr<SWidget> GetOldWidget() override { return OldList; }
	TSharedPtr<SWidget> GetNewWidget() override { return NewList; }
};

class FMVVMDiffCustomProperty_Binding : public IDiffCustomProperty
{
public:
	FGuid NewBinding;
	FGuid OldBinding;

	FMVVMDiffCustomProperty_Binding(const FGuid NewBinding, const FGuid OldBinding)
		: NewBinding(NewBinding)
		, OldBinding(OldBinding)
	{
	}

	void HighlightProperty(const TSharedPtr<IDiffCustomObject> Widget) override
	{
		if (FMVVMDiffCustomObject_View* DiffWidget = StaticCast<FMVVMDiffCustomObject_View*>(Widget.Get()))
		{
			if (DiffWidget->NewList.IsValid())
			{
				DiffWidget->NewList->SelectBinding(NewBinding, true);
			}
			if (DiffWidget->OldList.IsValid())
			{
				DiffWidget->OldList->SelectBinding(OldBinding, true);
			}
		}
	}
};

class FMVVMDiffCustomProperty_Event : public IDiffCustomProperty
{
public:
	const UMVVMBlueprintViewEvent* NewEvent;
	const UMVVMBlueprintViewEvent* OldEvent;

	FMVVMDiffCustomProperty_Event(const UMVVMBlueprintViewEvent* NewEvent, const UMVVMBlueprintViewEvent* OldEvent)
		: NewEvent(NewEvent)
		, OldEvent(OldEvent)
	{
	}

	void HighlightProperty(const TSharedPtr<IDiffCustomObject> Widget) override
	{
		if (FMVVMDiffCustomObject_View* DiffWidget = StaticCast<FMVVMDiffCustomObject_View*>(Widget.Get()))
		{
			if (DiffWidget->NewList.IsValid())
			{
				DiffWidget->NewList->SelectEvent(NewEvent, true);
			}
			if (DiffWidget->OldList.IsValid())
			{
				DiffWidget->OldList->SelectEvent(OldEvent, true);
			}
		}
	}
};

class FMVVMDiffCustomProperty_Condition : public IDiffCustomProperty
{
public:
	const UMVVMBlueprintViewCondition* NewCondition;
	const UMVVMBlueprintViewCondition* OldCondition;

	FMVVMDiffCustomProperty_Condition(const UMVVMBlueprintViewCondition* NewCondition, const UMVVMBlueprintViewCondition* OldCondition)
		: NewCondition(NewCondition)
		, OldCondition(OldCondition)
	{
	}

	void HighlightProperty(const TSharedPtr<IDiffCustomObject> Widget) override
	{
		if (FMVVMDiffCustomObject_View* DiffWidget = StaticCast<FMVVMDiffCustomObject_View*>(Widget.Get()))
		{
			if (DiffWidget->NewList.IsValid())
			{
				DiffWidget->NewList->SelectCondition(NewCondition, true);
			}
			if (DiffWidget->OldList.IsValid())
			{
				DiffWidget->OldList->SelectCondition(OldCondition, true);
			}
		}
	}
};

class FMVVMDiffCustomProperty_Parameter : public IDiffCustomProperty
{
public:
	FMVVMBlueprintPinId NewParameter;
	FMVVMBlueprintPinId OldParameter;

	FMVVMDiffCustomProperty_Parameter(FMVVMBlueprintPinId NewParameter, FMVVMBlueprintPinId OldParameter)
		: NewParameter(NewParameter)
		, OldParameter(OldParameter)
	{
	}

	void HighlightProperty(const TSharedPtr<IDiffCustomObject> Widget) override
	{
		if (FMVVMDiffCustomObject_View* DiffWidget = StaticCast<FMVVMDiffCustomObject_View*>(Widget.Get()))
		{
			DiffWidget->NewList->SelectParameter(NewParameter, true);
			DiffWidget->OldList->SelectParameter(OldParameter, true);
		}
	}
};

struct FMVVMWidgetBlueprintDiffProvider : public UE::MVVM::FMVVMDiffCustomObjectProvider
{
	TSharedPtr<IDiffCustomObject> CreateObjectDiff(const UWidgetBlueprint* NewBlueprint, const UWidgetBlueprint* OldBlueprint)
	{
		return MakeShared<FMVVMDiffCustomObject_View>(NewBlueprint, OldBlueprint);
	}
	
	TSharedPtr<IDiffCustomProperty> CreatePropertyBindingDiff(FGuid NewBinding, FGuid OldBinding)
	{
		return MakeShared<FMVVMDiffCustomProperty_Binding>(NewBinding, OldBinding);
	}
	
	TSharedPtr<IDiffCustomProperty> CreatePropertyConditionDiff(const UMVVMBlueprintViewCondition* NewCondition, const UMVVMBlueprintViewCondition* OldCondition)
	{
		return MakeShared<FMVVMDiffCustomProperty_Condition>(NewCondition, OldCondition);
	}
	
	TSharedPtr<IDiffCustomProperty> CreatePropertyConditionDiff(const UMVVMBlueprintViewEvent* NewEvent, const UMVVMBlueprintViewEvent* OldEvent)
	{
		return MakeShared<FMVVMDiffCustomProperty_Event>(NewEvent, OldEvent);
	}

	TSharedPtr<IDiffCustomProperty> CreatePropertyParameterDiff(const FMVVMBlueprintPinId& NewParameter, const FMVVMBlueprintPinId& OldParameter)
	{
		return MakeShared<FMVVMDiffCustomProperty_Parameter>(NewParameter, OldParameter);
	}
};
