// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MVVMWidgetContextMenuExtension.h"
#include "Framework/MVVMBindingEditorHelper.h"
#include "MVVMEditorSubsystem.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"

#define LOCTEXT_NAMESPACE "FMVVMBindingEditorHelper"

namespace UE::MVVM
{
namespace Private
{

void ExecuteCreateWidgetBindings(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor)
{
	TSet<FWidgetReference> Widgets = BlueprintEditor->GetSelectedWidgets();
	UWidgetBlueprint* Blueprint = BlueprintEditor->GetWidgetBlueprintObj();

	TArray<FGuid> BindingIds;
	UE::MVVM::FMVVMBindingEditorHelper::CreateWidgetBindings(Blueprint, Widgets, BindingIds);
}

bool CanCreateWidgetBindings(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor)
{
	TSet<FWidgetReference> Widgets = BlueprintEditor->GetSelectedWidgets();
	if (Widgets.IsEmpty())
	{
		return false;
	}

	UWidgetBlueprint* Blueprint = BlueprintEditor->GetWidgetBlueprintObj();
	if (Blueprint == nullptr)
	{
		return false;
	}

	UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	if (EditorSubsystem == nullptr)
	{
		return false;
	}

	return true;
}

}

void FWidgetContextMenuExtension::ExtendContextMenu(FMenuBuilder& MenuBuilder, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, FVector2D TargetLocation) const
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("MVVM_CreateBinding", "Create Widget Binding"),
		LOCTEXT("MVVM_CreateBindingTooltip", "Creates View Binding(s) for the currently selected widgets"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(&Private::ExecuteCreateWidgetBindings, BlueprintEditor),
			FCanExecuteAction(),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateStatic(&Private::CanCreateWidgetBindings, BlueprintEditor)
		)
	);
}
}

#undef LOCTEXT_NAMESPACE
