// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IHasWidgetContextMenuExtensibility.h"

class FMenuBuilder;
class FWidgetBlueprintEditor;

namespace UE::MVVM
{
class FWidgetContextMenuExtension : public IWidgetContextMenuExtension
{
	//~ Begin IWidgetContextMenuExtension overrides
	virtual void ExtendContextMenu(FMenuBuilder& MenuBuilder, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, FVector2D TargetLocation) const override;
	//~ End IWidgetContextMenuExtension overrides
};
}
