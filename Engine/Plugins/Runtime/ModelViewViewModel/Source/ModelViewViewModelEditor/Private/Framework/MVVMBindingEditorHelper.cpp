// Copyright Epic Games, Inc. All Rights Reserved.
#include "MVVMBindingEditorHelper.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MVVMBlueprintViewBinding.h"
#include "MVVMEditorSubsystem.h"
#include "MVVMPropertyPath.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"

namespace UE::MVVM
{

bool FMVVMBindingEditorHelper::CreateWidgetBindings(UWidgetBlueprint* Blueprint, TSet<FWidgetReference> Widgets, TArray<FGuid>& OutBindingIds)
{
	if (Blueprint == nullptr)
	{
		return false;
	}

	UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	if (EditorSubsystem == nullptr)
	{
		return false;
	}

	for (const FWidgetReference& WidgetReference : Widgets)
	{
		if (WidgetReference.IsValid() && WidgetReference.GetTemplate())
		{
			FMVVMBlueprintViewBinding& Binding = EditorSubsystem->AddBinding(Blueprint);

			FMVVMBlueprintPropertyPath Path;
			if (WidgetReference.GetTemplate()->GetFName() == Blueprint->GetFName())
			{
				Path.SetSelfContext();
			}
			else
			{
				Path.SetWidgetName(WidgetReference.GetTemplate()->GetFName());
			}

			EditorSubsystem->SetDestinationPathForBinding(Blueprint, Binding, Path, false);
			OutBindingIds.Add(Binding.BindingId);
		}
	}

	return !OutBindingIds.IsEmpty();
}

} // namespace UE::MVVM
