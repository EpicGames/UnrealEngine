// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

struct FWidgetReference;

class FMenuBuilder;
class FWidgetBlueprintEditor;
class UWidgetBlueprint;

namespace UE::MVVM
{
struct FMVVMBindingEditorHelper
{
public:
	static bool CreateWidgetBindings(UWidgetBlueprint* Blueprint, TSet<FWidgetReference> Widgets, TArray<FGuid>& OutBindingIds);

};
} // namespace UE::MVVM