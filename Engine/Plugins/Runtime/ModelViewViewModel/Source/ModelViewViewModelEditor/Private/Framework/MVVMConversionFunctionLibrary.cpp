// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MVVMConversionFunctionLibrary.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Bindings/MVVMConversionFunctionHelper.h"
#include "Blueprint/UserWidget.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMEditorSubsystem.h"
#include "WidgetBlueprint.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "BlueprintActionDatabase.h"
#include "Editor.h"
#include "EdGraphSchema_K2.h"
#include "Editor/EditorEngine.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/Class.h"


#define LOCTEXT_NAMESPACE "MVVMConversionFunctionLibrary"

extern UNREALED_API class UEditorEngine* GEditor;

namespace UE::MVVM::ConversionFunctionLibrary
{

namespace Private
{
/**
 *
 */
struct FFunctionEntry_UFunction : public FFunctionEntry
{
	using FFunctionEntry::FFunctionEntry;

public:
	static TSharedPtr<FFunctionEntry_UFunction> Create(const UFunction* InFunction, const FProperty* InReturnValue, TArray<const FProperty*>&& InArguments)
	{
		if (InFunction == nullptr)
		{
			return TSharedPtr<FFunctionEntry_UFunction>();
		}

		TSharedRef<FFunctionEntry_UFunction> Item = MakeShared<FFunctionEntry_UFunction>(FConversionFunctionValue(InFunction));
		Item->ReturnValue = InReturnValue;
		Item->Arguments = InArguments;

		return Item;
	}
	
	TArray<const FProperty*> Arguments;
	const FProperty* ReturnValue = nullptr;
};

/**
 *
 */
struct FFunctionEntry_Node : public FFunctionEntry
{
	using FFunctionEntry::FFunctionEntry;

public:
	static TSharedPtr<FFunctionEntry_Node> Create(const TSubclassOf<UK2Node> InFunction, const UEdGraphPin* InReturnValue, TArray<UEdGraphPin*>&& InArguments)
	{
		if (InFunction == nullptr)
		{
			return TSharedPtr<FFunctionEntry_Node>();
		}

		TSharedRef<FFunctionEntry_Node> Item = MakeShared<FFunctionEntry_Node>(FConversionFunctionValue(InFunction));
		Item->ReturnValue = InReturnValue;
		Item->Arguments = InArguments;

		return Item;
	}
	
	TArray<UEdGraphPin*> Arguments;
	const UEdGraphPin* ReturnValue = nullptr;
};

} //namespace Private


FFunctionEntry::FFunctionEntry(FConversionFunctionValue Function)
	: ConversionFunctionValue(Function)
{
}

FCollection::FCollection()
{
	FCoreUObjectDelegates::OnAssetLoaded.AddRaw(this, &FCollection::HandleObjectLoaded);

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.OnAssetAdded().AddRaw(this, &FCollection::HandleAssetAdded);
	AssetRegistry.OnAssetRemoved().AddRaw(this, &FCollection::HandleAssetRemoved);
	AssetRegistry.OnAssetRenamed().AddRaw(this, &FCollection::HandleAssetRenamed);

	FEditorDelegates::OnAssetsPreDelete.AddRaw(this, &FCollection::HandleObjectPendingDelete);
	FKismetEditorUtilities::OnBlueprintUnloaded.AddRaw(this, &FCollection::HandleBlueprintUnloaded);

	FModuleManager::Get().OnModulesChanged().AddRaw(this, &FCollection::HandleModulesChanged);
	FCoreUObjectDelegates::ReloadCompleteDelegate.AddRaw(this, &FCollection::HandleReloadComplete);

	GetMutableDefault<UMVVMDeveloperProjectSettings>()->OnLibrarySettingChanged.AddRaw(this, &FCollection::Rebuild);
}

FCollection::~FCollection()
{
	FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
	{
		if (IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet())
		{
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
			AssetRegistry->OnAssetRenamed().RemoveAll(this);
		}
	}

	FEditorDelegates::OnAssetsPreDelete.RemoveAll(this);
	FKismetEditorUtilities::OnBlueprintUnloaded.RemoveAll(this);

	FModuleManager::Get().OnModulesChanged().RemoveAll(this);
	FCoreUObjectDelegates::ReloadCompleteDelegate.RemoveAll(this);

	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		GetMutableDefault<UMVVMDeveloperProjectSettings>()->OnLibrarySettingChanged.RemoveAll(this);

		for (TPair<FObjectKey, FFunctionContainer> PairClass : ClassOrBlueprintToFunctions)
		{
			if (PairClass.Value.bIsUserWidget)
			{
				UnegisterBlueprintCallback(PairClass.Key.ResolveObjectPtr());
			}
		}
	}
}

void FCollection::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ConversionFunctionNodes);
}

FString FCollection::GetReferencerName() const
{
	return TEXT("MVVMConversionFunctionLibrary::FCollection");
}

void FCollection::RefreshIfNeeded()
{
	if (bRefreshAll)
	{
		for (TPair<FObjectKey, FFunctionContainer> PairClass : ClassOrBlueprintToFunctions)
		{
			if (PairClass.Value.bIsUserWidget)
			{
				UnegisterBlueprintCallback(PairClass.Key.ResolveObjectPtr());
			}
		}

		ClassOrBlueprintToFunctions.Empty();
		ObjectToRefresh.Empty();
		ModuleToRefresh.Empty();

		NumberOfFunctions = 0;
		bRefreshAll = false;
		Build();
	}

	if (ModuleToRefresh.Num() > 0 || ObjectToRefresh.Num() > 0)
	{
		TArray<const UClass*> AllowedClasses = GetDefault<UMVVMDeveloperProjectSettings>()->GetAllowedConversionFunctionClasses();
		TArray<const UClass*> DeniedClasses = GetDefault<UMVVMDeveloperProjectSettings>()->GetDeniedConversionFunctionClasses();
		for (FName ModuleName : ModuleToRefresh)
		{		
			const FName ModuleScriptPackageName = FPackageName::GetModuleScriptPackageName(ModuleName);
			if (GetDefault<UMVVMDeveloperProjectSettings>()->DeniedModuleForConversionFunctions.Contains(ModuleScriptPackageName))
			{
				continue;
			}
			
			if (const UPackage* ModuleScriptPackage = FindPackage(nullptr, *ModuleScriptPackageName.ToString()))
			{
				TArray<UObject*> ObjectsToProcess;
				GetObjectsWithPackage(ModuleScriptPackage, ObjectsToProcess, EGetObjectsFlags::None, RF_ClassDefaultObject);
				for (UObject* Object : ObjectsToProcess)
				{
					Build_Class(AllowedClasses, DeniedClasses, Object);
				}
			}
		}
		ModuleToRefresh.Empty();

		for (FObjectKey ObjectKey : ObjectToRefresh)
		{
			// Remove the class
			FFunctionContainer FunctionEntriesToRemove;
			ClassOrBlueprintToFunctions.RemoveAndCopyValue(ObjectKey, FunctionEntriesToRemove);
			NumberOfFunctions -= FunctionEntriesToRemove.Functions.Num();

			// Add the class
			if (UObject* Object = ObjectKey.ResolveObjectPtr())
			{
				if (FunctionEntriesToRemove.bIsUserWidget)
				{
					UnegisterBlueprintCallback(Object);
				}

				Build_Class(AllowedClasses, DeniedClasses, Object);
			}
		}
		ObjectToRefresh.Empty();
	}
}

bool FCollection::IsClassSupported(const TArray<const UClass*>& AllowClasses, const TArray<const UClass*>& DenyClasses, const UClass* Class) const
{
	auto InDenyList = [&DenyClasses, Class]()
	{
		for (const UClass* DenyClass : DenyClasses)
		{
			if (Class->IsChildOf(DenyClass))
			{
				return true;
			}
		}
		return false;
	};

	FName ModuleName = Class->GetClassPathName().GetPackageName();
	bool bIsModuleDenied = GetDefault<UMVVMDeveloperProjectSettings>()->DeniedModuleForConversionFunctions.Contains(ModuleName);
	if (bIsModuleDenied)
	{
		return false;
	}

	// Ignore skeleton classes
	if (FKismetEditorUtilities::IsClassABlueprintSkeleton(Class))
	{
		return false;
	}

	if (!Class->HasAnyClassFlags(CLASS_Deprecated | CLASS_Hidden | CLASS_NewerVersionExists) && Class->GetPackage() != GetTransientPackage())
	{
		if (Class->IsChildOf(UUserWidget::StaticClass()))
		{
			return !InDenyList();
		}
		else
		{
			// Is it a child of an allowed class
			for (const UClass* AllowClass : AllowClasses)
			{
				if (Class->IsChildOf(AllowClass))
				{
					// Confirm that it's not in the deny
					return !InDenyList();
				}
			}
		}
	}
	return false;
}

void FCollection::Build()
{
	FScopedSlowTask SlowTask = FScopedSlowTask(0, LOCTEXT("BuildingConversionFunctionLibrary", "Loading Conversion function library"));

	ConversionFunctionNodes.Reset();

	TArray<const UClass*> AllSupportedClass;
	AllSupportedClass.Reserve(128);

	// Make sure the base class are loaded.
	TArray<const UClass*> AllowClasses;
	TArray<const UClass*> DenyClasses = GetDefault<UMVVMDeveloperProjectSettings>()->GetDeniedConversionFunctionClasses();

	{
		AllowClasses.Reserve(GetDefault<UMVVMDeveloperProjectSettings>()->AllowedClassForConversionFunctions.Num());
		for (const FSoftClassPath& SoftClass : GetDefault<UMVVMDeveloperProjectSettings>()->AllowedClassForConversionFunctions)
		{
			if (UClass* Class = SoftClass.TryLoadClass<UObject>())
			{
				AllowClasses.Add(Class);
			}
		}
	}

	for (TObjectIterator<UClass> It; It; ++It)
	{
		const UClass* Class = *It;
		if (IsClassSupported(AllowClasses, DenyClasses, Class))
		{
			AllSupportedClass.Add(Class);
		}
	}

	for (const UClass* Class : AllSupportedClass)
	{
		if (Class->IsChildOf(UK2Node::StaticClass()))
		{
			AddNode(TSubclassOf<UK2Node>(const_cast<UClass*>(Class)));
		}
		else
		{
			AddClassFunctions(Class);
		}
	}
}

void FCollection::Build_Class(const TArray<const UClass*>& AllowClasses, const TArray<const UClass*>& DenyClasses, const UObject* Object)
{
	const UClass* Class = Cast<const UClass>(Object);
	if (const UBlueprint* Blueprint = Cast<const UBlueprint>(Object))
	{
		Class = Blueprint->GeneratedClass ? Blueprint->GeneratedClass : Blueprint->SkeletonGeneratedClass;
	}

	if (Class)
	{
		bool bIsAllowed = IsClassSupported(AllowClasses, DenyClasses, Class);
		if (bIsAllowed)
		{
			if (Class->IsChildOf(UK2Node::StaticClass()))
			{
				AddNode(TSubclassOf<UK2Node>(const_cast<UClass*>(Class)));
			}
			else
			{
				AddClassFunctions(Class);
			}
		}
	}
}

void FCollection::AddClassFunctions(const UClass* Class)
{
	RegisterBlueprintCallback(Class);

	UBlueprint* BpOwner = Cast<UBlueprint>(Class->ClassGeneratedBy);
	auto IsInheritedBlueprintFunction = [BpOwner](const UFunction* Function)
		{
			if (BpOwner)
			{
				if (UClass* ParentClass = BpOwner->ParentClass)
				{
					FName FuncName = Function->GetFName();
					return ParentClass->FindFunctionByName(FuncName, EIncludeSuperFlag::IncludeSuper) != nullptr;
				}
			}
			return false;
		};

	const UMVVMDeveloperProjectSettings* MVVMDeveloperProjectSettings = GetDefault<UMVVMDeveloperProjectSettings>();
	check(MVVMDeveloperProjectSettings);

	const bool bIsUserWidget = Class->IsChildOf<UUserWidget>();
	FFunctionContainer* FunctionContainer = nullptr;
	for (TFieldIterator<UFunction> FunctionIt(Class, EFieldIteratorFlags::ExcludeSuper); FunctionIt; ++FunctionIt)
	{
		const UFunction* Function = *FunctionIt;
		// Any functions in a WidgetBlueprint, or the functions have to be static functions in a BlueprintFunctionLibrary.
		bool bIsFromWidgetBlueprint = bIsUserWidget && Function->HasAllFunctionFlags(FUNC_BlueprintPure | FUNC_Const);
		bool bFromBlueprintFunctionLibrary = !bIsUserWidget && Function->HasAllFunctionFlags(FUNC_Static | FUNC_BlueprintPure);
		if (!bIsFromWidgetBlueprint && !bFromBlueprintFunctionLibrary)
		{
			continue;
		}

		// As one return value.
		const FProperty* ReturnProperty = UE::MVVM::BindingHelper::GetReturnProperty(Function);
		if (ReturnProperty == nullptr)
		{
			continue;
		}

		// As at least one argument
		bool bValidProperty = false;
		TArray<const FProperty*> ArgumentsResult = UE::MVVM::BindingHelper::GetAllArgumentProperties(Function);
		if (ArgumentsResult.Num() == 0)
		{
			continue;
		}

		// Is function virtual and overridden by this class.
		if (IsInheritedBlueprintFunction(Function))
		{
			continue;
		}

		// Is it deprecated
		if (!UEdGraphSchema_K2::CanUserKismetCallFunction(Function))
		{
			continue;
		}

		// Apply general filtering for functions
		const bool bIsValidConversionFunction = bFromBlueprintFunctionLibrary && MVVMDeveloperProjectSettings->IsConversionFunctionAllowed(BpOwner, Function);
		const bool bIsValidBlueprintFunction = FBlueprintActionDatabase::IsFunctionAllowed(Function, FBlueprintActionDatabase::EPermissionsContext::Node);

		if (!bIsValidConversionFunction && !bIsValidBlueprintFunction)
		{
			continue;
		}

		// Create the function entry
		TSharedPtr<Private::FFunctionEntry_UFunction> FunctionItem = Private::FFunctionEntry_UFunction::Create(Function, ReturnProperty, MoveTemp(ArgumentsResult));
		if (FunctionItem)
		{
			if (FunctionContainer == nullptr)
			{
				FunctionContainer = &(ClassOrBlueprintToFunctions.FindOrAdd(FObjectKey(Class->ClassGeneratedBy ? Class->ClassGeneratedBy.Get() : Class)));
				FunctionContainer->bIsUserWidget = bIsUserWidget;
				check(FunctionContainer->bIsNode == false);
			}
			FunctionContainer->Functions.Add(FunctionItem.ToSharedRef());
			++NumberOfFunctions;
		}
	}
}

void FCollection::AddNode(TSubclassOf<UK2Node> Function)
{
	UK2Node* NewNode = NewObject<UK2Node>(GetTransientPackage(), Function.Get());
	NewNode->AllocateDefaultPins();
	NewNode->PostPlacedNewNode();
	ConversionFunctionNodes.Add(NewNode);

 	TArray<UEdGraphPin*> InputPins = UE::MVVM::ConversionFunctionHelper::FindInputPins(NewNode);
	if (InputPins.Num() == 0)
	{
		return;
	}

	UEdGraphPin* OutputPin = UE::MVVM::ConversionFunctionHelper::FindOutputPin(NewNode);
	if (OutputPin == nullptr)
	{
		return;
	}

	TSharedPtr<Private::FFunctionEntry_Node> FunctionItem = Private::FFunctionEntry_Node::Create(Function, OutputPin, MoveTemp(InputPins));
	if (FunctionItem)
	{
		FFunctionContainer& FunctionContainer = ClassOrBlueprintToFunctions.FindOrAdd(FObjectKey(Function.Get()));
		FunctionContainer.Functions.Add(FunctionItem.ToSharedRef());
		FunctionContainer.bIsUserWidget = false;
		FunctionContainer.bIsNode = true;
		check(FunctionContainer.Functions.Num() == 1);
	}
}

void FCollection::RegisterBlueprintCallback(const UClass* Class)
{
	if (UBlueprint* BlueprintAsset = Cast<UBlueprint>(Class->ClassGeneratedBy))
	{
		UBlueprint::FChangedEvent& OnBPChanged = BlueprintAsset->OnChanged();
		UBlueprint::FCompiledEvent& OnBPCompiled = BlueprintAsset->OnCompiled();
		if (!OnBPChanged.IsBoundToObject(this))
		{
			OnBPChanged.AddRaw(this, &FCollection::HandleBlueprintChanged);
		}
		if (!OnBPCompiled.IsBoundToObject(this))
		{
			OnBPCompiled.AddRaw(this, &FCollection::HandleBlueprintChanged);
		}
	}
}

void FCollection::UnegisterBlueprintCallback(UObject* ObjectKey)
{
	if (UBlueprint* Blueprint = Cast<UBlueprint>(ObjectKey))
	{
		Blueprint->OnChanged().RemoveAll(this);
		Blueprint->OnCompiled().RemoveAll(this);
	}
}

void FCollection::Rebuild()
{
	bRefreshAll = true;
}

TArray<::UE::MVVM::FConversionFunctionValue> FCollection::GetFunctions(const UWidgetBlueprint* WidgetBlueprint) const
{
	return GetFunctions(WidgetBlueprint, nullptr, nullptr);
}

TArray<::UE::MVVM::FConversionFunctionValue> FCollection::GetFunctions(const UWidgetBlueprint* WidgetBlueprint, const FProperty* ArgumentType, const FProperty* ReturnType) const
{
	const_cast<FCollection*>(this)->RefreshIfNeeded();
	
	const bool bHasArgumentsTest = ArgumentType != nullptr;
	const bool bHasResultTest = ReturnType != nullptr;
	
	TArray<::UE::MVVM::FConversionFunctionValue> Result;
	Result.Reserve(NumberOfFunctions);
	
	check(WidgetBlueprint);
	const UClass* WidgetBlueprintClass = WidgetBlueprint->GeneratedClass ? WidgetBlueprint->GeneratedClass : WidgetBlueprint->SkeletonGeneratedClass;
	check(WidgetBlueprintClass);
	
	for (const auto& FunctionKeyValue : ClassOrBlueprintToFunctions)
	{
		if (FunctionKeyValue.Value.bIsUserWidget)
		{
			// test if it's this userwidget before adding the functions
			const UObject* Object = FunctionKeyValue.Key.ResolveObjectPtr();
			if (ensure(Object))
			{
				// Can be a UWidgetBlueprintGeneratedClass or a UWidgetBlueprint
				if (const UWidgetBlueprint* WidgetBlueprintKey = Cast<const UWidgetBlueprint>(Object))
				{
					const UClass* WidgetBlueprintKeyClass = WidgetBlueprintKey->GeneratedClass ? WidgetBlueprintKey->GeneratedClass : WidgetBlueprintKey->SkeletonGeneratedClass;
					if (!WidgetBlueprintClass->IsChildOf(WidgetBlueprintKeyClass))
					{
						continue;
					}
				}
				else if (const UClass* ClassKey = Cast<const UClass>(Object))
				{
					if (!WidgetBlueprintClass->IsChildOf(ClassKey))
					{
						continue;
					}
				}
				else
				{
					ensure(false);
					continue;
				}
			}
		}
				
		if (!bHasArgumentsTest && !bHasResultTest)
		{
			for (const TSharedRef<FFunctionEntry>& FunctionEntry : FunctionKeyValue.Value.Functions)
			{
				Result.Add(FunctionEntry->ConversionFunctionValue);
			}
		}
		else
		{
			FEdGraphPinType ReturnPinType;
			if (ReturnType)
			{
				GetDefault<UEdGraphSchema_K2>()->ConvertPropertyToPinType(ReturnType, ReturnPinType);
			}
			FEdGraphPinType ArgumentPinType;
			if (ArgumentType)
			{
				GetDefault<UEdGraphSchema_K2>()->ConvertPropertyToPinType(ArgumentType, ArgumentPinType);
			}

			if (!FunctionKeyValue.Value.bIsNode)
			{
				for (const TSharedRef<FFunctionEntry>& FunctionEntryIt : FunctionKeyValue.Value.Functions)
				{
					const TSharedRef<Private::FFunctionEntry_UFunction>& FunctionEntry = StaticCastSharedRef<Private::FFunctionEntry_UFunction>(FunctionEntryIt);
					if (ReturnType && !UE::MVVM::BindingHelper::ArePropertiesCompatible(FunctionEntry->ReturnValue, ReturnType))
					{
						continue;
					}

					if (bHasArgumentsTest)
					{
						bool bFound = false;
						for (const FProperty* Property : FunctionEntry->Arguments)
						{
							if (UE::MVVM::BindingHelper::ArePropertiesCompatible(Property, ArgumentType))
							{
								bFound = true;
								break;
							}
						}
						if (!bFound)
						{
							continue;
						}
					}

					Result.Add(FunctionEntry->ConversionFunctionValue);
				}
			}
			else
			{
				check(FunctionKeyValue.Value.Functions.Num() == 1);
				const TSharedRef<Private::FFunctionEntry_Node>& FunctionEntry = StaticCastSharedRef<Private::FFunctionEntry_Node>(FunctionKeyValue.Value.Functions[0]);
				bool bIgnoreArray = true;

				if (ReturnType && !GetDefault<UEdGraphSchema_K2>()->ArePinTypesCompatible(ReturnPinType, FunctionEntry->ReturnValue->PinType, WidgetBlueprintClass, bIgnoreArray))
				{
					continue;
				}

				if (bHasArgumentsTest)
				{
					bool bFound = false;
					for (const UEdGraphPin* PropertyPinType : FunctionEntry->Arguments)
					{
						if (!GetDefault<UEdGraphSchema_K2>()->ArePinTypesCompatible(ArgumentPinType, PropertyPinType->PinType, WidgetBlueprintClass, bIgnoreArray))
						{
							bFound = true;
							break;
						}
					}
					if (!bFound)
					{
						continue;
					}
				}

				Result.Add(FunctionEntry->ConversionFunctionValue);
			}
		}
	}
	
	return Result;
}

const TSharedPtr<FFunctionEntry> FCollection::FindFunction(::UE::MVVM::FConversionFunctionValue FunctionValue) const
{
	const_cast<FCollection*>(this)->RefreshIfNeeded();

	if (FunctionValue.IsFunction())
	{
		const UFunction* Function = FunctionValue.GetFunction();

		// Find the function's Blueprint or the function's native class.
		const FFunctionContainer* FoundCountainer = nullptr;
		if (Function->GetClass()->ClassGeneratedBy)
		{
			FoundCountainer = ClassOrBlueprintToFunctions.Find(FObjectKey(Function->GetOuterUClass()->ClassGeneratedBy));
		}
		else
		{
			FoundCountainer = ClassOrBlueprintToFunctions.Find(FObjectKey(Function->GetOuterUClass()));
		}
		if (FoundCountainer == nullptr)
		{
			return TSharedPtr<FFunctionEntry>();
		}

		for (const TSharedRef<FFunctionEntry>& Entry : FoundCountainer->Functions)
		{
			if (Entry->GetFunction() == Function)
			{
				return Entry;
			}
		}
	}
	else if (FunctionValue.IsNode())
	{
		TSubclassOf<UK2Node> Function = FunctionValue.GetNode();

		const FFunctionContainer* FoundCountainer = ClassOrBlueprintToFunctions.Find(FObjectKey(Function.Get()));
		return FoundCountainer && FoundCountainer->Functions.Num() == 1 ? FoundCountainer->Functions[0] : TSharedPtr<FFunctionEntry>();
	}

	return TSharedPtr<FFunctionEntry>();
}

namespace Private
{
bool IsObjectValidForCollection(UObject* Object)
{
	return Object 
		&& !Object->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor | PKG_ForDiffing)
		&& (Cast<UBlueprint>(Object) || Cast<UClass>(Object)) 
		&& Object->IsAsset();
}
} // namespace

void FCollection::AddObjectToRefresh(UObject* Object)
{
	if (Private::IsObjectValidForCollection(Object))
	{
		ObjectToRefresh.Add(FObjectKey(Object));
	}
}

void FCollection::HandleBlueprintChanged(UBlueprint* Blueprint)
{
	AddObjectToRefresh(Blueprint);
}

void FCollection::HandleBlueprintUnloaded(UBlueprint* Blueprint)
{
	AddObjectToRefresh(Blueprint);
}

void FCollection::HandleObjectLoaded(UObject* Object)
{
	AddObjectToRefresh(Object);
}

void FCollection::HandleObjectPendingDelete(TArray<UObject*> const& ObjectsForDelete)
{
	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	for (UObject* DeletingObject : ObjectsForDelete)
	{
		AddObjectToRefresh(DeletingObject);
	}
}

void FCollection::HandleAssetAdded(FAssetData const& NewAssetInfo)
{
	if (NewAssetInfo.IsAssetLoaded())
	{
		UObject* AssetObject = NewAssetInfo.GetAsset();
		AddObjectToRefresh(AssetObject);
	}
}

void FCollection::HandleAssetRemoved(FAssetData const& NewAssetInfo)
{
	if (NewAssetInfo.IsAssetLoaded())
	{
		UObject* AssetObject = NewAssetInfo.GetAsset();
		AddObjectToRefresh(AssetObject);
	}
}

void FCollection::HandleAssetRenamed(FAssetData const& NewAssetInfo, const FString& OldName)
{
	if (NewAssetInfo.IsAssetLoaded())
	{
		UObject* AssetObject = NewAssetInfo.GetAsset();
		AddObjectToRefresh(AssetObject);
	}
}

void FCollection::HandleModulesChanged(FName ModuleName, EModuleChangeReason ModuleChangeReason)
{
	switch (ModuleChangeReason)
	{
	case EModuleChangeReason::ModuleLoaded:
		ModuleToRefresh.Add(ModuleName);
		break;

	case EModuleChangeReason::ModuleUnloaded:
		bRefreshAll = true;
		break;

	default:
		break;
	}
}

void FCollection::HandleReloadComplete(EReloadCompleteReason Reason)
{
	bRefreshAll = true;
}

} // namespace UE::MVVM::BindingEntry

#undef LOCTEXT_NAMESPACE
