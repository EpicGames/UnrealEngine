// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphPin.h"
#include "Types/MVVMConversionFunctionValue.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"

class UBlueprint;
class UFunction;
class UK2Node;
class UWidgetBlueprint;

namespace UE::MVVM::ConversionFunctionLibrary
{

struct FCategoryEntry;
class FCollection;

/**
 *
 */
struct FFunctionEntry
{
private:
	friend FCollection;

public:
	FFunctionEntry() = default;
	FFunctionEntry(FConversionFunctionValue Function);

	const ::UE::MVVM::FConversionFunctionValue& GetFunction() const
	{
		return ConversionFunctionValue;
	}

private:
	::UE::MVVM::FConversionFunctionValue ConversionFunctionValue;
};


/**
 * Collection of all available conversion functions.
 * The collection rebuild when a new object is loaded and when the WidgetBlueprint is compiled.
 */
class FCollection : public FGCObject
{
public:
	FCollection();
	~FCollection();

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject

public:
	[[nodiscard]] TArray<::UE::MVVM::FConversionFunctionValue> GetFunctions(const UWidgetBlueprint* Blueprint) const;
	[[nodiscard]] TArray<::UE::MVVM::FConversionFunctionValue> GetFunctions(const UWidgetBlueprint* Blueprint, const FProperty* ArgumentType, const FProperty* ReturnType) const;

	[[nodiscard]] const TSharedPtr<FFunctionEntry> FindFunction(::UE::MVVM::FConversionFunctionValue) const;

	/** Rebuild the library when a setting outside of MVVM changed. */
	void Rebuild();

private:
	void RefreshIfNeeded();
	void Build();
	void Build_Class(const TArray<const UClass*>& AllowClasses, const TArray<const UClass*>& DenyClasses, const UObject* Class);
	bool IsClassSupported(const TArray<const UClass*>& AllowClasses, const TArray<const UClass*>& DenyClasses, const UClass* Class) const;
	void AddClassFunctions(const UClass* Class);
	void AddNode(TSubclassOf<UK2Node> Function);
	void RegisterBlueprintCallback(const UClass* Class);
	void UnegisterBlueprintCallback(UObject* Class);

private:
	void AddObjectToRefresh(UObject* Object);

	void HandleBlueprintChanged(UBlueprint* Blueprint);
	void HandleBlueprintUnloaded(UBlueprint* Blueprint);
	void HandleObjectLoaded(UObject* NewObject);
	void HandleObjectPendingDelete(TArray<UObject*> const& ObjectsForDelete);
	void HandleAssetAdded(FAssetData const& AssetInfo);
	void HandleAssetRemoved(FAssetData const& AssetInfo);
	void HandleAssetRenamed(FAssetData const& AssetInfo, const FString& InOldName);
	void HandleModulesChanged(FName InModuleName, EModuleChangeReason InModuleChangeReason);
	void HandleReloadComplete(EReloadCompleteReason Reason);

private:
	struct FFunctionContainer
	{
		TArray<TSharedRef<FFunctionEntry>> Functions;
		bool bIsUserWidget = false;
		bool bIsNode = false;
	};
	TMap<FObjectKey, FFunctionContainer> ClassOrBlueprintToFunctions;
	TArray<TObjectPtr<UK2Node>> ConversionFunctionNodes;

	TSet<FObjectKey> ObjectToRefresh; // can only be GeneratedClass or Blueprint
	TSet<FName> ModuleToRefresh;
	int32 NumberOfFunctions = 0;
	bool bRefreshAll = true;
};

} // namespace UE::MVVM::ConversionFunctionLibrary
