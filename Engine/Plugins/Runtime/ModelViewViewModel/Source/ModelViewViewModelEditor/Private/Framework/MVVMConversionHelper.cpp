// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MVVMConversionHelper.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMBlueprintFunctionReference.h"
#include "WidgetBlueprint.h"

#define LOCTEXT_NAMESPACE "BindingListView_Helper"

namespace UE::MVVM
{

FString FConversionHelper::GetBindToDestinationStringFromConversionFunction(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewConversionFunction* ConversionFunction)
{
	static const FName NAME_MVVMBindToDestination(TEXT("MVVMBindToDestination"));
	FString MVVMBindToDestinationString;
	if (const UFunction* Function = ConversionFunction->GetConversionFunction().GetFunction(WidgetBlueprint))
	{
		MVVMBindToDestinationString = Function->GetMetaData(NAME_MVVMBindToDestination);
	}
	else if (ConversionFunction->GetConversionFunction().GetType() == EMVVMBlueprintFunctionReferenceType::Node)
	{
		if (TSubclassOf<UK2Node> Node = ConversionFunction->GetConversionFunction().GetNode())
		{
			MVVMBindToDestinationString = Node->GetMetaData(NAME_MVVMBindToDestination);
		}
	}
	return MVVMBindToDestinationString;
}


} // namespace UE::MVVM::BindingEntry

#undef LOCTEXT_NAMESPACE
