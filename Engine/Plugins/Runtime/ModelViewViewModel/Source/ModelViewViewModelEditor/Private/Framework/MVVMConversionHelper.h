// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"

class UMVVMBlueprintViewConversionFunction;
class UWidgetBlueprint;

namespace UE::MVVM
{

struct FConversionHelper
{
	static FString GetBindToDestinationStringFromConversionFunction(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewConversionFunction* ConversionFunction);
};

} // namespace UE::MVVM
