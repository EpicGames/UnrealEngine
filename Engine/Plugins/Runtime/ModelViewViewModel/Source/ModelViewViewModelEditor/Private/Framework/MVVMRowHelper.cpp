// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MVVMRowHelper.h"

#include "Blueprint/WidgetTree.h"
#include "BlueprintEditor.h"
#include "BlueprintModes/WidgetBlueprintApplicationModes.h"
#include "Details/WidgetPropertyDragDropOp.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Editor/EditorEngine.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Factories.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/Guid.h"
#include "Misc/StringOutputDevice.h"
#include "ScopedTransaction.h"
#include "UnrealExporter.h"
#include "UObject/UObjectGlobals.h"
#include "Exporters/Exporter.h"
#include "Widgets/ViewModelFieldDragDropOp.h"
#include "WidgetBlueprint.h"

#include "Styling/AppStyle.h"

#include "MVVMEditorSubsystem.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewBinding.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMBlueprintViewCondition.h"
#include "MVVMMessageLog.h"
#include "MVVMPropertyPath.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "Misc/Base64.h"
#include "Misc/MessageDialog.h"
#include "Types/MVVMBindingEntry.h"
#include "Types/MVVMBindingMode.h"

extern UNREALED_API UEditorEngine* GEditor;

#define LOCTEXT_NAMESPACE "BindingListView_Helper"

namespace UE::MVVM::BindingEntry
{
struct FMVVMBindingClipboardData
{
	UWidgetBlueprint* Blueprint = nullptr;

	using Type = TVariant<FMVVMBlueprintViewBinding, UMVVMBlueprintViewCondition*, UMVVMBlueprintViewEvent*>;
	TArray<Type> Items;

	/* 
	* When copy/pasting the following import objects are the target outer chain when importing condition and event objects.
	*/
	UWidgetBlueprint* ImportBlueprint = nullptr;
	UMVVMBlueprintView* ImportView = nullptr;
	UMVVMWidgetBlueprintExtension_View* ImportExtensionView = nullptr;
};


namespace Private
{
// Text import/export uses the unit seperator ascii character code as a delimiter. 
// Using a non-printable character because all objects and structs are exported as readable text.
static constexpr TStaticArray<FString::ElementType, 1> BindingClipboardDelimiter = { 0x1f };

bool ExportClipboardData(UMVVMBlueprintView* BlueprintView, const TArray<TSharedPtr<FBindingEntry>>& Entries, FMVVMBindingClipboardData& OutClipboardData)
{
	TArray<TSharedPtr<FBindingEntry>> EntriesToExport;
	EntriesToExport.Append(Entries);

	while (!EntriesToExport.IsEmpty())
	{
		TSharedPtr<FBindingEntry> Entry = EntriesToExport.Pop();

		switch (Entry->GetRowType())
		{
		case FBindingEntry::ERowType::Binding:
		{
			if (FMVVMBlueprintViewBinding* Binding = Entry->GetBinding(BlueprintView))
			{
				FMVVMBindingClipboardData::Type Item;
				Item.Set<FMVVMBlueprintViewBinding>(*Binding);
				OutClipboardData.Items.Add(Item);
			}
			break;
		}
		case FBindingEntry::ERowType::Event:
		{
			FMVVMBindingClipboardData::Type Item;
			Item.Set<UMVVMBlueprintViewEvent*>(Entry->GetEvent());
			OutClipboardData.Items.Add(Item);
			break;
		}
		case FBindingEntry::ERowType::Condition:
		{
			FMVVMBindingClipboardData::Type Item;
			Item.Set<UMVVMBlueprintViewCondition*>(Entry->GetCondition());
			OutClipboardData.Items.Add(Item);
			break;
		}
		case FBindingEntry::ERowType::Group:
		{
			EntriesToExport.Append(Entry->GetFilteredChildren());
			break;
		}
		case FBindingEntry::ERowType::BindingParameter:
		{
			UE_LOG(LogMVVM, Warning, TEXT("Failed to copy %s. Parameter Copy/Paste not supported."), *Entry->GetBindingParameterId().ToString());
			break;
		}
		case FBindingEntry::ERowType::EventParameter:
		{
			UE_LOG(LogMVVM, Warning, TEXT("Failed to copy %s. Parameter Copy/Paste not supported."), *Entry->GetEventParameterId().ToString());
			break;
		}
		case FBindingEntry::ERowType::ConditionParameter:
		{
			UE_LOG(LogMVVM, Warning, TEXT("Failed to copy %s. Parameter Copy/Paste not supported."), *Entry->GetConditionParameterId().ToString());
			break;
		}
		default:
		{
			break;
		}
		}
	}

	if (const UMVVMWidgetBlueprintExtension_View* Extension = BlueprintView->GetOuterUMVVMWidgetBlueprintExtension_View())
	{
		OutClipboardData.Blueprint = Extension->GetWidgetBlueprint();
	}
	return !OutClipboardData.Items.IsEmpty();
}

bool ImportClipboardData(UMVVMBlueprintView* BlueprintView, FMVVMBindingClipboardData& OutClipboardData)
{
	class FClipboardObjectFactory : public FCustomizableTextObjectFactory
	{
	public:
		FClipboardObjectFactory(FMVVMBindingClipboardData& ClipboardData)
			: FCustomizableTextObjectFactory(GWarn)
			, ClipboardData(ClipboardData)
		{}

		virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override final
		{
			return ObjectClass == UMVVMBlueprintViewEvent::StaticClass() 
				|| ObjectClass == UMVVMBlueprintViewCondition::StaticClass() 
				|| ObjectClass == UWidgetBlueprint::StaticClass();
		}

		virtual void ProcessConstructedObject(UObject* NewObject) override final
		{
			if (NewObject->GetClass() == UWidgetBlueprint::StaticClass())
			{
				ClipboardData.Blueprint = StaticCast<UWidgetBlueprint*>(NewObject);
			}
			else if (NewObject->GetClass() == UMVVMBlueprintViewCondition::StaticClass())
			{
				FMVVMBindingClipboardData::Type Item;
				Item.Set<UMVVMBlueprintViewCondition*>(StaticCast<UMVVMBlueprintViewCondition*>(NewObject));
				ClipboardData.Items.Add(Item);
			}
			else if (NewObject->GetClass() == UMVVMBlueprintViewEvent::StaticClass())
			{
				FMVVMBindingClipboardData::Type Item;
				Item.Set<UMVVMBlueprintViewEvent*>(StaticCast<UMVVMBlueprintViewEvent*>(NewObject));
				ClipboardData.Items.Add(Item);
			}
			else
			{				
				UE_LOG(LogMVVM, Warning, TEXT("Failed to import binding %s from clipboard"), *(NewObject->GetFName().ToString()));
			}
		}
	private:
		FMVVMBindingClipboardData& ClipboardData;
	};

	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	FString TextToParse;
	FBase64::Decode(TextToImport, TextToParse);

	TArray<FString> Items;
	TextToParse.ParseIntoArray(Items, BindingClipboardDelimiter.GetData());

	OutClipboardData.ImportBlueprint = NewObject<UWidgetBlueprint>(); 
	OutClipboardData.ImportExtensionView = NewObject<UMVVMWidgetBlueprintExtension_View>(OutClipboardData.ImportBlueprint); 
	OutClipboardData.ImportView = NewObject<UMVVMBlueprintView>(OutClipboardData.ImportExtensionView ); 
	for (const FString& ItemText : Items)
	{
		const FString StructId = TBaseStructure<FMVVMBlueprintViewBinding>::Get()->GetFName().ToString();
		if (ItemText.StartsWith(StructId))
		{
			check(ItemText.Len() > StructId.Len());
			const TCHAR* ItemTextData = &ItemText[StructId.Len()];

			FMVVMBlueprintViewBinding BindingCopied;
			FStringOutputDevice ErrorOutput;
			FMVVMBlueprintViewBinding::StaticStruct()->ImportText(ItemTextData, &BindingCopied, BlueprintView, PPF_None, &ErrorOutput, StructId);

			FMVVMBindingClipboardData::Type Item;
			Item.Set<FMVVMBlueprintViewBinding>(BindingCopied);
			OutClipboardData.Items.Add(Item);
		}
		else
		{
			FClipboardObjectFactory Factory(OutClipboardData);
			Factory.ProcessBuffer(OutClipboardData.ImportView, RF_NoFlags, ItemText);
		}
	}

	return !OutClipboardData.Items.IsEmpty();
}

bool GatherPropertyPath(UWidgetBlueprint* WidgetBlueprint, UStruct* Struct, const FMVVMBlueprintPropertyPath& PathToFind, UClass* ContextClass, FMVVMBlueprintPropertyPath& OutputPath)
{
	for (const FMVVMBlueprintFieldPath& Path : PathToFind.GetFieldPaths())
	{
		bool bFound = false;

		if (Struct != nullptr)
		{
			const FName PropertyName = Path.GetFieldName(ContextClass);
			if (const FProperty* Property = Struct->FindPropertyByName(PropertyName))
			{
				bFound = true;
				OutputPath.AppendPropertyPath(WidgetBlueprint, UE::MVVM::FMVVMConstFieldVariant(Property));

				if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property))
				{
					Struct = ObjectProperty->PropertyClass.Get();
				}
				else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
				{
					Struct = StructProperty->Struct.Get();
				}
			}
			else if (const UClass* Class = dynamic_cast<UClass*>(Struct))
			{
				if (const UFunction* Function = Class->FindFunctionByName(PropertyName))
				{
					bFound = true;
					OutputPath.AppendPropertyPath(WidgetBlueprint, UE::MVVM::FMVVMConstFieldVariant(Function));
				}
			}
		}

		if (!bFound)
		{
			return false;
		}
	}

	return true;
}

/*
	Attempt to find the desired property path under the target widget's ownership. If not found, defaults back to the original target path.
*/
FMVVMBlueprintPropertyPath TryGetCommonPropertyPath(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* View, const FMVVMBlueprintPropertyPath& TargetPath, const FMVVMBlueprintPropertyPath& DesiredPath)
{
	FMVVMBlueprintPropertyPath NewPath = TargetPath;
	NewPath.ResetPropertyPath();

	UClass* ContextClass = WidgetBlueprint->SkeletonGeneratedClass ? WidgetBlueprint->SkeletonGeneratedClass : WidgetBlueprint->GeneratedClass;

	switch (TargetPath.GetSource(WidgetBlueprint))
	{
	case EMVVMBlueprintFieldPathSource::SelfContext:
	{
		// Both paths are under that same Blueprint, so all relative paths are available
		if (TargetPath.GetSource(WidgetBlueprint) == DesiredPath.GetSource(WidgetBlueprint))
		{
			for (const FMVVMBlueprintFieldPath& Path : DesiredPath.GetFieldPaths())
			{
				NewPath.AppendPropertyPath(WidgetBlueprint, Path.GetField(ContextClass));
			}
			break;
		}
		else
		{
			UClass* TargetClass = WidgetBlueprint->GeneratedClass.Get();
			if (!GatherPropertyPath(WidgetBlueprint, TargetClass, DesiredPath, ContextClass, NewPath))
			{
				NewPath = TargetPath;
			}
		}
		break;
	}
	case EMVVMBlueprintFieldPathSource::Widget:
	{
		if (UWidget* Widget = WidgetBlueprint->WidgetTree->FindWidget(TargetPath.GetWidgetName()))
		{
			UClass* TargetClass = Widget->GetClass();
			if (!GatherPropertyPath(WidgetBlueprint, TargetClass, DesiredPath, ContextClass, NewPath))
			{
				NewPath = TargetPath;
			}
		}
		break;
	}
	case EMVVMBlueprintFieldPathSource::ViewModel:
	{
		UE_LOG(LogMVVM, Error, TEXT("MVVM: View Bindings does not support ViewModel properties as destination bindings"));
		break;
	}
	default:
	{
		// If target path is empty, do nothing. Will support all source property paths by default. 
		break;
	}
	}

	return NewPath;
}

FMVVMBlueprintPropertyPath MigratePropertyPathSource(const UMVVMBlueprintView* TargetBlueprintView, const UMVVMBlueprintView* SourceBlueprintView, const FMVVMBlueprintPropertyPath& SourcePropertyPath)
{
	if (!ensure(SourceBlueprintView) || !ensure(TargetBlueprintView))
	{
		return SourcePropertyPath;
	}

	// If the views are the same. Note: Object instances are not the same when deserialized via clipboard. Fallback to fully qualified object name to determine equivalence
	if (SourceBlueprintView->GetFullName() == TargetBlueprintView->GetFullName())
	{
		return SourcePropertyPath;
	}


	UWidgetBlueprint* SourceWidgetBlueprint = SourceBlueprintView->GetOuterUMVVMWidgetBlueprintExtension_View() ? SourceBlueprintView->GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint() : nullptr;
	if (!ensureMsgf(SourceWidgetBlueprint != nullptr, TEXT("MVVM: Failed to migrate source property path")))
	{
		return SourcePropertyPath;
	}
				
	FMVVMBlueprintPropertyPath TargetPath = SourcePropertyPath;

	switch(SourcePropertyPath.GetSource(SourceWidgetBlueprint))
	{
		case EMVVMBlueprintFieldPathSource::ViewModel:
		{
			// Try to find the source viewmodel in the target view. If found, reset source to this viewmodel.
			TargetPath.ResetSource();

			const FMVVMBlueprintViewModelContext* SourceViewModel = SourceBlueprintView->FindViewModel(SourcePropertyPath.GetViewModelId());
			if (SourceViewModel == nullptr)
			{
				UE_LOG(LogMVVM, Warning, TEXT("MVVM: Failed to find source viewmodel for source property path %s"), *(SourcePropertyPath.ToString(SourceWidgetBlueprint, true, false)));
				return TargetPath;
			}

			if (const FMVVMBlueprintViewModelContext* TargetViewModel = TargetBlueprintView->FindViewModel(SourceViewModel->GetViewModelName()))
			{
				TargetPath.SetViewModelId(TargetViewModel->GetViewModelId());
			}
			break;
		}
		case EMVVMBlueprintFieldPathSource::SelfContext:
		case EMVVMBlueprintFieldPathSource::Widget:
		case EMVVMBlueprintFieldPathSource::None:
		{
			// Do nothing. SelfContext source path is agnostic to widgetblueprint.
			// Relative widget paths will work if hierarchy pasted is the same across blueprints. Otherwise will fail as expected.
			break;
		}
	}
	return TargetPath;
}

const UMVVMBlueprintView* FindMVVMBlueprintView(const UBlueprint* Blueprint)
{
	for (const UBlueprintExtension* Extension : Blueprint->GetExtensions())
	{
		if (Extension && Extension->GetClass() == UMVVMWidgetBlueprintExtension_View::StaticClass())
		{
			const UMVVMWidgetBlueprintExtension_View* ExtensionView = StaticCast<const UMVVMWidgetBlueprintExtension_View*>(Extension);
			return ExtensionView->GetBlueprintView();
		}
	}
	return nullptr;
}
}

void FRowHelper::GatherAllChildBindings(UMVVMBlueprintView* BlueprintView, const TConstArrayView<TSharedPtr<FBindingEntry>> Entries, TArray<FGuid>& OutBindingIds, TArray<UMVVMBlueprintViewEvent*>& OutEvents, TArray<UMVVMBlueprintViewCondition*>& OutConditions, bool bRecurse)
{
	for (const TSharedPtr<FBindingEntry>& Entry : Entries)
	{
		if (Entry->GetRowType() == FBindingEntry::ERowType::Binding)
		{
			const FMVVMBlueprintViewBinding* Binding = Entry->GetBinding(BlueprintView);
			if (Binding != nullptr)
			{
				OutBindingIds.AddUnique(Binding->BindingId);
			}
		}

		if (Entry->GetRowType() == FBindingEntry::ERowType::Event)
		{
			UMVVMBlueprintViewEvent* Event = Entry->GetEvent();
			if (Event)
			{
				OutEvents.AddUnique(Event);
			}
		}

		if (Entry->GetRowType() == FBindingEntry::ERowType::Condition)
		{
			UMVVMBlueprintViewCondition* Condition = Entry->GetCondition();
			if (Condition)
			{
				OutConditions.AddUnique(Condition);
			}
		}

		if (bRecurse)
		{
			GatherAllChildBindings(BlueprintView, Entry->GetAllChildren(), OutBindingIds, OutEvents, OutConditions, bRecurse);
		}
	}
}

void FRowHelper::DeleteEntries(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* BlueprintView, TArrayView<const TSharedPtr<FBindingEntry>> Selection)
{
	if (WidgetBlueprint && BlueprintView)
	{
		TArray<FGuid> BindingIdsToRemove;
		TArray<UMVVMBlueprintViewEvent*> EventsToRemove;
		TArray<UMVVMBlueprintViewCondition*> ConditionsToRemove;
		GatherAllChildBindings(BlueprintView, Selection, BindingIdsToRemove, EventsToRemove, ConditionsToRemove);

		if (BindingIdsToRemove.Num() == 0 && EventsToRemove.Num() == 0 && ConditionsToRemove.Num() == 0)
		{
			return;
		}

		TArray<FText> BindingDisplayNames;
		for (const FGuid& BindingId : BindingIdsToRemove)
		{
			if (FMVVMBlueprintViewBinding* Binding = BlueprintView->GetBinding(BindingId))
			{
				BindingDisplayNames.Add(FText::FromString(Binding->GetDisplayNameString(WidgetBlueprint)));
			}
		}
		for (const UMVVMBlueprintViewEvent* Event : EventsToRemove)
		{
			BindingDisplayNames.Add(Event->GetDisplayName(true));
		}
		for (const UMVVMBlueprintViewCondition* Condition : ConditionsToRemove)
		{
			BindingDisplayNames.Add(Condition->GetDisplayName(true));
		}

		const FText Message = FText::Format(BindingDisplayNames.Num() == 1 ?
			LOCTEXT("ConfirmDeleteSingle", "Are you sure that you want to delete this binding/event?\n\n{1}") :
			LOCTEXT("ConfirmDeleteMultiple", "Are you sure that you want to delete these {0} bindings/events?\n\n{1}"),
			BindingDisplayNames.Num(),
			FText::Join(FText::FromString("\n"), BindingDisplayNames));

		const FText Title = LOCTEXT("DeleteBindings", "Delete Bindings?");
		if (FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::Yes, Message, Title) == EAppReturnType::Yes)
		{
			FScopedTransaction Transaction(LOCTEXT("DeleteBindingsTransaction", "Delete Bindings"));
			BlueprintView->Modify();

			for (const FGuid& BindingId : BindingIdsToRemove)
			{
				if (FMVVMBlueprintViewBinding* Binding = BlueprintView->GetBinding(BindingId))
				{
					BlueprintView->RemoveBinding(Binding);
				}
			}
			for (UMVVMBlueprintViewEvent* Event : EventsToRemove)
			{
				BlueprintView->RemoveEvent(Event);
			}
			for (UMVVMBlueprintViewCondition* Condition : ConditionsToRemove)
			{
				BlueprintView->RemoveCondition(Condition);
			}
		}
	}
}

void FRowHelper::DuplicateEntries(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* BlueprintView, TArrayView<const TSharedPtr<FBindingEntry>> Selection, TArray<const TSharedPtr<FBindingEntry>>& OutSelection)
{
	if (WidgetBlueprint && BlueprintView)
	{
		TArray<FGuid> BindingIdsToDuplicate;
		TArray<UMVVMBlueprintViewEvent*> EventsToDuplicate;
		TArray<UMVVMBlueprintViewCondition*> ConditionsToDuplicate;
		GatherAllChildBindings(BlueprintView, Selection, BindingIdsToDuplicate, EventsToDuplicate, ConditionsToDuplicate, false);

		if (BindingIdsToDuplicate.Num() == 0 && EventsToDuplicate.Num() == 0 && ConditionsToDuplicate.Num() == 0)
		{
			return;
		}

		OutSelection.Reserve(Selection.Num());

		FScopedTransaction Transaction(LOCTEXT("DuplicateBindingsTransaction", "Duplicate Bindings"));
		BlueprintView->Modify();

		for (const FGuid& BindingId : BindingIdsToDuplicate)
		{
			if (FMVVMBlueprintViewBinding* Binding = BlueprintView->GetBinding(BindingId))
			{
				const FMVVMBlueprintViewBinding* NewBinding = BlueprintView->DuplicateBinding(Binding);

				TSharedPtr<FBindingEntry> NewEntry = MakeShared<FBindingEntry>();
				NewEntry->SetBindingId(NewBinding->BindingId);

				OutSelection.Add(NewEntry);
			}
		}

		for (UMVVMBlueprintViewEvent* Event : EventsToDuplicate)
		{
			UMVVMBlueprintViewEvent* NewEvent = BlueprintView->DuplicateEvent(Event);

			TSharedPtr<FBindingEntry> NewEntry = MakeShared<FBindingEntry>();
			NewEntry->SetEvent(NewEvent);

			OutSelection.Add(NewEntry);
		}

		for (UMVVMBlueprintViewCondition* Condition : ConditionsToDuplicate)
		{
			UMVVMBlueprintViewCondition* NewCondition = BlueprintView->DuplicateCondition(Condition);

			TSharedPtr<FBindingEntry> NewEntry = MakeShared<FBindingEntry>();
			NewEntry->SetCondition(NewCondition);

			OutSelection.Add(NewEntry);
		}
	}
}

void FRowHelper::CopyEntries(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* BlueprintView, const TArray<TSharedPtr<FBindingEntry>>& Entries)
{
	if (WidgetBlueprint == nullptr || BlueprintView == nullptr || Entries.IsEmpty())
	{
		return;
	}
		
	FMVVMBindingClipboardData ClipboardData;
	if (!Private::ExportClipboardData(BlueprintView, Entries, ClipboardData))
	{
		return;
	}
	
	FString CopyText;

	if(ClipboardData.Blueprint != nullptr)
	{
		FStringOutputDevice Output;
		const FExportObjectInnerContext Context;

		UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));
		UExporter::ExportToOutputDevice(&Context, ClipboardData.Blueprint, nullptr, Output, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, ClipboardData.Blueprint->GetOuter());

		if (Output.IsEmpty())
		{
			UE_LOG(LogMVVM, Warning, TEXT("Failed to export mvvm view for copy"));
		}
		else
		{
			CopyText.Append(*Output);
		}
	}

	for (FMVVMBindingClipboardData::Type Item : ClipboardData.Items)
	{
		bool bFailed = false;
		UScriptStruct* Struct = nullptr;
		const void* StructData = nullptr;
		UObject* Object = nullptr;

		if (Item.IsType<FMVVMBlueprintViewBinding>())
		{
			FMVVMBlueprintViewBinding* Binding = &Item.Get<FMVVMBlueprintViewBinding>();
			Binding->Conversion.SavePinValues(WidgetBlueprint);

			StructData = Binding;
			Struct = FMVVMBlueprintViewBinding::StaticStruct();
		}
		else if (Item.IsType<UMVVMBlueprintViewCondition*>())
		{
			UMVVMBlueprintViewCondition* Condition = Item.Get<UMVVMBlueprintViewCondition*>();
			Condition->SavePinValues();

			Object = Condition;
		}
		else if (Item.IsType<UMVVMBlueprintViewEvent*>())
		{
			UMVVMBlueprintViewEvent* Event = Item.Get<UMVVMBlueprintViewEvent*>();
			Event->SavePinValues();

			Object = Event;
		}

		if (Struct != nullptr && StructData != nullptr)
		{
			FString Text;
			Struct->ExportText(Text, StructData, nullptr, BlueprintView, PPF_None, nullptr);

			if (Text.IsEmpty())
			{
				bFailed = true;
				continue;
			}

			if (!CopyText.IsEmpty())
			{
				CopyText.Append(Private::BindingClipboardDelimiter);
			}

			// Prepend struct type to support explicit struct deserialization. 
			CopyText.Append(Struct->GetFName().ToString());
			CopyText.Append(Text);
		}
		else if (Object != nullptr)
		{
			FStringOutputDevice Output;
			const FExportObjectInnerContext Context;

			UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));
			UExporter::ExportToOutputDevice(&Context, Object, nullptr, Output, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, Object->GetOuter());

			if (Output.IsEmpty())
			{
				bFailed = true;
				continue;
			}

			if (!CopyText.IsEmpty())
			{
				CopyText.Append(Private::BindingClipboardDelimiter);
			}

			CopyText.Append(*Output);
		}
		
		if (bFailed)
		{
			if (Item.IsType<FMVVMBlueprintViewBinding>())
			{
				const FMVVMBlueprintViewBinding& Binding = Item.Get<FMVVMBlueprintViewBinding>();
				UE_LOG(LogMVVM, Warning, TEXT("Failed to export binding %s for copy"), *(Binding.GetDisplayNameString(WidgetBlueprint, true)));
			}
			else if (Item.IsType<UMVVMBlueprintViewCondition*>())
			{
				UMVVMBlueprintViewCondition* Condition = Item.Get<UMVVMBlueprintViewCondition*>();
				UE_LOG(LogMVVM, Warning, TEXT("Failed to export condition %s for copy"), *(Condition->GetDisplayName(true).ToString()));
			}
			else if (Item.IsType<UMVVMBlueprintViewEvent*>())
			{
				UMVVMBlueprintViewEvent* Event = Item.Get<UMVVMBlueprintViewEvent*>();
				UE_LOG(LogMVVM, Warning, TEXT("Failed to export event %s for copy"), *(Event->GetDisplayName(true).ToString()));
			}
		}
	}

	if (!CopyText.IsEmpty())
	{
		FString Encoded = FBase64::Encode(CopyText);
		FPlatformApplicationMisc::ClipboardCopy(*Encoded);
	}
}

void FRowHelper::PasteEntries(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* BlueprintView, const TArray<TSharedPtr<FBindingEntry>>& Entries)
{
	// When pasting new bindings, this will attempt to preserve the destination path. 
	// If the relative destination path that was copied exists under the selected source, the use this property path. Otherwise, fallback to previous path. 
	// Treated as a special condition with copy/paste behavior to preserve the existing binding group hierarchy

	if (WidgetBlueprint == nullptr || BlueprintView == nullptr)
	{
		return;
	}

	FMVVMBindingClipboardData ClipboardData;
	Private::ImportClipboardData(BlueprintView, ClipboardData);

	if (Entries.IsEmpty())
	{
		// Duplicate the Bindings when pasting into nothing
		FScopedTransaction Transaction(LOCTEXT("PasteBindingsTransaction", "Paste Bindings"));
		BlueprintView->Modify();

		for (const FMVVMBindingClipboardData::Type& Item : ClipboardData.Items)
		{
			if (Item.IsType<FMVVMBlueprintViewBinding>())
			{
				const FMVVMBlueprintViewBinding& Binding = Item.Get<FMVVMBlueprintViewBinding>();
				const FMVVMBlueprintViewBinding* NewBinding = BlueprintView->DuplicateBinding(&Binding);

				TSharedPtr<FBindingEntry> NewEntry = MakeShared<FBindingEntry>();
				NewEntry->SetBindingId(NewBinding->BindingId);
			}
			else if (Item.IsType<UMVVMBlueprintViewCondition*>())
			{
				UMVVMBlueprintViewCondition* Condition = Item.Get<UMVVMBlueprintViewCondition*>();
				UMVVMBlueprintViewCondition* NewCondition = BlueprintView->DuplicateCondition(Condition);

				TSharedPtr<FBindingEntry> NewEntry = MakeShared<FBindingEntry>();
				NewEntry->SetCondition(NewCondition);
			}
			else if (Item.IsType<UMVVMBlueprintViewEvent*>())
			{
				UMVVMBlueprintViewEvent* Event = Item.Get<UMVVMBlueprintViewEvent*>();
				UMVVMBlueprintViewEvent* NewEvent = BlueprintView->DuplicateEvent(Event);

				TSharedPtr<FBindingEntry> NewEntry = MakeShared<FBindingEntry>();
				NewEntry->SetEvent(NewEvent);
			}
		}
		return;
	}

	if (Entries.Num() != 1 || ClipboardData.Items.Num() != 1)
	{
		UE_LOG(LogMVVM, Warning, TEXT("Copy/Pasting into multiple entries is not supported."));
		return;
	}

	const TSharedPtr<FBindingEntry> Entry = Entries.Last();
	const FMVVMBindingClipboardData::Type& ClipboardItem = ClipboardData.Items.Last();
	UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();

	const UMVVMBlueprintView* ClipboardBlueprintView = Private::FindMVVMBlueprintView(ClipboardData.Blueprint);

	if (ClipboardItem.IsType<FMVVMBlueprintViewBinding>())
	{
		if (Entry->GetRowType() != FBindingEntry::ERowType::Binding)
		{
			UE_LOG(LogMVVM, Error, TEXT("Failed to paste into Property Binding, mismatched types."));
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("PasteBindingTransaction", "Paste Binding"));
		BlueprintView->Modify();

		const FMVVMBlueprintViewBinding& BindingToPaste = ClipboardItem.Get<FMVVMBlueprintViewBinding>();

		FMVVMBlueprintViewBinding* BindingSelected = Entry->GetBinding(BlueprintView);
		BindingSelected->BindingType = BindingToPaste.BindingType;
		BindingSelected->SourcePath = Private::MigratePropertyPathSource(BlueprintView, ClipboardBlueprintView, BindingToPaste.SourcePath);

		// Copy over conversion functions
		if (UMVVMBlueprintViewConversionFunction* ConversionFunction = BindingToPaste.Conversion.SourceToDestinationConversion)
		{
			BindingSelected->Conversion.SourceToDestinationConversion = DuplicateObject<UMVVMBlueprintViewConversionFunction>(ConversionFunction, WidgetBlueprint);
			BindingSelected->Conversion.SourceToDestinationConversion->RecreateWrapperGraph(WidgetBlueprint);
		}

		if (UMVVMBlueprintViewConversionFunction* ConversionFunction = BindingToPaste.Conversion.DestinationToSourceConversion)
		{
			BindingSelected->Conversion.DestinationToSourceConversion = DuplicateObject<UMVVMBlueprintViewConversionFunction>(ConversionFunction, WidgetBlueprint);
			BindingSelected->Conversion.DestinationToSourceConversion->RecreateWrapperGraph(WidgetBlueprint);
		}

		const FMVVMBlueprintPropertyPath& DesiredPath = BindingToPaste.DestinationPath; 
		const FMVVMBlueprintPropertyPath& TargetPath = BindingSelected->DestinationPath;
		const FMVVMBlueprintPropertyPath NewPath = Private::TryGetCommonPropertyPath(WidgetBlueprint, BlueprintView, TargetPath, DesiredPath);

		EditorSubsystem->SetDestinationPathForBinding(WidgetBlueprint, *BindingSelected, NewPath, false);

		BlueprintView->OnBindingsUpdated.Broadcast();
	}
	else if (ClipboardItem.IsType<UMVVMBlueprintViewCondition*>())
	{
		if (Entry->GetRowType() != FBindingEntry::ERowType::Condition)
		{
			UE_LOG(LogMVVM, Error, TEXT("Failed to paste into Condition, mismatched types."));
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("PasteConditionTransaction", "Paste Condition"));
		BlueprintView->Modify();

		// Retain the condition path when pasting. Copy over all other properties
		UMVVMBlueprintViewCondition* ConditionSelected = Entry->GetCondition();
		UMVVMBlueprintViewCondition* ConditionToPaste = ClipboardItem.Get<UMVVMBlueprintViewCondition*>();
		UMVVMBlueprintViewCondition* NewCondition = DuplicateObject<UMVVMBlueprintViewCondition>(ConditionToPaste, BlueprintView);

		FMVVMBlueprintPropertyPath ConditionPathMigrated = Private::MigratePropertyPathSource(BlueprintView, ClipboardBlueprintView, ConditionSelected->GetConditionPath());

		// Set the old operation path values to the newly duplicated condition. 
		NewCondition->SetConditionPath(ConditionPathMigrated);
		NewCondition->SetOperation(ConditionSelected->GetOperation());
		NewCondition->SetOperationValue(ConditionSelected->GetOperationValue());
		NewCondition->SetOperationMaxValue(ConditionSelected->GetOperationMaxValue());
		
		BlueprintView->ReplaceCondition(ConditionSelected, NewCondition);
	}
	else if (ClipboardItem.IsType<UMVVMBlueprintViewEvent*>())
	{
		if (Entry->GetRowType() != FBindingEntry::ERowType::Event)
		{
			UE_LOG(LogMVVM, Error, TEXT("Failed to paste into Event, mismatched types."));
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("PasteEventTransaction", "Paste Event"));
		BlueprintView->Modify();

		// Retain the event path when pasting. Copy over all other properties
		UMVVMBlueprintViewEvent* EventSelected = Entry->GetEvent();
		UMVVMBlueprintViewEvent* EventToPaste = ClipboardItem.Get<UMVVMBlueprintViewEvent*>();
		UMVVMBlueprintViewEvent* NewEvent = DuplicateObject<UMVVMBlueprintViewEvent>(EventToPaste, BlueprintView);

		const FMVVMBlueprintPropertyPath EventPathMigrated = Private::MigratePropertyPathSource(BlueprintView, ClipboardBlueprintView, EventSelected->GetEventPath());
		
		// Set the old event path values to the newly duplicated event. Rebuild to update conversion graph 
		NewEvent->SetEventPath(EventPathMigrated);
		NewEvent->RecreateWrapperGraph();
		NewEvent->UpdatePinValues();

		BlueprintView->ReplaceEvent(EventSelected, NewEvent);
	}
}

bool FRowHelper::HasBlueprintGraph(UMVVMBlueprintView* BlueprintView, const TSharedPtr<FBindingEntry> Entry)
{
	if (!Entry.IsValid())
	{
		return false;
	}

	if (Entry->GetRowType() == FBindingEntry::ERowType::Binding)
	{
		if (FMVVMBlueprintViewBinding* ViewBinding = Entry->GetBinding(BlueprintView))
		{
			UMVVMBlueprintViewConversionFunction* ConversionFunctionA = ViewBinding->Conversion.GetConversionFunction(true);
			UMVVMBlueprintViewConversionFunction* ConversionFunctionB = ViewBinding->Conversion.GetConversionFunction(false);
			return ConversionFunctionA != nullptr || ConversionFunctionB != nullptr;
		}
	}
	else if (Entry->GetRowType() == FBindingEntry::ERowType::Event)
	{
		if (UMVVMBlueprintViewEvent* Event = Entry->GetEvent())
		{
			return Event->GetWrapperGraph() != nullptr;
		}
	}
	else if (Entry->GetRowType() == FBindingEntry::ERowType::Condition)
	{
		if (UMVVMBlueprintViewCondition* Condition = Entry->GetCondition())
		{
			return Condition->GetWrapperGraph() != nullptr;
		}
	}
	return false;
}

void FRowHelper::ShowBlueprintGraph(FBlueprintEditor* BlueprintEditor, UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* BlueprintView, TArrayView<const TSharedPtr<FBindingEntry>> Entries)
{
	auto ShowGraph = [WidgetBlueprint, BlueprintEditor](UEdGraph* Graph)
		{
			if (Graph && BlueprintEditor)
			{
				if (Graph->HasAnyFlags(RF_Transient))
				{
					BlueprintEditor->SetCurrentMode(FWidgetBlueprintApplicationModes::GraphMode);
					BlueprintEditor->OpenDocument(Graph, FDocumentTracker::OpenNewDocument);
				}
				else
				{
					FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Graph);
				}
			}
		};

	const UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	for (const TSharedPtr<FBindingEntry>& Entry : Entries)
	{
		if (Entry->GetRowType() == FBindingEntry::ERowType::Binding)
		{
			if (FMVVMBlueprintViewBinding* ViewBinding = Entry->GetBinding(BlueprintView))
			{
				UMVVMBlueprintViewConversionFunction* ConversionFunctionA = ViewBinding->Conversion.GetConversionFunction(true);
				UMVVMBlueprintViewConversionFunction* ConversionFunctionB = ViewBinding->Conversion.GetConversionFunction(false);
				if (ConversionFunctionA)
				{
					ShowGraph(ConversionFunctionA->GetWrapperGraph());
				}
				if (ConversionFunctionB)
				{
					ShowGraph(ConversionFunctionB->GetWrapperGraph());
				}
			}
		}
		else if (Entry->GetRowType() == FBindingEntry::ERowType::Event)
		{
			if (UMVVMBlueprintViewEvent* Event = Entry->GetEvent())
			{
				ShowGraph(Event->GetWrapperGraph());
			}
		}
		else if (Entry->GetRowType() == FBindingEntry::ERowType::Condition)
		{
			if (UMVVMBlueprintViewCondition* Condition = Entry->GetCondition())
			{
				ShowGraph(Condition->GetWrapperGraph());
			}
		}
	}
}

TOptional<FMVVMBlueprintPropertyPath> FRowHelper::DropFieldSelector(const UWidgetBlueprint* WidgetBlueprint, const FDragDropEvent& DragDropEvent, bool bIsSource)
{
	if (WidgetBlueprint == nullptr)
	{
		return TOptional<FMVVMBlueprintPropertyPath>();
	}

	TSharedPtr<FDecoratedDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return TOptional<FMVVMBlueprintPropertyPath>();
	}

	if (DragDropOp->IsOfType<FViewModelFieldDragDropOp>())
	{
		// Accept valid view model fields when we are dropping into the Source box.
		TSharedPtr<FViewModelFieldDragDropOp> ViewModelFieldDragDropOp = DragDropEvent.GetOperationAs<FViewModelFieldDragDropOp>();
		if (!bIsSource || !ViewModelFieldDragDropOp->ViewModelId.IsValid())
		{
			return TOptional<FMVVMBlueprintPropertyPath>();
		}

		UWidgetBlueprint* DragDropWidgetBP = ViewModelFieldDragDropOp->WidgetBP.Get();
		if (WidgetBlueprint != DragDropWidgetBP)
		{
			return TOptional<FMVVMBlueprintPropertyPath>();
		}

		TArray<FFieldVariant> FieldPath = ViewModelFieldDragDropOp->DraggedField;
		FMVVMBlueprintPropertyPath PropertyPath;
		for (const FFieldVariant& Field : FieldPath)
		{
			PropertyPath.AppendPropertyPath(WidgetBlueprint, FMVVMConstFieldVariant(Field));
		}

		UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		PropertyPath.SetViewModelId(ViewModelFieldDragDropOp->ViewModelId);
		return PropertyPath;
	}
	else if (DragDropOp->IsOfType<FWidgetPropertyDragDropOp>())
	{
		TSharedPtr<FWidgetPropertyDragDropOp> WidgetPropertyDragDropOp = DragDropEvent.GetOperationAs<FWidgetPropertyDragDropOp>();
		
		UWidgetBlueprint* DragDropWidgetBP = WidgetPropertyDragDropOp->WidgetBP.Get();
		UWidget* OwnerWidgetPtr = WidgetPropertyDragDropOp->OwnerWidget.Get();
		if (WidgetBlueprint != DragDropWidgetBP || OwnerWidgetPtr == nullptr)
		{
			return TOptional<FMVVMBlueprintPropertyPath>();
		}

		TArray<FFieldVariant> FieldPath = WidgetPropertyDragDropOp->DraggedPropertyPath;
		FMVVMBlueprintPropertyPath PropertyPath;
		for (const FFieldVariant& Field : FieldPath)
		{
			PropertyPath.AppendPropertyPath(WidgetBlueprint, FMVVMConstFieldVariant(Field));
		}

		if (OwnerWidgetPtr->GetClass() == WidgetBlueprint->GeneratedClass)
		{
			PropertyPath.SetSelfContext();
		}
		else
		{
			PropertyPath.SetWidgetName(OwnerWidgetPtr->GetFName());
		}

		return PropertyPath;
	}

	return TOptional<FMVVMBlueprintPropertyPath>();
}

FReply FRowHelper::DragOverFieldSelector(const UWidgetBlueprint* WidgetBlueprint, const FDragDropEvent& DragDropEvent, bool bIsSource)
{
	TSharedPtr<FDecoratedDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return FReply::Unhandled();
	}

	// Accept view model fields when we are dropping into the Source box.
	if (DragDropOp->IsOfType<FViewModelFieldDragDropOp>() && bIsSource)
	{
		TSharedPtr<FViewModelFieldDragDropOp> ViewModelFieldDragDropOp = DragDropEvent.GetOperationAs<FViewModelFieldDragDropOp>();

		UWidgetBlueprint* DragDropWidgetBP = ViewModelFieldDragDropOp->WidgetBP.Get();
		if (DragDropWidgetBP == WidgetBlueprint)
		{
			DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
		}
	}
	else if (DragDropOp->IsOfType<FWidgetPropertyDragDropOp>())
	{
		TSharedPtr<FWidgetPropertyDragDropOp> WidgetPropertyDragDropOp = DragDropEvent.GetOperationAs<FWidgetPropertyDragDropOp>();

		UWidgetBlueprint* DragDropWidgetBP = WidgetPropertyDragDropOp->WidgetBP.Get();
		UWidget* OwnerWidgetPtr = WidgetPropertyDragDropOp->OwnerWidget.Get();
		if (WidgetBlueprint == DragDropWidgetBP && OwnerWidgetPtr != nullptr)
		{
			DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
		}
	}
	else
	{
		DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
	}

	return FReply::Handled();
}

namespace Private
{

TArray<TSharedPtr<FBindingEntry>> WeakToSharedPtr(const TSharedPtr<TArray<TWeakPtr<FBindingEntry>>>& CopiedEntries)
{
	TArray<TSharedPtr<FBindingEntry>> Result;
	Result.Reserve(CopiedEntries->Num());
	for (TWeakPtr<FBindingEntry> Entry : *CopiedEntries)
	{
		if (TSharedPtr<FBindingEntry> Pin = Entry.Pin())
		{
			Result.Add(Pin);
		}
	}
	return Result;
}

void HandleDeleteEntries(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* View, TSharedPtr<TArray<TWeakPtr<FBindingEntry>>> Entries)
{
	FRowHelper::DeleteEntries(WidgetBlueprint, View, WeakToSharedPtr(Entries));
}

void HandleDuplicateEntries(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* View, TSharedPtr<TArray<TWeakPtr<FBindingEntry>>> Entries, FOnContextMenuEntryCallback OnSelectionChanged)
{
	TArray<const TSharedPtr<FBindingEntry>> NewSelection;
	FRowHelper::DuplicateEntries(WidgetBlueprint, View, WeakToSharedPtr(Entries), NewSelection);

	OnSelectionChanged.ExecuteIfBound(NewSelection);
}

void HandleCopyEntries(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* View, TSharedPtr<TArray<TWeakPtr<FBindingEntry>>> Entries)
{
	FRowHelper::CopyEntries(WidgetBlueprint, View, WeakToSharedPtr(Entries));
}

void HandlePasteEntries(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* View, TSharedPtr<TArray<TWeakPtr<FBindingEntry>>> Entries)
{
	FRowHelper::PasteEntries(WidgetBlueprint, View, WeakToSharedPtr(Entries));
}

void HandleResetSelectedPin(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* View, TSharedPtr<TArray<TWeakPtr<FBindingEntry>>> Entries)
{
	const UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	for (const TSharedPtr<FBindingEntry>& Entry : WeakToSharedPtr(Entries))
	{
		if (Entry->GetRowType() == FBindingEntry::ERowType::EventParameter)
		{
			EditorSubsystem->ResetPinToDefaultValue(WidgetBlueprint, Entry->GetEvent(), Entry->GetEventParameterId());
		}
		else if (Entry->GetRowType() == FBindingEntry::ERowType::BindingParameter)
		{
			if (FMVVMBlueprintViewBinding* Binding = Entry->GetBinding(View))
			{
				const bool bSourceToDestination = UE::MVVM::IsForwardBinding(Binding->BindingType);
				EditorSubsystem->ResetPinToDefaultValue(WidgetBlueprint, *Binding, Entry->GetBindingParameterId(), bSourceToDestination);
			}
		}
	}
}

void HandleBreakSelectedPin(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* View, TSharedPtr<TArray<TWeakPtr<FBindingEntry>>> Entries)
{
	const UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	for (const TSharedPtr<FBindingEntry>& Entry : WeakToSharedPtr(Entries))
	{
		if (Entry->GetRowType() == FBindingEntry::ERowType::EventParameter)
		{
			EditorSubsystem->SplitPin(WidgetBlueprint, Entry->GetEvent(), Entry->GetEventParameterId());
		}
		else if (Entry->GetRowType() == FBindingEntry::ERowType::BindingParameter)
		{
			if (FMVVMBlueprintViewBinding* Binding = Entry->GetBinding(View))
			{
				const bool bSourceToDestination = UE::MVVM::IsForwardBinding(Binding->BindingType);
				EditorSubsystem->SplitPin(WidgetBlueprint, *Binding, Entry->GetBindingParameterId(), bSourceToDestination);
			}
		}
	}
}

void HandleRecombineSelectedPin(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* View, TSharedPtr<TArray<TWeakPtr<FBindingEntry>>> Entries)
{
	const UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	for (const TSharedPtr<FBindingEntry>& Entry : WeakToSharedPtr(Entries))
	{
		if (Entry->GetRowType() == FBindingEntry::ERowType::EventParameter)
		{
			EditorSubsystem->RecombinePin(WidgetBlueprint, Entry->GetEvent(), Entry->GetEventParameterId());
		}
		else if (Entry->GetRowType() == FBindingEntry::ERowType::BindingParameter)
		{
			if (FMVVMBlueprintViewBinding* Binding = Entry->GetBinding(View))
			{
				const bool bSourceToDestination = UE::MVVM::IsForwardBinding(Binding->BindingType);
				EditorSubsystem->RecombinePin(WidgetBlueprint, *Binding, Entry->GetBindingParameterId(), bSourceToDestination);
			}
		}
	}
}

void HandleResetOrphanedSelectedPin(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* View, TSharedPtr<TArray<TWeakPtr<FBindingEntry>>> Entries)
{
	const UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	for (const TSharedPtr<FBindingEntry>& Entry : WeakToSharedPtr(Entries))
	{
		if (Entry->GetRowType() == FBindingEntry::ERowType::EventParameter)
		{
			EditorSubsystem->ResetOrphanedPin(WidgetBlueprint, Entry->GetEvent(), Entry->GetEventParameterId());
		}
		else if (Entry->GetRowType() == FBindingEntry::ERowType::BindingParameter)
		{
			if (FMVVMBlueprintViewBinding* Binding = Entry->GetBinding(View))
			{
				const bool bSourceToDestination = UE::MVVM::IsForwardBinding(Binding->BindingType);
				EditorSubsystem->ResetOrphanedPin(WidgetBlueprint, *Binding, Entry->GetBindingParameterId(), bSourceToDestination);
			}
		}
	}
}

} //namespace

FMenuBuilder FRowHelper::CreateContextMenu(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* View, TConstArrayView<TSharedPtr<FBindingEntry>> Entries, FOnContextMenuEntryCallback OnSelectionChanged)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	if (WidgetBlueprint && View && Entries.Num() > 0)
	{
		TSharedPtr<TArray<TWeakPtr<FBindingEntry>>> CopiedEntries;
		CopiedEntries = MakeShared<TArray<TWeakPtr<FBindingEntry>>>();
		CopiedEntries->Reserve(Entries.Num());
		for (const TSharedPtr<FBindingEntry>& Entry : Entries)
		{
			CopiedEntries->Add(Entry);
		}

		const UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		{
			bool bCanRemoveEntry = true;
			bool bCanCopyEntry = true;
			bool bCanPasteEntry = Entries.Num() <= 1;
			bool bCanDuplicateEntry = true;
			for (const TSharedPtr<FBindingEntry>& Entry : Entries)
			{
				switch (Entry->GetRowType())
				{
				case FBindingEntry::ERowType::Group:
					bCanDuplicateEntry = false;
					bCanPasteEntry = false;
					break;
				case FBindingEntry::ERowType::Binding:
				case FBindingEntry::ERowType::Event:
				case FBindingEntry::ERowType::Condition:
					break;
				default:
					bCanCopyEntry = false;
					bCanPasteEntry = false;
					bCanRemoveEntry = false;
					bCanDuplicateEntry = false;
					break;
				}
			}

			if (bCanPasteEntry && Entries.Num() == 1) // Check paste type
			{
				bCanPasteEntry = false;

				// Only allow pasting a single entry into entry of same type
				FMVVMBindingClipboardData ClipboardData;
				Private::ImportClipboardData(View, ClipboardData);
				if (ClipboardData.Items.Num() == 1)
				{
					const TSharedPtr<FBindingEntry> Entry = Entries.Last();
					const FMVVMBindingClipboardData::Type& Item = ClipboardData.Items.Last();

					if (Item.IsType<FMVVMBlueprintViewBinding>())
					{
						bCanPasteEntry = Entry->GetRowType() == FBindingEntry::ERowType::Binding;
					}
					else if (Item.IsType<UMVVMBlueprintViewCondition*>())
					{
						bCanPasteEntry = Entry->GetRowType() == FBindingEntry::ERowType::Condition;
					}
					else if (Item.IsType<UMVVMBlueprintViewEvent*>())
					{
						bCanPasteEntry = Entry->GetRowType() == FBindingEntry::ERowType::Event;
					}
				}
			}

			FUIAction RemoveAction;
			RemoveAction.ExecuteAction = FExecuteAction::CreateStatic(&Private::HandleDeleteEntries, WidgetBlueprint, View, CopiedEntries);
			RemoveAction.CanExecuteAction = FCanExecuteAction::CreateLambda([bCanRemoveEntry]() { return bCanRemoveEntry; });
			MenuBuilder.AddMenuEntry(LOCTEXT("RemoveBinding", "Remove"),
				LOCTEXT("RemoveBindingTooltip", "Remove bindings or events."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
				RemoveAction,
				NAME_None,
				EUserInterfaceActionType::Button,
				NAME_None,
				LOCTEXT("DELETE", "DELETE"));

			FUIAction DuplicateAction;
			DuplicateAction.ExecuteAction = FExecuteAction::CreateStatic(&Private::HandleDuplicateEntries, WidgetBlueprint, View, CopiedEntries, OnSelectionChanged);
			DuplicateAction.CanExecuteAction = FCanExecuteAction::CreateLambda([bCanDuplicateEntry]() { return bCanDuplicateEntry; });
			MenuBuilder.AddMenuEntry(LOCTEXT("DuplicateBinding", "Duplicate"),
				LOCTEXT("DuplicateBindingTooltip", "Duplicate bindings or events."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"),
				DuplicateAction,
				NAME_None,
				EUserInterfaceActionType::Button,
				NAME_None,
				LOCTEXT("CRTL+D", "CRTL+D"));

			FUIAction CopyAction;
			CopyAction.ExecuteAction = FExecuteAction::CreateStatic(&Private::HandleCopyEntries, WidgetBlueprint, View, CopiedEntries);
			CopyAction.CanExecuteAction = FCanExecuteAction::CreateLambda([bCanCopyEntry]() { return bCanCopyEntry; });
			MenuBuilder.AddMenuEntry(LOCTEXT("CopyBinding", "Copy"),
				LOCTEXT("CopyBindingTooltip", "Copy bindings or events."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Copy"),
				CopyAction,
				NAME_None,
				EUserInterfaceActionType::Button,
				NAME_None,
				LOCTEXT("CRTL+C", "CRTL+C"));

			FUIAction PasteAction;
			PasteAction.ExecuteAction = FExecuteAction::CreateStatic(&Private::HandlePasteEntries, WidgetBlueprint, View, CopiedEntries);
			PasteAction.CanExecuteAction = FCanExecuteAction::CreateLambda([bCanPasteEntry]() { return bCanPasteEntry; });
			MenuBuilder.AddMenuEntry(
				LOCTEXT("PasteBinding", "Paste"),
				LOCTEXT("PasteBindingTooltip", "Paste bindings or events."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Paste"),
				PasteAction,
				NAME_None,
				EUserInterfaceActionType::Button,
				NAME_None,
				LOCTEXT("CRTL+V", "CRTL+V"));

		}

		{
			bool bCanSplitPin = true;
			bool bCanRecombinePin = true;
			bool bCanRecombinePinVisible = true;
			bool bCanResetPin = true;
			bool bCanResetPinVisible = true;
			bool bCanResetOrphanedPin = true;
			auto SetAll = [&](bool Value)
				{
					bCanSplitPin = Value;
					bCanRecombinePin = Value;
					bCanRecombinePinVisible = Value;
					bCanResetPin = Value;
					bCanResetPinVisible = Value;
					bCanResetOrphanedPin = Value;
				};
			auto AllFalse = [&]()
				{
					return !bCanRecombinePin && !bCanSplitPin && !bCanRecombinePinVisible && bCanResetPin && !bCanResetPinVisible && !bCanResetOrphanedPin;
				};

			for (const TSharedPtr<FBindingEntry>& Entry : Entries)
			{
				if (Entry->GetRowType() == FBindingEntry::ERowType::EventParameter)
				{
					if (!EditorSubsystem->CanSplitPin(WidgetBlueprint, Entry->GetEvent(), Entry->GetEventParameterId()))
					{
						bCanSplitPin = false;
					}
					if (!EditorSubsystem->CanRecombinePin(WidgetBlueprint, Entry->GetEvent(), Entry->GetEventParameterId()))
					{
						bCanRecombinePin = false;
					}
					if (!EditorSubsystem->CanResetPinToDefaultValue(WidgetBlueprint, Entry->GetEvent(), Entry->GetEventParameterId()))
					{
						bCanResetPin = false;
					}
					if (!EditorSubsystem->CanResetOrphanedPin(WidgetBlueprint, Entry->GetEvent(), Entry->GetEventParameterId()))
					{
						bCanResetOrphanedPin = false;
					}

					UMVVMBlueprintViewEvent* ViewEvent = Entry->GetEvent();
					UEdGraphPin* GraphPin = ViewEvent ? ViewEvent->GetOrCreateGraphPin(Entry->GetEventParameterId()) : nullptr;
					if (GraphPin == nullptr)
					{
						bCanRecombinePinVisible = false;
						bCanResetPinVisible = false;
					}
					else
					{
						if (GraphPin->ParentPin == nullptr)
						{
							bCanRecombinePinVisible = false;
						}
						if (!FMVVMBlueprintPin::IsInputPin(GraphPin) || GetDefault<UEdGraphSchema_K2>()->ShouldHidePinDefaultValue(GraphPin))
						{
							bCanResetPinVisible = false;
						}
					}
				}
				else if (Entry->GetRowType() == FBindingEntry::ERowType::BindingParameter)
				{
					const FMVVMBlueprintViewBinding* Binding = Entry->GetBinding(View);
					if (Binding)
					{
						const bool bSourceToDestination = UE::MVVM::IsForwardBinding(Binding->BindingType);
						if (!EditorSubsystem->CanSplitPin(WidgetBlueprint, *Binding, Entry->GetBindingParameterId(), bSourceToDestination))
						{
							bCanSplitPin = false;
						}
						if (!EditorSubsystem->CanRecombinePin(WidgetBlueprint, *Binding, Entry->GetBindingParameterId(), bSourceToDestination))
						{
							bCanRecombinePin = false;
						}
						if (!EditorSubsystem->CanResetPinToDefaultValue(WidgetBlueprint, *Binding, Entry->GetBindingParameterId(), bSourceToDestination))
						{
							bCanResetPin = false;
						}
						if (!EditorSubsystem->CanResetOrphanedPin(WidgetBlueprint, *Binding, Entry->GetBindingParameterId(), bSourceToDestination))
						{
							bCanResetOrphanedPin = false;
						}

						UEdGraphPin* GraphPin = EditorSubsystem->GetConversionFunctionArgumentPin(WidgetBlueprint, *Binding, Entry->GetBindingParameterId(), bSourceToDestination);
						if (GraphPin == nullptr)
						{
							bCanRecombinePinVisible = false;
							bCanResetPinVisible = false;
						}
						else
						{
							if (GraphPin->ParentPin == nullptr)
							{
								bCanRecombinePinVisible = false;
							}
							if (!FMVVMBlueprintPin::IsInputPin(GraphPin) || GetDefault<UEdGraphSchema_K2>()->ShouldHidePinDefaultValue(GraphPin))
							{
								bCanResetPinVisible = false;
							}
						}
					}
					else
					{
						SetAll(false);
					}
				}
				else
				{
					SetAll(false);
				}

				if (AllFalse())
				{
					break;
				}
			}

			if (bCanResetPinVisible)
			{
				FUIAction ResetPinAction;
				ResetPinAction.ExecuteAction = FExecuteAction::CreateStatic(&Private::HandleResetSelectedPin, WidgetBlueprint, View, CopiedEntries);
				ResetPinAction.CanExecuteAction = FCanExecuteAction::CreateLambda([bCanResetPin]() { return bCanResetPin; });
				MenuBuilder.AddMenuEntry(LOCTEXT("ResetPin", "Reset to Default Value"),
					LOCTEXT("ResetPinTooltip", "Reset value of this pin to the default"),
					FSlateIcon(),
					ResetPinAction);
			}
			if (bCanSplitPin)
			{
				FUIAction SplitPinAction;
				SplitPinAction.ExecuteAction = FExecuteAction::CreateStatic(&Private::HandleBreakSelectedPin, WidgetBlueprint, View, CopiedEntries);
				MenuBuilder.AddMenuEntry(LOCTEXT("BreakPin", "Split Struct Pin"),
					LOCTEXT("BreakPinTooltip", "Breaks a struct pin in to a separate pin per element."),
					FSlateIcon(),
					SplitPinAction);
			}
			if (bCanRecombinePinVisible)
			{
				FUIAction RecombinePinAction;
				RecombinePinAction.ExecuteAction = FExecuteAction::CreateStatic(&Private::HandleRecombineSelectedPin, WidgetBlueprint, View, CopiedEntries);
				RecombinePinAction.CanExecuteAction = FCanExecuteAction::CreateLambda([bCanRecombinePin]() { return bCanRecombinePin; });
				MenuBuilder.AddMenuEntry(LOCTEXT("RecombinePin", "Recombine Struct Pin"),
					LOCTEXT("RecombinePinTooltip", "Takes struct pins that have been broken in to composite elements and combines them back to a single struct pin."),
					FSlateIcon(),
					RecombinePinAction);
			}
			if (bCanResetOrphanedPin)
			{
				FUIAction ResetOrphanedPinAction;
				ResetOrphanedPinAction.ExecuteAction = FExecuteAction::CreateStatic(&Private::HandleResetOrphanedSelectedPin, WidgetBlueprint, View, CopiedEntries);
				MenuBuilder.AddMenuEntry(LOCTEXT("ResetOrphanedPin", "Remove the Orphaned Struct Pin"),
					LOCTEXT("ResetOrphanedPinTooltip", "Removes pins that used to exist but do not exist anymore."),
					FSlateIcon(),
					ResetOrphanedPinAction);
			}
		}
	}

	return MenuBuilder;
}

} // namespace UE::MVVM::BindingEntry

#undef LOCTEXT_NAMESPACE
