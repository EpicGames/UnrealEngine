// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MVVMPropertyPath.h"

class FBlueprintEditor;
class FDragDropEvent;
class UWidgetBlueprint;
class UMVVMBlueprintView;
struct FMVVMBlueprintViewBinding;
class UMVVMBlueprintViewEvent;
class UMVVMBlueprintViewCondition;
class UMVVMWidgetBlueprintExtension_View;

namespace UE::MVVM { struct FBindingEntry; }

namespace UE::MVVM::BindingEntry
{

DECLARE_DELEGATE_OneParam(FOnContextMenuEntryCallback, TConstArrayView<const TSharedPtr<FBindingEntry>> /*Entries*/);

struct FRowHelper
{
	static void GatherAllChildBindings(UMVVMBlueprintView* BlueprintView, const TConstArrayView<TSharedPtr<FBindingEntry>> Entries, TArray<FGuid>& OutBindingIds, TArray<UMVVMBlueprintViewEvent*>& OutEvents, TArray<UMVVMBlueprintViewCondition*>& OutConditions, bool bRecurse = true);

	static void DeleteEntries(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* BlueprintView, TArrayView<const TSharedPtr<FBindingEntry>> Selection);
	static void DuplicateEntries(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* BlueprintView, TArrayView<const TSharedPtr<FBindingEntry>> Selection, TArray<const TSharedPtr<FBindingEntry>>& OutSelection);

	static void CopyEntries(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* BlueprintView, const TArray<TSharedPtr<FBindingEntry>>& Entries);
	static void PasteEntries(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* BlueprintView, const TArray<TSharedPtr<FBindingEntry>>& Entries);

	static bool HasBlueprintGraph(UMVVMBlueprintView* BlueprintView, const TSharedPtr<FBindingEntry> Entry);
	static void ShowBlueprintGraph(FBlueprintEditor* Editor, UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* BlueprintView, TArrayView<const TSharedPtr<FBindingEntry>> Selection);

	static TOptional<FMVVMBlueprintPropertyPath> DropFieldSelector(const UWidgetBlueprint* WidgetBlueprint, const FDragDropEvent& DragDropEvent, bool bIsSource);
	static FReply DragOverFieldSelector(const UWidgetBlueprint* WidgetBlueprint, const FDragDropEvent& DragDropEvent, bool bIsSource);

	static FMenuBuilder CreateContextMenu(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintView* View, TConstArrayView<TSharedPtr<FBindingEntry>> Entries, FOnContextMenuEntryCallback OnSelectionChanged = FOnContextMenuEntryCallback());
};

} // namespace UE::MVVM
