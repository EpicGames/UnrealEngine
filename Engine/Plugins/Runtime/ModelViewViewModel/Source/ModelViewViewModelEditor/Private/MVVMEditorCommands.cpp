// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMEditorCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "MVVMEditorCommands"

FMVVMEditorCommands::FMVVMEditorCommands()
	: TCommands<FMVVMEditorCommands>(
		TEXT("MVVMEditorCommands"), 
		NSLOCTEXT("Contexts", "MVVM Editor", "MVVM Editor"), 
		NAME_None,
		FAppStyle::Get().GetStyleSetName())
{}

void FMVVMEditorCommands::RegisterCommands()
{
#if PLATFORM_MAC
	// On mac command and ctrl are automatically swapped. Command + Space is spotlight search so we use ctrl+space on mac to avoid the conflict
	UI_COMMAND(ToggleMVVMDrawer, "Toggle MVVM Drawer", "Toggle MVVM Drawer", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Command | EModifierKey::Shift, EKeys::B));
#else
	UI_COMMAND(ToggleMVVMDrawer, "Toggle MVVM Drawer", "Toggle MVVM Drawer", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::B));
#endif

	UI_COMMAND(PopoutEditor, "Popout Editor", "Popout a standalone Property Editor for this object.", EUserInterfaceActionType::Button, FInputChord());
}

const FMVVMEditorCommands& FMVVMEditorCommands::Get()
{
	return TCommands<FMVVMEditorCommands>::Get();
}

#undef LOCTEXT_NAMESPACE
