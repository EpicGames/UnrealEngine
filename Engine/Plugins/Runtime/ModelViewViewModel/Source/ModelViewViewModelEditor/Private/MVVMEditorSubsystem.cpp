// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMEditorSubsystem.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Bindings/MVVMConversionFunctionHelper.h"
#include "Blueprint/WidgetTree.h"
#include "BlueprintActionDatabase.h"
#include "BlueprintActionFilter.h"
#include "BlueprintNodeSpawner.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "Framework/MVVMConversionFunctionLibrary.h"
#include "Framework/MVVMConversionHelper.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_BreakStruct.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "MVVMBlueprintInstancedViewModel.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMBlueprintViewCondition.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMSubsystem.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "ScopedTransaction.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Types/MVVMAvailableBinding.h"
#include "Types/MVVMBindingSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMEditorSubsystem)

#define LOCTEXT_NAMESPACE "MVVMEditorSubsystem"

namespace UE::MVVM::Private
{
	void OnBindingPreEditChange(UMVVMBlueprintView* BlueprintView, FName PropertyName)
	{
		FProperty* ChangedProperty = FMVVMBlueprintViewBinding::StaticStruct()->FindPropertyByName(PropertyName);
		check(ChangedProperty != nullptr);

		FEditPropertyChain EditChain;
		EditChain.AddTail(UMVVMBlueprintView::StaticClass()->FindPropertyByName("Bindings"));
		EditChain.AddTail(ChangedProperty);
		EditChain.SetActivePropertyNode(ChangedProperty);

		BlueprintView->PreEditChange(EditChain);
	}

	void OnBindingPostEditChange(UMVVMBlueprintView* BlueprintView, FName PropertyName)
	{
		FProperty* ChangedProperty = FMVVMBlueprintViewBinding::StaticStruct()->FindPropertyByName(PropertyName);
		check(ChangedProperty != nullptr);

		FEditPropertyChain EditChain;
		EditChain.AddTail(UMVVMBlueprintView::StaticClass()->FindPropertyByName("Bindings"));
		EditChain.AddTail(ChangedProperty);
		EditChain.SetActivePropertyNode(ChangedProperty);

		FPropertyChangedEvent ChangeEvent(ChangedProperty, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChainEvent(EditChain, ChangeEvent);
		BlueprintView->PostEditChangeChainProperty(ChainEvent);
	}
	
	void OnEventPreEditChange(UMVVMBlueprintViewEvent* Event, FName PropertyName)
	{
		FProperty* ChangedProperty = UMVVMBlueprintViewEvent::StaticClass()->FindPropertyByName(PropertyName);
		check(ChangedProperty != nullptr);

		FEditPropertyChain EditChain;
		EditChain.AddTail(ChangedProperty);
		EditChain.SetActivePropertyNode(ChangedProperty);

		Event->SetFlags(RF_Transactional);
		Event->PreEditChange(EditChain);
	}

	void OnEventPostEditChange(UMVVMBlueprintViewEvent* Event, FName PropertyName)
	{
		FProperty* ChangedProperty = UMVVMBlueprintViewEvent::StaticClass()->FindPropertyByName(PropertyName);
		check(ChangedProperty != nullptr);

		FEditPropertyChain EditChain;
		EditChain.AddTail(ChangedProperty);
		EditChain.SetActivePropertyNode(ChangedProperty);

		FPropertyChangedEvent ChangeEvent(ChangedProperty, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChainEvent(EditChain, ChangeEvent);
		Event->PostEditChangeChainProperty(ChainEvent);
	}

	void OnConditionPreEditChange(UMVVMBlueprintViewCondition* Condition, FName PropertyName)
	{
		FProperty* ChangedProperty = UMVVMBlueprintViewCondition::StaticClass()->FindPropertyByName(PropertyName);
		check(ChangedProperty != nullptr);

		FEditPropertyChain EditChain;
		EditChain.AddTail(ChangedProperty);
		EditChain.SetActivePropertyNode(ChangedProperty);

		Condition->SetFlags(RF_Transactional);
		Condition->PreEditChange(EditChain);
	}

	void OnConditionPostEditChange(UMVVMBlueprintViewCondition* Condition, FName PropertyName)
	{
		FProperty* ChangedProperty = UMVVMBlueprintViewCondition::StaticClass()->FindPropertyByName(PropertyName);
		check(ChangedProperty != nullptr);

		FEditPropertyChain EditChain;
		EditChain.AddTail(ChangedProperty);
		EditChain.SetActivePropertyNode(ChangedProperty);

		FPropertyChangedEvent ChangeEvent(ChangedProperty, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChainEvent(EditChain, ChangeEvent);
		Condition->PostEditChangeChainProperty(ChainEvent);
	}

	UK2Node_FunctionResult* FindFunctionResult(UEdGraph* Graph)
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (UK2Node_FunctionResult* FunctionResult = Cast<UK2Node_FunctionResult>(Node))
			{
				return FunctionResult;
			}
		}
		return nullptr;
	}

	UEdGraphNode* FindConversionNode(UEdGraph* Graph)
	{
		UK2Node_FunctionResult* FunctionResult = UE::MVVM::Private::FindFunctionResult(Graph);
		if (!ensureMsgf(FunctionResult != nullptr, TEXT("Function result node not found in conversion function wrapper!")))
		{
			return nullptr;
		}

		if (!ensureMsgf(FunctionResult->UserDefinedPins.Num() == 1, TEXT("Function result should have exactly one return value.")))
		{
			return nullptr;
		}

		UEdGraphPin* ResultPin = FunctionResult->FindPin(FunctionResult->UserDefinedPins[0]->PinName, EGPD_Input);
		if (!ensureMsgf(ResultPin != nullptr, TEXT("Function result pin not found.")))
		{
			return nullptr;
		}

		if (!ensureMsgf(ResultPin->LinkedTo.Num() != 0, TEXT("Result pin not linked to anything")))
		{
			return nullptr;
		}

		// finally found our conversion node
		UEdGraphNode* ConversionNode = ResultPin->LinkedTo[0]->GetOwningNode();
		return ConversionNode;
	}

	bool LoadBlueprintReferencersForViewModelRename(const UBlueprint* Blueprint)
	{
		if (Blueprint)
		{
			FARFilter Filter;
			Filter.bIncludeOnlyOnDiskAssets = true;
			Filter.bRecursiveClasses = true;
			Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());

			IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
			AssetRegistry.GetReferencers(Blueprint->GetPackage()->GetFName(), Filter.PackageNames, UE::AssetRegistry::EDependencyCategory::Package);

			TArray<FAssetData> BlueprintReferencers;
			if (Filter.PackageNames.IsEmpty())
			{
				// Nothing references the blueprint
				return true;
			}
			
			AssetRegistry.GetAssets(Filter, BlueprintReferencers);

			BlueprintReferencers = BlueprintReferencers.FilterByPredicate([](const FAssetData& Asset)
			{
				constexpr bool bLoad = false;
				return Asset.FastGetAsset(bLoad) == nullptr;
			});

			if (!BlueprintReferencers.IsEmpty())
			{
				static const FTextFormat LoadingWarningFormat = LOCTEXT("LoadingBlueprintReferencersWarningFormat", "There {0}|plural(one=is,other=are) {0} unloaded {0}|plural(one=asset,other=assets) that might reference this view model.\nDo you want to load the assets to automatically update them now?");
				const EAppReturnType::Type DialogResult = FMessageDialog::Open(EAppMsgType::YesNoCancel, FText::Format(LoadingWarningFormat, BlueprintReferencers.Num()));
				if (DialogResult == EAppReturnType::Cancel)
				{
					return false;
				}

				if (DialogResult == EAppReturnType::Yes)
				{
					FScopedSlowTask LoadReferencersSlowTask = FScopedSlowTask(BlueprintReferencers.Num(), LOCTEXT("LoadingRenamedViewModelBindings", "Loading Dependent Blueprints"));
				
					constexpr float TimeBeforeDialog = 1.f;
					LoadReferencersSlowTask.MakeDialogDelayed(TimeBeforeDialog);
					for (const FAssetData& Asset : BlueprintReferencers)
					{
						LoadReferencersSlowTask.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("LoadingBlueprintReferencerFormat", "Loading {0}"), FText::FromName(Asset.AssetName)));
					
						constexpr bool bLoad = true;
						Asset.FastGetAsset(bLoad);
					}
				}
			}
		}

		return true;
	}

} //namespace Private


UMVVMBlueprintView* UMVVMEditorSubsystem::RequestView(UWidgetBlueprint* WidgetBlueprint) const
{
	UMVVMWidgetBlueprintExtension_View* Extension = UMVVMWidgetBlueprintExtension_View::RequestExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	Extension->SetFlags(RF_Transactional);

	if (Extension->GetBlueprintView() == nullptr)
	{
		Extension->CreateBlueprintViewInstance();
	}
	return Extension->GetBlueprintView();
}

UMVVMBlueprintView* UMVVMEditorSubsystem::GetView(const UWidgetBlueprint* WidgetBlueprint) const
{
	if (WidgetBlueprint)
	{
		if (UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint))
		{
			return ExtensionView->GetBlueprintView();
		}
	}
	return nullptr;
}

FGuid UMVVMEditorSubsystem::AddViewModel(UWidgetBlueprint* WidgetBlueprint, const UClass* ViewModelClass)
{
	FGuid Result;
	if (ViewModelClass && ViewModelClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
	{
		if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
		{
			FString DefaultViewModelName = GetDefaultViewModelName(ViewModelClass);
			FString ViewModelName = DefaultViewModelName;
			FKismetNameValidator NameValidator(WidgetBlueprint);

			bool bHasInterfaceOfSameType = false;
			const TArrayView<const FMVVMBlueprintViewModelContext> ViewModels = View->GetViewModels();
			if (const FMVVMBlueprintViewModelContext* ViewModel = ViewModels.FindByPredicate([&ViewModelClass](FMVVMBlueprintViewModelContext ViewModel) { return ViewModel.GetViewModelClass() == ViewModelClass && ViewModel.bUseAsInterface; }))
			{
				bHasInterfaceOfSameType = true;
			}

			int32 Index = 1;
			while (NameValidator.IsValid(ViewModelName) != EValidatorResult::Ok)
			{
				ViewModelName = DefaultViewModelName + "_";
				ViewModelName.AppendInt(Index);

				++Index;
			}
			const bool bHasViewModelWithDefaultName = ViewModelName != DefaultViewModelName;

			bool bSupportUseAsInterfaceSetting = false;
			IConsoleVariable* CVarSupportUseAsInterfaceSetting = IConsoleManager::Get().FindConsoleVariable(TEXT("MVVM.SupportUseAsInterfaceSetting")); ensure(CVarSupportUseAsInterfaceSetting);
			if (CVarSupportUseAsInterfaceSetting)
			{
				bSupportUseAsInterfaceSetting = CVarSupportUseAsInterfaceSetting->GetBool();
			}
			const bool bCanAddAsInterface = bSupportUseAsInterfaceSetting && !bHasInterfaceOfSameType && !bHasViewModelWithDefaultName;

			FMVVMBlueprintViewModelContext Context = FMVVMBlueprintViewModelContext(ViewModelClass, *ViewModelName);
			if (Context.IsValid())
			{
				Context.Resolver = Context.CreateDefaultResolver(WidgetBlueprint);
				Context.bUseAsInterface = bCanAddAsInterface;

				const FScopedTransaction Transaction(LOCTEXT("AddViewModel", "Add viewmodel"));
				View->Modify();
				View->AddViewModel(Context);
				Result = Context.GetViewModelId();
			}
		}
	}
	return Result;
}

FString UMVVMEditorSubsystem::GetDefaultViewModelName(const UClass* ViewModelClass)
{
	if (ViewModelClass == nullptr)
	{
		return "";
	}

	FString ClassName = ViewModelClass->ClassGeneratedBy != nullptr ? ViewModelClass->ClassGeneratedBy->GetName() : ViewModelClass->GetAuthoredName();
	if (Cast<UBlueprintGeneratedClass>(ViewModelClass) && ClassName.EndsWith(TEXT("_C")))
	{
		ClassName.RemoveAt(ClassName.Len() - 2, 2);
	}

	return ClassName;
}

FGuid UMVVMEditorSubsystem::AddInstancedViewModel(UWidgetBlueprint* WidgetBlueprint)
{
	UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	UMVVMBlueprintView* View = ExtensionView ? ExtensionView->GetBlueprintView() : nullptr;

	if (!View)
	{
		return FGuid();
	}

	const FScopedTransaction Transaction(LOCTEXT("AddInstancedViewModel", "Add instanced viewmodel"));

	FGuid Result;

	FName UniqueName = MakeUniqueObjectName(View, UMVVMBlueprintInstancedViewModel_PropertyBag::StaticClass(), "InstancedViewmodel");
	UMVVMBlueprintInstancedViewModel_PropertyBag* NewInstancedViewModel = NewObject<UMVVMBlueprintInstancedViewModel_PropertyBag>(View, UniqueName);
	NewInstancedViewModel->GenerateClass(true);
	FMVVMBlueprintViewModelContext Context = FMVVMBlueprintViewModelContext(NewInstancedViewModel->GetGeneratedClass(), UniqueName);
	if (Context.IsValid())
	{
		Context.InstancedViewModel = NewInstancedViewModel;
		Context.CreationType = EMVVMBlueprintViewModelContextCreationType::CreateInstance;
		View->Modify();
		View->AddViewModel(Context);
		Result = Context.GetViewModelId();
	}
	else
	{
		auto RenameToTransient = [](UObject* ObjectToRename)
		{
			FName TrashName = MakeUniqueObjectName(GetTransientPackage(), ObjectToRename->GetClass(), *FString::Printf(TEXT("TRASH_%s"), *ObjectToRename->GetName()));
			ObjectToRename->Rename(*TrashName.ToString(), GetTransientPackage());
		};
		if (NewInstancedViewModel->GetGeneratedClass())
		{
			RenameToTransient(NewInstancedViewModel->GetGeneratedClass());
		}
		RenameToTransient(NewInstancedViewModel);
	}

	return Result;
}

void UMVVMEditorSubsystem::RemoveViewModel(UWidgetBlueprint* WidgetBlueprint, FName ViewModel)
{
	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		if (const FMVVMBlueprintViewModelContext* ViewModelContext = View->FindViewModel(ViewModel))
		{
			if (ViewModelContext->bCanRemove)
			{
				View->RemoveViewModel(ViewModelContext->GetViewModelId());
			}
		}
	}
}

bool UMVVMEditorSubsystem::VerifyViewModelRename(UWidgetBlueprint* WidgetBlueprint, FName ViewModel, FName NewViewModel, FText& OutError)
{
	FKismetNameValidator Validator(WidgetBlueprint);
	EValidatorResult ValidatorResult = Validator.IsValid(NewViewModel);
	if (ValidatorResult != EValidatorResult::Ok)
	{
		if (ViewModel == NewViewModel && (ValidatorResult == EValidatorResult::AlreadyInUse || ValidatorResult == EValidatorResult::ExistingName))
		{
			// Continue successfully
		}
		else
		{
			OutError = INameValidatorInterface::GetErrorText(NewViewModel.ToString(), ValidatorResult);
			return false;
		}
	}
	return true;
}

bool UMVVMEditorSubsystem::RenameViewModel(UWidgetBlueprint* WidgetBlueprint, FName ViewModel, FName NewViewModel, FText& OutError)
{
	if (!VerifyViewModelRename(WidgetBlueprint, ViewModel, NewViewModel, OutError))
	{
		return false;
	}

	UMVVMBlueprintView* View = GetView(WidgetBlueprint);
	if (View == nullptr)
	{
		return false;
	}

	const FMVVMBlueprintViewModelContext* ViewModelContext = View->FindViewModel(ViewModel);
	if (ViewModelContext && ViewModelContext->bCanRename)
	{
		// Load any dependent BPs so that they can update any bindings that include the renamed view model
		// This is required because FMVVMBlueprintFieldPath::BindingReference did not always serialize the GUID for the generated view model properties, so it could not resolve the field if it was renamed
		// Do this before starting the transaction so that any changes during load are excluded
		if (UE::MVVM::Private::LoadBlueprintReferencersForViewModelRename(WidgetBlueprint))
		{
			const FScopedTransaction Transaction(LOCTEXT("RenameViewModel", "Rename Viewmodel"));
			View->Modify();
			return View->RenameViewModel(ViewModel, NewViewModel);
		}
	}

	return false;
}

bool UMVVMEditorSubsystem::ReparentViewModel(UWidgetBlueprint* WidgetBlueprint, FName ViewModel, const UClass* ViewModelClass, FText& OutError)
{
	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		if (const FMVVMBlueprintViewModelContext* ViewModelContext = View->FindViewModel(ViewModel))
		{
			FScopedTransaction Transaction(LOCTEXT("ReparentViewmodel", "Reparent Viewmodel"));
			return View->ReparentViewModel(ViewModelContext->GetViewModelId(), ViewModelClass);
		}
	}
	return false;
}

FMVVMBlueprintViewBinding& UMVVMEditorSubsystem::AddBinding(UWidgetBlueprint* WidgetBlueprint)
{
	UMVVMBlueprintView* View = RequestView(WidgetBlueprint);

	FScopedTransaction Transaction(LOCTEXT("MVVMAddBinding", "MVVM Add Binding"));
	View->Modify();

	return View->AddDefaultBinding();
}

void UMVVMEditorSubsystem::RemoveBinding(UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding)
{
	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		FScopedTransaction Transaction(LOCTEXT("MVVMRemoveBinding", "MVVM Remove Binding"));
		View->Modify();

		View->RemoveBinding(&Binding);
	}
}

UMVVMBlueprintViewEvent* UMVVMEditorSubsystem::AddEvent(UWidgetBlueprint* WidgetBlueprint)
{
	if (GetDefault<UMVVMDeveloperProjectSettings>()->bAllowBindingEvent)
	{
		UMVVMBlueprintView* View = RequestView(WidgetBlueprint);

		FScopedTransaction Transaction(LOCTEXT("MVVMAddEvent", "MVVM Add Event"));
		View->Modify();

		return View->AddDefaultEvent();
	}
	return nullptr;
}

void UMVVMEditorSubsystem::RemoveEvent(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event)
{
	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		FScopedTransaction Transaction(LOCTEXT("MVVMRemoveEvent", "MVVM Remove Event"));
		View->Modify();

		View->RemoveEvent(Event);
	}
}

UMVVMBlueprintViewCondition* UMVVMEditorSubsystem::AddCondition(UWidgetBlueprint* WidgetBlueprint)
{
	if (!GetDefault<UMVVMDeveloperProjectSettings>()->bAllowConditionBinding)
	{
		return nullptr;
	}

	UMVVMBlueprintView* View = RequestView(WidgetBlueprint);

	FScopedTransaction Transaction(LOCTEXT("MVVMAddCondition", "MVVM Add Condition"));
	View->Modify();

	return View->AddDefaultCondition();
}

void UMVVMEditorSubsystem::RemoveCondition(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewCondition* Condition)
{
	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		FScopedTransaction Transaction(LOCTEXT("MVVMRemoveCondition", "MVVM Remove Condition"));
		View->Modify();

		View->RemoveCondition(Condition);
	}
}


UFunction* UMVVMEditorSubsystem::GetConversionFunction(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, bool bSourceToDestination) const
{
	if (UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(bSourceToDestination))
	{
		FMVVMBlueprintFunctionReference Result = ConversionFunction->GetConversionFunction();
		if (Result.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
		{
			return const_cast<UFunction*>(Result.GetFunction(WidgetBlueprint));
		}
	}
	return nullptr;
}

UEdGraphPin* UMVVMEditorSubsystem::GetConversionFunctionArgumentPin(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination) const
{
	if (UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(bSourceToDestination))
	{
		return ConversionFunction->GetOrCreateGraphPin(const_cast<UWidgetBlueprint*>(WidgetBlueprint), ParameterId);
	}
	return nullptr;
}

void UMVVMEditorSubsystem::SetSourceToDestinationConversionFunction(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const UFunction* NewConversionFunction)
{
	SetSourceToDestinationConversionFunction(WidgetBlueprint, Binding, FMVVMBlueprintFunctionReference(WidgetBlueprint, NewConversionFunction));
}

void UMVVMEditorSubsystem::SetSourceToDestinationConversionFunction(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, FMVVMBlueprintFunctionReference NewConversionFunction)
{
	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		if (NewConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
		{
			const UFunction* NewFunction = NewConversionFunction.GetFunction(WidgetBlueprint);

			// The new source path will be the conversion function, only validate whether the conversion's return type matches the destination
			if (!IsValidConversionFunction(WidgetBlueprint, NewFunction, FMVVMBlueprintPropertyPath(), Binding.DestinationPath))
			{
				NewConversionFunction = FMVVMBlueprintFunctionReference();
			}
		}
		else if (NewConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Node)
		{
			if (NewConversionFunction.GetNode().Get() == nullptr)
			{
				NewConversionFunction = FMVVMBlueprintFunctionReference();
			}
		}

		const FMVVMBlueprintFunctionReference PreviousConversionFunction = Binding.Conversion.SourceToDestinationConversion != nullptr ? Binding.Conversion.SourceToDestinationConversion->GetConversionFunction() : FMVVMBlueprintFunctionReference();
		if (PreviousConversionFunction != NewConversionFunction)
		{
			FScopedTransaction Transaction(LOCTEXT("SetConversionFunction", "Set Conversion Function"));

			WidgetBlueprint->Modify();

			UE::MVVM::Private::OnBindingPreEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, Conversion));

			if (Binding.Conversion.SourceToDestinationConversion)
			{
				Binding.Conversion.SourceToDestinationConversion->RemoveWrapperGraph(WidgetBlueprint);
				Binding.Conversion.SourceToDestinationConversion = nullptr;
			}
			Binding.SourcePath = FMVVMBlueprintPropertyPath();

			if (NewConversionFunction.GetType() != EMVVMBlueprintFunctionReferenceType::None)
			{
				Binding.Conversion.SourceToDestinationConversion = NewObject<UMVVMBlueprintViewConversionFunction>(WidgetBlueprint);
				FName GraphName = UE::MVVM::ConversionFunctionHelper::CreateWrapperName(Binding, true);

				// Set Destination Path in case this is an async conversion function which will handle destination update in graph
				Binding.Conversion.SourceToDestinationConversion->SetDestinationPath(Binding.DestinationPath);
				Binding.Conversion.SourceToDestinationConversion->Initialize(WidgetBlueprint, GraphName, NewConversionFunction);
			}

			GenerateBindToDestinationPathsForBinding(WidgetBlueprint, Binding);

			UE::MVVM::Private::OnBindingPostEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, Conversion));
			FBlueprintEditorUtils::MarkBlueprintAsModified(WidgetBlueprint);
		}
	}
}

void UMVVMEditorSubsystem::SetDestinationToSourceConversionFunction(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const UFunction* NewConversionFunction)
{
	SetDestinationToSourceConversionFunction(WidgetBlueprint, Binding, FMVVMBlueprintFunctionReference(WidgetBlueprint, NewConversionFunction));
}

void UMVVMEditorSubsystem::SetDestinationToSourceConversionFunction(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, FMVVMBlueprintFunctionReference NewConversionFunction)
{
	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		if (NewConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
		{
			const UFunction* NewFunction = NewConversionFunction.GetFunction(WidgetBlueprint);
			// The new destination path will be the conversion function, only validate whether the conversion's return type matches the destination
			if (!IsValidConversionFunction(WidgetBlueprint, NewFunction, Binding.SourcePath, FMVVMBlueprintPropertyPath()))
			{
				NewConversionFunction = FMVVMBlueprintFunctionReference();
			}
		}
		else if (NewConversionFunction.GetType() == EMVVMBlueprintFunctionReferenceType::Node)
		{
			if (NewConversionFunction.GetNode().Get() == nullptr)
			{
				NewConversionFunction = FMVVMBlueprintFunctionReference();
			}
		}

		const FMVVMBlueprintFunctionReference PreviousConversionFunction = Binding.Conversion.DestinationToSourceConversion != nullptr ? Binding.Conversion.DestinationToSourceConversion->GetConversionFunction() : FMVVMBlueprintFunctionReference();
		if (PreviousConversionFunction != NewConversionFunction)
		{
			FScopedTransaction Transaction(LOCTEXT("SetConversionFunction", "Set Conversion Function"));

			WidgetBlueprint->Modify();

			UE::MVVM::Private::OnBindingPreEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, Conversion));

			if (Binding.Conversion.DestinationToSourceConversion)
			{
				Binding.Conversion.DestinationToSourceConversion->RemoveWrapperGraph(WidgetBlueprint);
				Binding.Conversion.DestinationToSourceConversion = nullptr;
			}
			Binding.DestinationPath = FMVVMBlueprintPropertyPath();

			if (NewConversionFunction.GetType() != EMVVMBlueprintFunctionReferenceType::None)
			{
				Binding.Conversion.DestinationToSourceConversion = NewObject<UMVVMBlueprintViewConversionFunction>(WidgetBlueprint);
				FName GraphName = UE::MVVM::ConversionFunctionHelper::CreateWrapperName(Binding, true);

				// Set Destination Path in case this is an async conversion function which will handle destination update in graph
				Binding.Conversion.DestinationToSourceConversion->SetDestinationPath(Binding.DestinationPath);
				Binding.Conversion.DestinationToSourceConversion->Initialize(WidgetBlueprint, GraphName, NewConversionFunction);
			}

			UE::MVVM::Private::OnBindingPostEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, Conversion));
			FBlueprintEditorUtils::MarkBlueprintAsModified(WidgetBlueprint);
		}
	}
}

void UMVVMEditorSubsystem::SetDestinationPathForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, FMVVMBlueprintPropertyPath PropertyPath, bool bAllowEventConversion)
{
	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		bool bHasConversion = Binding.Conversion.DestinationToSourceConversion != nullptr;
		bool bEventSupported = bAllowEventConversion && UMVVMBlueprintViewEvent::Supports(WidgetBlueprint, PropertyPath);

		if (bEventSupported || bHasConversion || Binding.DestinationPath != PropertyPath)
		{
			FScopedTransaction Transaction(LOCTEXT("SetBindingProperty", "Set Binding Property"));

			UE::MVVM::Private::OnBindingPreEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, DestinationPath));

			UMVVMBlueprintViewEvent* Event = nullptr;
			if (bEventSupported)
			{
				Event = AddEvent(WidgetBlueprint);
			}

			if (Event)
			{
				Event->SetEventPath(PropertyPath);
				View->RemoveBinding(&Binding);
			}
			else
			{
				if (Binding.Conversion.DestinationToSourceConversion)
				{
					Binding.Conversion.DestinationToSourceConversion->RemoveWrapperGraph(WidgetBlueprint);
					Binding.Conversion.DestinationToSourceConversion = nullptr;
				}
				Binding.DestinationPath = PropertyPath;
			}

			GenerateBindToDestinationPathsForBinding(WidgetBlueprint, Binding);

			UE::MVVM::Private::OnBindingPostEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, DestinationPath));
			FBlueprintEditorUtils::MarkBlueprintAsModified(WidgetBlueprint);
		}
	}
}

void UMVVMEditorSubsystem::SetSourcePathForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, FMVVMBlueprintPropertyPath PropertyPath)
{
	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		bool bHasConversion = Binding.Conversion.SourceToDestinationConversion != nullptr;
		if (bHasConversion || Binding.SourcePath != PropertyPath)
		{
			FScopedTransaction Transaction(LOCTEXT("SetBindingProperty", "Set Binding Property"));

			UE::MVVM::Private::OnBindingPreEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, SourcePath));

			if (Binding.Conversion.SourceToDestinationConversion)
			{
				Binding.Conversion.SourceToDestinationConversion->RemoveWrapperGraph(WidgetBlueprint);
				Binding.Conversion.SourceToDestinationConversion = nullptr;
			}
			Binding.SourcePath = PropertyPath;

			UE::MVVM::Private::OnBindingPostEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, SourcePath));
			FBlueprintEditorUtils::MarkBlueprintAsModified(WidgetBlueprint);
		}
	}
}

void UMVVMEditorSubsystem::OverrideExecutionModeForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, EMVVMExecutionMode Mode)
{
	if (!Binding.bOverrideExecutionMode || Binding.OverrideExecutionMode != Mode)
	{
		if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
		{
			FScopedTransaction Transaction(LOCTEXT("SetExecutionMode", "Set Execution Mode"));

			UE::MVVM::Private::OnBindingPreEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, OverrideExecutionMode));

			Binding.bOverrideExecutionMode = true;
			Binding.OverrideExecutionMode = Mode;

			UE::MVVM::Private::OnBindingPostEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, OverrideExecutionMode));
			FBlueprintEditorUtils::MarkBlueprintAsModified(WidgetBlueprint);
		}
	}
}

void UMVVMEditorSubsystem::ResetExecutionModeForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding)
{
	if (Binding.bOverrideExecutionMode)
	{
		if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
		{
			FScopedTransaction Transaction(LOCTEXT("ResetExecutionMode", "Reset Execution Mode"));

			UE::MVVM::Private::OnBindingPreEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, OverrideExecutionMode));

			Binding.bOverrideExecutionMode = false;

			UE::MVVM::Private::OnBindingPostEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, OverrideExecutionMode));
			FBlueprintEditorUtils::MarkBlueprintAsModified(WidgetBlueprint);
		}
	}
}

void UMVVMEditorSubsystem::SetBindingTypeForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, EMVVMBindingMode Type)
{
	if (Binding.BindingType != Type)
	{
		if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
		{
			FScopedTransaction Transaction(LOCTEXT("SetBindingType", "Set Binding Type"));

			UE::MVVM::Private::OnBindingPreEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, BindingType));

			Binding.BindingType = Type;

			UE::MVVM::Private::OnBindingPostEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, BindingType));
			FBlueprintEditorUtils::MarkBlueprintAsModified(WidgetBlueprint);
		}
	}
}

void UMVVMEditorSubsystem::SetEnabledForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, bool bEnabled)
{
	if (Binding.bEnabled != bEnabled)
	{
		if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
		{
			FScopedTransaction Transaction(LOCTEXT("SetBindingEnabled", "Set Binding Enabled"));

			UE::MVVM::Private::OnBindingPreEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, bEnabled));

			Binding.bEnabled = bEnabled;

			UE::MVVM::Private::OnBindingPostEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, bEnabled));
			FBlueprintEditorUtils::MarkBlueprintAsModified(WidgetBlueprint);
		}
	}
}

void UMVVMEditorSubsystem::SetCompileForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, bool bCompile)
{
	if (Binding.bCompile != bCompile)
	{
		if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
		{
			FScopedTransaction Transaction(LOCTEXT("SetBindingCompiled", "Set Binding Compiled"));

			UE::MVVM::Private::OnBindingPreEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, bCompile));

			Binding.bCompile = bCompile;

			UE::MVVM::Private::OnBindingPostEditChange(View, GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, bCompile));
			FBlueprintEditorUtils::MarkBlueprintAsModified(WidgetBlueprint);
		}
	}
}

void UMVVMEditorSubsystem::GenerateBindToDestinationPathsForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding)
{
	if (UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(UE::MVVM::IsForwardBinding(Binding.BindingType)))
	{
		FString MVVMBindToDestinationString = UE::MVVM::FConversionHelper::GetBindToDestinationStringFromConversionFunction(WidgetBlueprint,ConversionFunction);
		if (!MVVMBindToDestinationString.IsEmpty())
		{
			auto GetPinId = [&](const FString& InParamName)
				{
					for (const FMVVMBlueprintPin& Pin : Binding.Conversion.SourceToDestinationConversion->GetPins())
					{
						if (Pin.GetId().GetNames().Contains(InParamName))
						{
							return Pin.GetId();
						}
					}

					return FMVVMBlueprintPinId();
				};

			FMVVMBlueprintPinId MVVMBindToDestinationPinId = GetPinId(MVVMBindToDestinationString);
			if (MVVMBindToDestinationPinId.IsValid())
			{
				const UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
				Subsystem->SetPathForConversionFunctionArgument(WidgetBlueprint, Binding, MVVMBindToDestinationPinId, Binding.DestinationPath, true);
			}
		}
	}
}

void UMVVMEditorSubsystem::SetEventPath(UMVVMBlueprintViewEvent* Event, FMVVMBlueprintPropertyPath PropertyPath, bool bRequestBindingConversion)
{
	UMVVMBlueprintView* View = Event ? Event->GetOuterUMVVMBlueprintView() : nullptr;
	if (View)
	{
		UWidgetBlueprint* WidgetBlueprint = Event->GetOuterUMVVMBlueprintView()->GetOuterUMVVMWidgetBlueprintExtension_View()->GetWidgetBlueprint();

		FScopedTransaction Transaction(LOCTEXT("SetEventPath", "Set Event Path"));

		FName EventPath = "EventPath";
		UE::MVVM::Private::OnEventPreEditChange(Event, EventPath);

		bool bSupports = UMVVMBlueprintViewEvent::Supports(WidgetBlueprint, PropertyPath);

		if (bRequestBindingConversion || !bSupports)
		{
			FMVVMBlueprintViewBinding& Binding = AddBinding(WidgetBlueprint);
			SetDestinationPathForBinding(WidgetBlueprint, Binding, PropertyPath, false);
			View->RemoveEvent(Event);
		}
		if (bSupports)
		{
			Event->SetEventPath(PropertyPath);
		}
		else
		{
			Event->SetEventPath(FMVVMBlueprintPropertyPath());
		}

		UE::MVVM::Private::OnEventPostEditChange(Event, EventPath);
	}
}

void UMVVMEditorSubsystem::SetEventDestinationPath(UMVVMBlueprintViewEvent* Event, FMVVMBlueprintPropertyPath PropertyPath)
{
	const UMVVMBlueprintView* View = Event ? Event->GetOuterUMVVMBlueprintView() : nullptr;
	if (View)
	{
		FScopedTransaction Transaction(LOCTEXT("SetEventDestinationPath", "Set Destination Path"));

		FName DestinationPath = "DestinationPath";
		UE::MVVM::Private::OnEventPreEditChange(Event, DestinationPath);

		Event->SetDestinationPath(PropertyPath);

		UE::MVVM::Private::OnEventPostEditChange(Event, DestinationPath);
	}
}

void UMVVMEditorSubsystem::SetEventArgumentPath(UMVVMBlueprintViewEvent* Event,  const FMVVMBlueprintPinId& ParameterId, const FMVVMBlueprintPropertyPath& Path) const
{
	const UMVVMBlueprintView* View = Event ? Event->GetOuterUMVVMBlueprintView() : nullptr;
	if (View)
	{
		FScopedTransaction Transaction(LOCTEXT("SetEventPath", "Set Event Path"));

		UE::MVVM::Private::OnEventPreEditChange(Event, "SavedPins");
	
		Event->SetPinPath(ParameterId, Path);

		UE::MVVM::Private::OnEventPostEditChange(Event, "SavedPins");
	}
}

void UMVVMEditorSubsystem::SetEnabledForEvent(UMVVMBlueprintViewEvent* Event, bool bEnabled)
{
	if (Event->bEnabled != bEnabled)
	{
		const UMVVMBlueprintView* View = Event ? Event->GetOuterUMVVMBlueprintView() : nullptr;
		if (View)
		{
			FScopedTransaction Transaction(LOCTEXT("SetBindingEnabled", "Set Binding Enabled"));

			UE::MVVM::Private::OnEventPreEditChange(Event, GET_MEMBER_NAME_CHECKED(UMVVMBlueprintViewEvent, bEnabled));

			Event->bEnabled = bEnabled;

			UE::MVVM::Private::OnEventPostEditChange(Event, GET_MEMBER_NAME_CHECKED(UMVVMBlueprintViewEvent, bEnabled));
		}
	}
}

void UMVVMEditorSubsystem::SetCompileForEvent(UMVVMBlueprintViewEvent* Event, bool bCompile)
{
	if (Event->bCompile != bCompile)
	{
		const UMVVMBlueprintView* View = Event ? Event->GetOuterUMVVMBlueprintView() : nullptr;
		if (View)
		{
			FScopedTransaction Transaction(LOCTEXT("SetBindingCompiled", "Set Binding Compiled"));

			UE::MVVM::Private::OnEventPreEditChange(Event, GET_MEMBER_NAME_CHECKED(UMVVMBlueprintViewEvent, bCompile));

			Event->bCompile = bCompile;

			UE::MVVM::Private::OnEventPostEditChange(Event, GET_MEMBER_NAME_CHECKED(UMVVMBlueprintViewEvent, bCompile));
		}
	}
}

void UMVVMEditorSubsystem::SetConditionPath(UMVVMBlueprintViewCondition* Condition, FMVVMBlueprintPropertyPath PropertyPath, bool bRequestBindingConversion)
{
	const UMVVMBlueprintView* View = Condition ? Condition->GetOuterUMVVMBlueprintView() : nullptr;
	if (View)
	{
		FScopedTransaction Transaction(LOCTEXT("SetConditionPath", "Set Condition Path"));

		FName ConditionPath = "ConditionPath";
		UE::MVVM::Private::OnConditionPreEditChange(Condition, ConditionPath);

		Condition->SetConditionPath(PropertyPath);

		UE::MVVM::Private::OnConditionPostEditChange(Condition, ConditionPath);
	}
}

void UMVVMEditorSubsystem::SetConditionDestinationPath(UMVVMBlueprintViewCondition* Condition, FMVVMBlueprintPropertyPath PropertyPath)
{
	const UMVVMBlueprintView* View = Condition ? Condition->GetOuterUMVVMBlueprintView() : nullptr;
	if (View)
	{
		FScopedTransaction Transaction(LOCTEXT("SetConditionDestinationPath", "Set Condition Destination Path"));

		FName DestinationPath = "DestinationPath";
		UE::MVVM::Private::OnConditionPreEditChange(Condition, DestinationPath);

		Condition->SetDestinationPath(PropertyPath);

		UE::MVVM::Private::OnConditionPostEditChange(Condition, DestinationPath);
	}
}

void UMVVMEditorSubsystem::SetConditionArgumentPath(UMVVMBlueprintViewCondition* Condition, const FMVVMBlueprintPinId& PinId, const FMVVMBlueprintPropertyPath& PropertyPath) const
{
	const UMVVMBlueprintView* View = Condition ? Condition->GetOuterUMVVMBlueprintView() : nullptr;
	if (View)
	{
		FScopedTransaction Transaction(LOCTEXT("SetConditionParameterPath", "Set Condition Parameter Path"));

		UE::MVVM::Private::OnConditionPreEditChange(Condition, "SavedPins");

		Condition->SetPinPath(PinId, PropertyPath);

		UE::MVVM::Private::OnConditionPostEditChange(Condition, "SavedPins");
	}

}

void UMVVMEditorSubsystem::SetEnabledForCondition(UMVVMBlueprintViewCondition* Condition, bool bEnabled)
{
	if (Condition->bEnabled != bEnabled)
	{
		const UMVVMBlueprintView* View = Condition ? Condition->GetOuterUMVVMBlueprintView() : nullptr;
		if (View)
		{
			FScopedTransaction Transaction(LOCTEXT("SetBindingEnabled", "Set Binding Enabled"));

			UE::MVVM::Private::OnConditionPreEditChange(Condition, GET_MEMBER_NAME_CHECKED(UMVVMBlueprintViewCondition, bEnabled));

			Condition->bEnabled = bEnabled;

			UE::MVVM::Private::OnConditionPostEditChange(Condition, GET_MEMBER_NAME_CHECKED(UMVVMBlueprintViewCondition, bEnabled));
		}
	}
}

void UMVVMEditorSubsystem::SetCompileForCondition(UMVVMBlueprintViewCondition* Condition, bool bCompile)
{
	if (Condition->bCompile != bCompile)
	{
		const UMVVMBlueprintView* View = Condition ? Condition->GetOuterUMVVMBlueprintView() : nullptr;
		if (View)
		{
			FScopedTransaction Transaction(LOCTEXT("SetBindingCompiled", "Set Binding Compiled"));

			UE::MVVM::Private::OnConditionPreEditChange(Condition, GET_MEMBER_NAME_CHECKED(UMVVMBlueprintViewCondition, bCompile));

			Condition->bCompile = bCompile;

			UE::MVVM::Private::OnConditionPostEditChange(Condition, GET_MEMBER_NAME_CHECKED(UMVVMBlueprintViewCondition, bCompile));
		}
	}
}

void UMVVMEditorSubsystem::SetConditionOperation(UMVVMBlueprintViewCondition* Condition, EMVVMConditionOperation Operation)
{
	const UMVVMBlueprintView* View = Condition ? Condition->GetOuterUMVVMBlueprintView() : nullptr;
	if (View)
	{
		FScopedTransaction Transaction(LOCTEXT("SetConditionOperation", "Set Condition Operation"));

		FName ConditionOperation = "ConditionOperation";
		UE::MVVM::Private::OnConditionPreEditChange(Condition, ConditionOperation);

		Condition->SetOperation(Operation);

		UE::MVVM::Private::OnConditionPostEditChange(Condition, ConditionOperation);
	}

}

void UMVVMEditorSubsystem::SetConditionOperationValue(UMVVMBlueprintViewCondition* Condition, float Value)
{
	const UMVVMBlueprintView* View = Condition ? Condition->GetOuterUMVVMBlueprintView() : nullptr;
	if (View)
	{
		FScopedTransaction Transaction(LOCTEXT("SetConditionOperationValue", "Set Condition Operation Value"));

		FName ValueName = "Value";
		UE::MVVM::Private::OnConditionPreEditChange(Condition, ValueName);

		Condition->SetOperationValue(Value);

		UE::MVVM::Private::OnConditionPostEditChange(Condition, ValueName);
	}
}

void UMVVMEditorSubsystem::SetConditionOperationMaxValue(UMVVMBlueprintViewCondition* Condition, float MaxValue)
{
	const UMVVMBlueprintView* View = Condition ? Condition->GetOuterUMVVMBlueprintView() : nullptr;
	if (View)
	{
		FScopedTransaction Transaction(LOCTEXT("SetConditionOperationMaxValue", "Set Condition Operation Maximum Value"));

		FName ValueName = "MaxValue";
		UE::MVVM::Private::OnConditionPreEditChange(Condition, ValueName);

		Condition->SetOperationMaxValue(MaxValue);

		UE::MVVM::Private::OnConditionPostEditChange(Condition, ValueName);
	}
}


namespace UE::MVVM::Private
{
UClass* GetCallingContext(const UWidgetBlueprint* WidgetBlueprint)
{
	return WidgetBlueprint->GeneratedClass ? WidgetBlueprint->GeneratedClass :
		WidgetBlueprint->SkeletonGeneratedClass ? WidgetBlueprint->SkeletonGeneratedClass : WidgetBlueprint->ParentClass;
}
}

bool UMVVMEditorSubsystem::IsValidConversionFunction(const UWidgetBlueprint* WidgetBlueprint, const UFunction* Function, const FMVVMBlueprintPropertyPath& Source, const FMVVMBlueprintPropertyPath& Destination) const
{
	if (WidgetBlueprint == nullptr || Function == nullptr)
	{
		return false;
	}

	UClass* CallingContext = UE::MVVM::Private::GetCallingContext(WidgetBlueprint);

	const FProperty* SourceProperty = nullptr;
	TArray<UE::MVVM::FMVVMConstFieldVariant> SourceFields = Source.GetFields(CallingContext);
	if (SourceFields.Num() > 0)
	{
		SourceProperty = SourceFields.Last().IsProperty() ? SourceFields.Last().GetProperty() : UE::MVVM::BindingHelper::GetReturnProperty(SourceFields.Last().GetFunction());
	}

	const FProperty* DestinationProperty = nullptr;
	TArray<UE::MVVM::FMVVMConstFieldVariant> DestFields = Destination.GetFields(CallingContext);
	if (DestFields.Num() > 0)
	{
		if (!DestFields.Last().IsEmpty())
		{
			DestinationProperty = DestFields.Last().IsProperty() ? DestFields.Last().GetProperty() : UE::MVVM::BindingHelper::GetFirstArgumentProperty(DestFields.Last().GetFunction());
		}
	}

	return IsValidConversionFunction(WidgetBlueprint, Function, SourceProperty, DestinationProperty);
}

bool UMVVMEditorSubsystem::IsValidConversionFunction(const UWidgetBlueprint* WidgetBlueprint, const UFunction* Function, const FProperty* ExpectedArgumentType, const FProperty* ExptectedReturnType) const
{
	if (WidgetBlueprint == nullptr || Function == nullptr)
	{
		return false;
	}

	if (!UMVVMBlueprintViewConversionFunction::IsValidConversionFunction(WidgetBlueprint, Function))
	{
		return false;
	}


	// check that at least one source -> argument binding is compatible
	if (ExpectedArgumentType)
	{
		TValueOrError<TArray<const FProperty*>, FText> ArgumentsResult = UE::MVVM::BindingHelper::TryGetArgumentsForConversionFunction(Function);
		check(ArgumentsResult.HasValue());

		bool bAnyCompatible = false;

		const TArray<const FProperty*>& ConversionArgProperties = ArgumentsResult.GetValue();
		for (const FProperty* ArgumentProperty : ConversionArgProperties)
		{
			if (UE::MVVM::BindingHelper::ArePropertiesCompatible(ExpectedArgumentType, ArgumentProperty))
			{
				bAnyCompatible = true;
				break;
			}
		}
		if (!bAnyCompatible)
		{
			return false;
		}
	}

	if (ExptectedReturnType)
	{
		TValueOrError<const FProperty*, FText> ReturnResult = UE::MVVM::BindingHelper::TryGetReturnTypeForConversionFunction(Function);
		check(ReturnResult.HasValue());

		// check that the return -> dest is valid
		if (!UE::MVVM::BindingHelper::ArePropertiesCompatible(ReturnResult.GetValue(), ExptectedReturnType))
		{
			return false;
		}
	}

	return true;
}

bool UMVVMEditorSubsystem::IsValidConversionNode(const UWidgetBlueprint* WidgetBlueprint, const TSubclassOf<UK2Node> Function, const FMVVMBlueprintPropertyPath& Source, const FMVVMBlueprintPropertyPath& Destination) const
{
	if (WidgetBlueprint == nullptr || Function.Get() == nullptr)
	{
		return false;
	}

	UClass* CallingContext = UE::MVVM::Private::GetCallingContext(WidgetBlueprint);

	const FProperty* SourceProperty = nullptr;
	TArray<UE::MVVM::FMVVMConstFieldVariant> SourceFields = Source.GetFields(CallingContext);
	if (SourceFields.Num() > 0)
	{
		SourceProperty = SourceFields.Last().IsProperty() ? SourceFields.Last().GetProperty() : UE::MVVM::BindingHelper::GetReturnProperty(SourceFields.Last().GetFunction());
	}

	const FProperty* DestinationProperty = nullptr;
	TArray<UE::MVVM::FMVVMConstFieldVariant> DestFields = Destination.GetFields(CallingContext);
	if (DestFields.Num() > 0)
	{
		if (!DestFields.Last().IsEmpty())
		{
			DestinationProperty = DestFields.Last().IsProperty() ? DestFields.Last().GetProperty() : UE::MVVM::BindingHelper::GetFirstArgumentProperty(DestFields.Last().GetFunction());
		}
	}

	return IsValidConversionNode(WidgetBlueprint, Function, SourceProperty, DestinationProperty);
}

bool UMVVMEditorSubsystem::IsValidConversionNode(const UWidgetBlueprint* WidgetBlueprint, const TSubclassOf<UK2Node> Function, const FProperty* ExpectedArgumentType, const FProperty* ExptectedReturnType) const
{
	if (WidgetBlueprint == nullptr || Function.Get() == nullptr)
	{
		return false;
	}

	if (!UMVVMBlueprintViewConversionFunction::IsValidConversionNode(WidgetBlueprint, Function))
	{
		return false;
	}

	UClass* CallingContext = UE::MVVM::Private::GetCallingContext(WidgetBlueprint);

	// check that at least one source -> argument binding is compatible
	if (ExpectedArgumentType)
	{
		FEdGraphPinType SourcePinType;
		if (!GetDefault<UEdGraphSchema_K2>()->ConvertPropertyToPinType(ExpectedArgumentType, SourcePinType))
		{
			return false;
		}

		TArray<UEdGraphPin*> InputPins = UE::MVVM::ConversionFunctionHelper::FindInputPins(Function.GetDefaultObject());
		if (InputPins.Num() == 0)
		{
			return false;
		}

		bool bAnyCompatible = false;
		for (const UEdGraphPin* ArgumentPin : InputPins)
		{
			bool bIgnoreArray = true;
			const bool bTypesMatch = GetDefault<UEdGraphSchema_K2>()->ArePinTypesCompatible(SourcePinType, ArgumentPin->PinType, CallingContext, bIgnoreArray);
			if (bTypesMatch)
			{
				bAnyCompatible = true;
				break;
			}
		}
		if (!bAnyCompatible)
		{
			return false;
		}
	}

	if (ExptectedReturnType)
	{
		FEdGraphPinType DestinationPinType;
		if (!GetDefault<UEdGraphSchema_K2>()->ConvertPropertyToPinType(ExptectedReturnType, DestinationPinType))
		{
			return false;
		}

		UEdGraphPin* ReturnPin = UE::MVVM::ConversionFunctionHelper::FindOutputPin(Function.GetDefaultObject());
		if (ReturnPin == nullptr)
		{
			return false;
		}

		// check that the return -> dest is valid
		bool bIgnoreArray = true;
		const bool bTypesMatch = GetDefault<UEdGraphSchema_K2>()->ArePinTypesCompatible(ReturnPin->PinType, DestinationPinType, CallingContext, bIgnoreArray);
		if (!bTypesMatch)
		{
			return false;
		}
	}

	return true;
}

bool UMVVMEditorSubsystem::IsValidConversionFunction(const UWidgetBlueprint* WidgetBlueprint, UE::MVVM::FConversionFunctionValue Function, const FProperty* ExpectedArgumentType, const FProperty* ExptectedReturnType) const
{
	if (Function.IsFunction())
	{
		return IsValidConversionFunction(WidgetBlueprint, Function.GetFunction(), ExpectedArgumentType, ExptectedReturnType);
	}
	else if (Function.IsNode())
	{
		return IsValidConversionNode(WidgetBlueprint, Function.GetNode(), ExpectedArgumentType, ExptectedReturnType);
	}
	return false;
}

bool UMVVMEditorSubsystem::IsSimpleConversionFunction(const UFunction* Function) const
{
	TValueOrError<const FProperty*, FText> ReturnResult = UE::MVVM::BindingHelper::TryGetReturnTypeForConversionFunction(Function);
	if (ReturnResult.HasError())
	{
		return false;
	}

	const FProperty* ReturnProperty = ReturnResult.GetValue();
	if (ReturnProperty == nullptr)
	{
		return false;
	}

	TValueOrError<TArray<const FProperty*>, FText> ArgumentsResult = UE::MVVM::BindingHelper::TryGetArgumentsForConversionFunction(Function);
	if (ArgumentsResult.HasError())
	{
		return false;
	}

	return ArgumentsResult.GetValue().Num() == 1;
}

UEdGraph* UMVVMEditorSubsystem::GetConversionFunctionGraph(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, bool bSourceToDestination) const
{
	if (const UMVVMBlueprintViewConversionFunction* Found = Binding.Conversion.GetConversionFunction(bSourceToDestination))
	{
		return Found->GetWrapperGraph();
	}
	return nullptr;
}

UK2Node_CallFunction* UMVVMEditorSubsystem::GetConversionFunctionNode(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, bool bSourceToDestination) const
{
	return nullptr;
}

TArray<UFunction*> UMVVMEditorSubsystem::GetAvailableConversionFunctions(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& Source, const FMVVMBlueprintPropertyPath& Destination) const
{
	if (WidgetBlueprint == nullptr)
	{
		return TArray<UFunction*>();
	}

	UClass* CallingContext = UE::MVVM::Private::GetCallingContext(WidgetBlueprint);

	const FProperty* SourceProperty = nullptr;
	TArray<UE::MVVM::FMVVMConstFieldVariant> SourceFields = Source.GetFields(CallingContext);
	if (SourceFields.Num() > 0)
	{
		SourceProperty = SourceFields.Last().IsProperty() ? SourceFields.Last().GetProperty() : UE::MVVM::BindingHelper::GetReturnProperty(SourceFields.Last().GetFunction());
	}

	const FProperty* DestinationProperty = nullptr;
	TArray<UE::MVVM::FMVVMConstFieldVariant> DestFields = Destination.GetFields(CallingContext);
	if (DestFields.Num() > 0)
	{
		if (!DestFields.Last().IsEmpty())
		{
			DestinationProperty = DestFields.Last().IsProperty() ? DestFields.Last().GetProperty() : UE::MVVM::BindingHelper::GetFirstArgumentProperty(DestFields.Last().GetFunction());
		}
	}

	TArray<UE::MVVM::FConversionFunctionValue> NewResult = GetConversionFunctions(WidgetBlueprint, SourceProperty, DestinationProperty);
	TArray<UFunction*> ConversionFunctions;
	ConversionFunctions.Reserve(NewResult.Num());

	Algo::TransformIf(NewResult, ConversionFunctions
		, [](const UE::MVVM::FConversionFunctionValue& Other)->bool{ return Other.IsFunction(); }
		, [](const UE::MVVM::FConversionFunctionValue& Other)->UFunction*{ return const_cast<UFunction*>(Other.GetFunction()); });

	ConversionFunctions.Sort([](const UFunction& A, const UFunction& B) -> bool
		{
			return A.GetFName().LexicalLess(B.GetFName());
		});
	return ConversionFunctions;
}

TArray<UE::MVVM::FConversionFunctionValue> UMVVMEditorSubsystem::GetConversionFunctions(const UWidgetBlueprint* WidgetBlueprint, const FProperty* ExpectedArgumentType, const FProperty* ExptectedReturnType) const
{
	EMVVMDeveloperConversionFunctionFilterType FilterType = GetDefault<UMVVMDeveloperProjectSettings>()->GetConversionFunctionFilter();
	if (FilterType == EMVVMDeveloperConversionFunctionFilterType::BlueprintActionRegistry)
	{
		TArray<UE::MVVM::FConversionFunctionValue> ConversionFunctions;

		auto AddFunction = [this, &ExptectedReturnType, &ExpectedArgumentType, WidgetBlueprint, &ConversionFunctions](const UFunction* Function)
		{
			if (IsValidConversionFunction(WidgetBlueprint, UE::MVVM::FConversionFunctionValue(Function), ExpectedArgumentType, ExptectedReturnType))
			{
				ConversionFunctions.Add(UE::MVVM::FConversionFunctionValue(Function));
			}
		};

		const FBlueprintActionDatabase::FActionRegistry& ActionRegistry = FBlueprintActionDatabase::Get().GetAllActions();
		for (auto It = ActionRegistry.CreateConstIterator(); It; ++It)
		{
			if (UObject* ActionObject = It->Key.ResolveObjectPtr())
			{
				for (const UBlueprintNodeSpawner* NodeSpawner : It->Value)
				{
					FBlueprintActionInfo BlueprintAction(ActionObject, NodeSpawner);
					const UFunction* Function = BlueprintAction.GetAssociatedFunction();
					if (Function != nullptr)
					{
						AddFunction(Function);
					}
				}
			}
		}
		return ConversionFunctions;
	}
	else if (FilterType == EMVVMDeveloperConversionFunctionFilterType::AllowedList)
	{
		if (!ConversionFunctionCollection.IsValid())
		{
			ConversionFunctionCollection = MakeUnique<UE::MVVM::ConversionFunctionLibrary::FCollection>();
		}
		return ConversionFunctionCollection->GetFunctions(WidgetBlueprint, ExpectedArgumentType, ExptectedReturnType);
	}

	return TArray<UE::MVVM::FConversionFunctionValue>();
}

TArray<FMVVMBlueprintPinId> UMVVMEditorSubsystem::GetConversionFunctionPins(const UWidgetBlueprint* WidgetBlueprint, const UE::MVVM::FConversionFunctionValue& Function, const FProperty* ExpectedArgumentType) const
{
	TArray<FMVVMBlueprintPinId> OutPins;
	if (Function.IsFunction())
	{
		TValueOrError<TArray<const FProperty*>, FText> ArgumentsResult = UE::MVVM::BindingHelper::TryGetArgumentsForConversionFunction(Function.GetFunction());
		if (ArgumentsResult.HasError())
		{
			return OutPins;
		}
		for (const FProperty* ArgumentProperty : ArgumentsResult.GetValue())
		{
			if (UE::MVVM::BindingHelper::ArePropertiesCompatible(ExpectedArgumentType, ArgumentProperty))
			{
				TArray<FName> Names = { ArgumentProperty->GetFName() };
				OutPins.Add(FMVVMBlueprintPinId(MoveTemp(Names)));
			}
		}

		return OutPins;
	}
	else if (Function.IsNode())
	{
		FEdGraphPinType SourcePinType;
		if (!GetDefault<UEdGraphSchema_K2>()->ConvertPropertyToPinType(ExpectedArgumentType, SourcePinType))
		{
			return OutPins;
		}

		UEdGraph* TempGraph = NewObject<UEdGraph>(GetTransientPackage(), UEdGraph::StaticClass());
		TempGraph->Schema = UEdGraphSchema_K2::StaticClass();

		// Setup input pins
		UK2Node* Node = NewObject<UK2Node>(TempGraph, Function.GetNode());
		Node->CreateNewGuid();
		Node->PostPlacedNewNode();
		Node->AllocateDefaultPins();

		TArray<UEdGraphPin*> InputPins = UE::MVVM::ConversionFunctionHelper::FindInputPins(Node);
		if (InputPins.Num() == 0)
		{
			return OutPins;
		}

		UClass* CallingContext = UE::MVVM::Private::GetCallingContext(WidgetBlueprint);

		bool bAnyCompatible = false;
		for (const UEdGraphPin* ArgumentPin : InputPins)
		{
			bool bIgnoreArray = true;
			if (GetDefault<UEdGraphSchema_K2>()->ArePinTypesCompatible(SourcePinType, ArgumentPin->PinType, CallingContext, bIgnoreArray))
			{
				TArray<FName> Names = { ArgumentPin->GetFName() };
				OutPins.Add(FMVVMBlueprintPinId(Names));
			}
		}
	}
	return OutPins;
}

FMVVMBlueprintPropertyPath UMVVMEditorSubsystem::GetPathForConversionFunctionArgument(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding,  const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination) const
{
	UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(bSourceToDestination);
	if (ConversionFunction == nullptr)
	{
		return FMVVMBlueprintPropertyPath();
	}

	UEdGraphPin* GraphPin = ConversionFunction->GetOrCreateGraphPin(const_cast<UWidgetBlueprint*>(WidgetBlueprint), ParameterId);
	if (GraphPin == nullptr)
	{
		return FMVVMBlueprintPropertyPath();
	}

	return UE::MVVM::ConversionFunctionHelper::GetPropertyPathForPin(WidgetBlueprint, GraphPin, false);
}

void UMVVMEditorSubsystem::SetPathForConversionFunctionArgument(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding,  const FMVVMBlueprintPinId& ParameterId, const FMVVMBlueprintPropertyPath& Path, bool bSourceToDestination) const
{
	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		const FText ParameterName = (!ParameterId.GetNames().IsEmpty()) ? FText::FromName(ParameterId.GetNames().Last()) : LOCTEXT("InvalidPin", "<Invalid Pin>");
		const FScopedTransaction Transaction(FText::Format(LOCTEXT("SetConversionFunctionParameter", "Changed conversion function parameter {0}"), ParameterName));

		FName NAME_Conversion = GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, Conversion);
		UE::MVVM::Private::OnBindingPreEditChange(View, NAME_Conversion);

		UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(bSourceToDestination);
		if (ConversionFunction)
		{
			ConversionFunction->SetGraphPin(WidgetBlueprint, ParameterId, Path);
		}
		FBlueprintEditorUtils::MarkBlueprintAsModified(WidgetBlueprint);

		UE::MVVM::Private::OnBindingPostEditChange(View, NAME_Conversion);
	}
}

namespace Private
{
	UEdGraphPin* GetGraphPin(const UMVVMEditorSubsystem* Subsystem, const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& ViewBinding, const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination)
	{
		if (WidgetBlueprint == nullptr)
		{
			return nullptr;
		}
		return Subsystem->GetConversionFunctionArgumentPin(WidgetBlueprint, ViewBinding, ParameterId, bSourceToDestination);
	}

	UEdGraphPin* GetGraphPin(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* ViewEvent,  const FMVVMBlueprintPinId& ParameterId)
	{
		if (WidgetBlueprint == nullptr || ViewEvent == nullptr)
		{
			return nullptr;
		}

		return ViewEvent->GetOrCreateGraphPin(ParameterId);
	}

	void DoAction(const UMVVMEditorSubsystem* Subsystem, UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* ViewEvent, FMVVMBlueprintViewBinding* Binding,  const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination
		, FText TransactionName
		, TFunctionRef<bool(const UEdGraphSchema_K2*, UEdGraphPin*)> Test, TFunctionRef<void(const UEdGraphSchema_K2*, UEdGraphPin*)> Action)
	{
		UEdGraphPin* GraphPin;
		if (ViewEvent)
		{
			GraphPin = Private::GetGraphPin(WidgetBlueprint, ViewEvent, ParameterId);
		}
		else if (Binding)
		{
			GraphPin = Private::GetGraphPin(Subsystem, WidgetBlueprint, *Binding, ParameterId, bSourceToDestination);
		}
		else
		{
			return;
		}

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		if (!Test(K2Schema, GraphPin))
		{
			return;
		}

		FScopedTransaction Transaction(TransactionName);
		if (ViewEvent)
		{
			FName NAME_SavedPin = "SavedPins";
			UE::MVVM::Private::OnEventPreEditChange(ViewEvent, NAME_SavedPin);
			Action(K2Schema, GraphPin);
			ViewEvent->SavePinValues();
			UE::MVVM::Private::OnEventPostEditChange(ViewEvent, NAME_SavedPin);
		}
		else if (Binding)
		{
			FName NAME_Conversion = GET_MEMBER_NAME_CHECKED(FMVVMBlueprintViewBinding, Conversion);
			UMVVMBlueprintView* View = Subsystem->GetView(WidgetBlueprint);
			UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding->Conversion.GetConversionFunction(bSourceToDestination);
			check(View);
			check(ConversionFunction);

			UE::MVVM::Private::OnBindingPreEditChange(View, NAME_Conversion);
			Action(K2Schema, GraphPin);
			ConversionFunction->SavePinValues(WidgetBlueprint);
			UE::MVVM::Private::OnBindingPostEditChange(View, NAME_Conversion);
		}
	}

	void SplitPin(const UMVVMEditorSubsystem* Subsystem, UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* ViewEvent, FMVVMBlueprintViewBinding* Binding,  const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination)
	{
		DoAction(Subsystem, WidgetBlueprint, ViewEvent, Binding, ParameterId, bSourceToDestination, LOCTEXT("BreakPin", "Split Struct Pin")
			, [](const UEdGraphSchema_K2* K2Schema, UEdGraphPin* GraphPin)->bool
			{
				return GraphPin != nullptr && K2Schema->CanSplitStructPin(*GraphPin);
			}
			, [](const UEdGraphSchema_K2* K2Schema, UEdGraphPin* GraphPin)
			{
				K2Schema->SplitPin(GraphPin);
			});
	}

	bool CanSplitPin(UEdGraphPin* GraphPin)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		return GraphPin ? K2Schema->CanSplitStructPin(*GraphPin) && !GraphPin->bOrphanedPin : false;
	}
	
	void RecombinePin(const UMVVMEditorSubsystem* Subsystem, UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* ViewEvent, FMVVMBlueprintViewBinding* Binding,  const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination)
	{
		DoAction(Subsystem, WidgetBlueprint, ViewEvent, Binding, ParameterId, bSourceToDestination, LOCTEXT("BreakPin", "Split Struct Pin")
			, [](const UEdGraphSchema_K2* K2Schema, UEdGraphPin* GraphPin)->bool
			{
				return GraphPin != nullptr && K2Schema->CanRecombineStructPin(*GraphPin);
			}
			, [](const UEdGraphSchema_K2* K2Schema, UEdGraphPin* GraphPin)
			{
				K2Schema->RecombinePin(GraphPin);
			});
	}

	bool CanRecombinePin(UEdGraphPin* GraphPin)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		return GraphPin ? K2Schema->CanRecombineStructPin(*GraphPin) && !GraphPin->bOrphanedPin : false;
	}
	
	void ResetPinToDefaultValue(const UMVVMEditorSubsystem* Subsystem, UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* ViewEvent, FMVVMBlueprintViewBinding* Binding,  const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination)
	{
		DoAction(Subsystem, WidgetBlueprint, ViewEvent, Binding, ParameterId, bSourceToDestination, LOCTEXT("BreakPin", "Split Struct Pin")
			, [](const UEdGraphSchema_K2* K2Schema, UEdGraphPin* GraphPin)->bool
			{
				return GraphPin != nullptr && !K2Schema->DoesDefaultValueMatchAutogenerated(*GraphPin);
			}
			, [](const UEdGraphSchema_K2* K2Schema, UEdGraphPin* GraphPin)
			{
				K2Schema->ResetPinToAutogeneratedDefaultValue(GraphPin);
			});
	}

	bool CanResetPinToDefaultValue(UEdGraphPin* GraphPin)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		return GraphPin ? !K2Schema->DoesDefaultValueMatchAutogenerated(*GraphPin) && !GraphPin->bOrphanedPin : false;
	}

	void ResetOrphanedPin(const UMVVMEditorSubsystem* Subsystem, UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* ViewEvent, FMVVMBlueprintViewBinding* Binding,  const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination)
	{
		DoAction(Subsystem, WidgetBlueprint, ViewEvent, Binding, ParameterId, bSourceToDestination, LOCTEXT("BreakPin", "Split Struct Pin")
			, [](const UEdGraphSchema_K2* K2Schema, UEdGraphPin* GraphPin)->bool
			{
				return GraphPin == nullptr || GraphPin->bOrphanedPin;
			}
			, [](const UEdGraphSchema_K2* K2Schema, UEdGraphPin* GraphPin)
				{
					if (GraphPin)
					{
						K2Schema->ResetPinToAutogeneratedDefaultValue(GraphPin);
					}
				});
	}
	
	bool CanResetOrphanedPin(UEdGraphPin* GraphPin)
	{
		return GraphPin ? GraphPin->bOrphanedPin : true;
	}
} //namespace

void UMVVMEditorSubsystem::SplitPin(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination) const
{
	Private::SplitPin(this, WidgetBlueprint, nullptr, &Binding, ParameterId, bSourceToDestination);
}

bool UMVVMEditorSubsystem::CanSplitPin(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination) const
{
	UEdGraphPin* GraphPin = Private::GetGraphPin(this, WidgetBlueprint, Binding, ParameterId, bSourceToDestination);
	return Private::CanSplitPin(GraphPin);
}

void UMVVMEditorSubsystem::SplitPin(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* ViewEvent, const FMVVMBlueprintPinId& ParameterId) const
{
	Private::SplitPin(this, WidgetBlueprint, ViewEvent, nullptr, ParameterId, true);
}

bool UMVVMEditorSubsystem::CanSplitPin(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& ParameterId) const
{
	UEdGraphPin* GraphPin = Private::GetGraphPin(WidgetBlueprint, Event, ParameterId);
	return Private::CanSplitPin(GraphPin);
}

void UMVVMEditorSubsystem::RecombinePin(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination) const
{
	Private::RecombinePin(this, WidgetBlueprint, nullptr, &Binding, ParameterId, bSourceToDestination);
}

bool UMVVMEditorSubsystem::CanRecombinePin(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination) const
{
	UEdGraphPin* GraphPin = Private::GetGraphPin(this, WidgetBlueprint, Binding, ParameterId, bSourceToDestination);
	return Private::CanRecombinePin(GraphPin);
}

void UMVVMEditorSubsystem::RecombinePin(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* ViewEvent, const FMVVMBlueprintPinId& ParameterId) const
{
	Private::RecombinePin(this, WidgetBlueprint, ViewEvent, nullptr, ParameterId, true);
}

bool UMVVMEditorSubsystem::CanRecombinePin(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& ParameterId) const
{
	UEdGraphPin* GraphPin = Private::GetGraphPin(WidgetBlueprint, Event, ParameterId);
	return Private::CanRecombinePin(GraphPin);
}

void UMVVMEditorSubsystem::ResetPinToDefaultValue(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination) const
{
	Private::ResetPinToDefaultValue(this, WidgetBlueprint, nullptr, &Binding, ParameterId, bSourceToDestination);
}

bool UMVVMEditorSubsystem::CanResetPinToDefaultValue(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination) const
{
	UEdGraphPin* GraphPin = Private::GetGraphPin(this, WidgetBlueprint, Binding, ParameterId, bSourceToDestination);
	return Private::CanResetPinToDefaultValue(GraphPin);
}

void UMVVMEditorSubsystem::ResetPinToDefaultValue(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* ViewEvent, const FMVVMBlueprintPinId& ParameterId) const
{
	Private::ResetPinToDefaultValue(this, WidgetBlueprint, ViewEvent, nullptr, ParameterId, true);
}

bool UMVVMEditorSubsystem::CanResetPinToDefaultValue(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& ParameterId) const
{
	UEdGraphPin* GraphPin = Private::GetGraphPin(WidgetBlueprint, Event, ParameterId);
	return Private::CanResetPinToDefaultValue(GraphPin);
}

void UMVVMEditorSubsystem::ResetOrphanedPin(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination) const
{
	Private::ResetOrphanedPin(this, WidgetBlueprint, nullptr, &Binding, ParameterId, bSourceToDestination);
}

bool UMVVMEditorSubsystem::CanResetOrphanedPin(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& ParameterId, bool bSourceToDestination) const
{
	UEdGraphPin* GraphPin = Private::GetGraphPin(this, WidgetBlueprint, Binding, ParameterId, bSourceToDestination);
	return Private::CanResetOrphanedPin(GraphPin);
}

void UMVVMEditorSubsystem::ResetOrphanedPin(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* ViewEvent, const FMVVMBlueprintPinId& ParameterId) const
{
	Private::ResetOrphanedPin(this, WidgetBlueprint, ViewEvent, nullptr, ParameterId, true);
}

bool UMVVMEditorSubsystem::CanResetOrphanedPin(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& ParameterId) const
{
	UEdGraphPin* GraphPin = Private::GetGraphPin(WidgetBlueprint, Event, ParameterId);
	return Private::CanResetOrphanedPin(GraphPin);
}

TArray<UE::MVVM::FBindingSource> UMVVMEditorSubsystem::GetBindableWidgets(const UWidgetBlueprint* WidgetBlueprint) const
{
	TArray<UE::MVVM::FBindingSource> Sources;

	const UWidgetTree* WidgetTree = WidgetBlueprint->WidgetTree;
	if (WidgetTree == nullptr)
	{
		return Sources;
	}

	TArray<UWidget*> AllWidgets;
	WidgetTree->GetAllWidgets(AllWidgets);

	Sources.Reserve(AllWidgets.Num() + 1);

	// Add current widget as a possible binding source
	if (UClass* BPClass = WidgetBlueprint->GeneratedClass)
	{
		TArray<FMVVMAvailableBinding> Bindings = UMVVMSubsystem::GetAvailableBindings(BPClass, WidgetBlueprint->GeneratedClass);
		if (Bindings.Num() > 0)
		{
			// at least one valid property, add it to our list
			Sources.Add(UE::MVVM::FBindingSource::CreateForBlueprint(WidgetBlueprint));
		}
	}

	for (const UWidget* Widget : AllWidgets)
	{
		TArray<FMVVMAvailableBinding> Bindings = UMVVMSubsystem::GetAvailableBindings(Widget->GetClass(), WidgetBlueprint->GeneratedClass);
		if (Bindings.Num() > 0)
		{
			// at least one valid property, add it to our list
			Sources.Add(UE::MVVM::FBindingSource::CreateForWidget(WidgetBlueprint, Widget));
		}
	}

	return Sources;
}

TArray<UE::MVVM::FBindingSource> UMVVMEditorSubsystem::GetAllViewModels(const UWidgetBlueprint* WidgetBlueprint) const
{
	TArray<UE::MVVM::FBindingSource> Sources;

	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		const TArrayView<const FMVVMBlueprintViewModelContext> ViewModels = View->GetViewModels();
		Sources.Reserve(ViewModels.Num());

		for (const FMVVMBlueprintViewModelContext& ViewModelContext : ViewModels)
		{
			Sources.Add(UE::MVVM::FBindingSource::CreateForViewModel(WidgetBlueprint, ViewModelContext));
		}
	}

	return Sources;
}

TArray<FMVVMAvailableBinding> UMVVMEditorSubsystem::GetChildViewModels(TSubclassOf<UObject> Class, TSubclassOf<UObject> Accessor)
{
	if (Class.Get() == nullptr)
	{
		return TArray<FMVVMAvailableBinding>();
	}

	TArray<FMVVMAvailableBinding> ViewModelAvailableBindingsList = UMVVMSubsystem::GetAvailableBindings(Class, Accessor);
	ViewModelAvailableBindingsList.RemoveAllSwap([Class](const FMVVMAvailableBinding& Value)
		{
			UE::MVVM::FMVVMFieldVariant Variant = UE::MVVM::BindingHelper::FindFieldByName(Class.Get(), Value.GetBindingName());
			const FProperty* Property = nullptr;
			if (Variant.IsProperty())
			{
				Property = Variant.GetProperty();
			}
			else if (Variant.IsFunction() && Variant.GetFunction())
			{
				Property = UE::MVVM::BindingHelper::GetReturnProperty(Variant.GetFunction());
			}

			if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
			{
				return !ObjectProperty->PropertyClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass());
			}
			return true;
		});

	return ViewModelAvailableBindingsList;
}

FGuid UMVVMEditorSubsystem::GetFirstBindingThatUsesViewModel(const UWidgetBlueprint* WidgetBlueprint, FGuid ViewModelId) const
{
	if (UMVVMBlueprintView* View = GetView(WidgetBlueprint))
	{
		for (const FMVVMBlueprintViewBinding& Binding : View->GetBindings())
		{
			if (Binding.SourcePath.GetViewModelId() == ViewModelId)
			{
				return Binding.BindingId;
			}
			if (Binding.DestinationPath.GetViewModelId() == ViewModelId)
			{
				return Binding.BindingId;
			}

			auto TestConversionFunction = [&](bool bForward)
			{
				UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(true);
				if (ConversionFunction)
				{
					for (const FMVVMBlueprintPin& Pin : ConversionFunction->GetPins())
					{
						if (Pin.UsedPathAsValue() && Pin.GetPath().GetSource(WidgetBlueprint) == EMVVMBlueprintFieldPathSource::ViewModel)
						{
							if (Pin.GetPath().GetViewModelId() == ViewModelId)
							{
								return Binding.BindingId;
							}
						}
					}
				}
				return FGuid();
			};

			if (UE::MVVM::IsForwardBinding(Binding.BindingType))
			{
				FGuid Forward = TestConversionFunction(true);
				if (Forward.IsValid())
				{
					return Forward;
				}
			}

			if (UE::MVVM::IsBackwardBinding(Binding.BindingType))
			{
				FGuid Backward = TestConversionFunction(false);
				if (Backward.IsValid())
				{
					return Backward;
				}
			}
		}
	}
	return FGuid();
}

#undef LOCTEXT_NAMESPACE

