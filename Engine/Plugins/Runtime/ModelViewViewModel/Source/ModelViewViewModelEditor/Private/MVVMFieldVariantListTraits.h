// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Views/TableViewTypeTraits.h"
#include "MVVMPropertyPath.h"

template <>
struct TIsValidListItem<UE::MVVM::FMVVMFieldVariant>
{
	enum
	{
		Value = true
	};
};

template <>
struct TListTypeTraits<UE::MVVM::FMVVMFieldVariant>
{
	typedef UE::MVVM::FMVVMFieldVariant NullableType;

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<UE::MVVM::FMVVMFieldVariant, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<UE::MVVM::FMVVMFieldVariant, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<UE::MVVM::FMVVMFieldVariant>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector&,
		TArray<UE::MVVM::FMVVMFieldVariant>&,
		TSet<UE::MVVM::FMVVMFieldVariant>&,
		TMap<const U*, UE::MVVM::FMVVMFieldVariant>&)
	{
	}

	static bool IsPtrValid(const UE::MVVM::FMVVMFieldVariant& InPtr)
	{
		return !InPtr.IsEmpty();
	}

	static void ResetPtr(UE::MVVM::FMVVMFieldVariant& InPtr)
	{
		InPtr.Reset();
	}

	static UE::MVVM::FMVVMFieldVariant MakeNullPtr()
	{
		return UE::MVVM::FMVVMFieldVariant();
	}

	static UE::MVVM::FMVVMFieldVariant NullableItemTypeConvertToItemType(const UE::MVVM::FMVVMFieldVariant& InPtr)
	{
		return InPtr;
	}

	static FString DebugDump(UE::MVVM::FMVVMFieldVariant InPtr)
	{
		return InPtr.GetName().ToString();
	}

	class SerializerType {};
};

template <>
struct TIsValidListItem<UE::MVVM::FMVVMConstFieldVariant>
{
	enum
	{
		Value = true
	};
};

template <>
struct TListTypeTraits<UE::MVVM::FMVVMConstFieldVariant>
{
	typedef UE::MVVM::FMVVMConstFieldVariant NullableType;

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<UE::MVVM::FMVVMConstFieldVariant, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<UE::MVVM::FMVVMConstFieldVariant, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<UE::MVVM::FMVVMConstFieldVariant>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector&,
		TArray<UE::MVVM::FMVVMConstFieldVariant>&,
		TSet<UE::MVVM::FMVVMConstFieldVariant>&,
		TMap<const U*, UE::MVVM::FMVVMConstFieldVariant>&)
	{
	}

	static bool IsPtrValid(const UE::MVVM::FMVVMConstFieldVariant& InPtr)
	{
		return !InPtr.IsEmpty();
	}

	static void ResetPtr(UE::MVVM::FMVVMConstFieldVariant& InPtr)
	{
		InPtr.Reset();
	}

	static UE::MVVM::FMVVMConstFieldVariant MakeNullPtr()
	{
		return UE::MVVM::FMVVMConstFieldVariant();
	}

	static UE::MVVM::FMVVMConstFieldVariant NullableItemTypeConvertToItemType(const UE::MVVM::FMVVMConstFieldVariant& InPtr)
	{
		return InPtr;
	}

	static FString DebugDump(UE::MVVM::FMVVMConstFieldVariant InPtr)
	{
		return InPtr.GetName().ToString();
	}

	class SerializerType {};
};

template <>
struct TIsValidListItem<FMVVMBlueprintPropertyPath>
{
	enum
	{
		Value = true
	};
};

template <>
struct TListTypeTraits<FMVVMBlueprintPropertyPath>
{
	typedef FMVVMBlueprintPropertyPath NullableType;

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<FMVVMBlueprintPropertyPath, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<FMVVMBlueprintPropertyPath, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<FMVVMBlueprintPropertyPath>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector&,
		TArray<FMVVMBlueprintPropertyPath>&,
		TSet<FMVVMBlueprintPropertyPath>&,
		TMap<const U*, FMVVMBlueprintPropertyPath>&)
	{
	}

	static bool IsPtrValid(const FMVVMBlueprintPropertyPath& InPtr)
	{
		return InPtr.IsValid();
	}

	static void ResetPtr(FMVVMBlueprintPropertyPath& InPtr)
	{
		InPtr = FMVVMBlueprintPropertyPath();
	}

	static FMVVMBlueprintPropertyPath MakeNullPtr()
	{
		return FMVVMBlueprintPropertyPath();
	}

	static FMVVMBlueprintPropertyPath NullableItemTypeConvertToItemType(const FMVVMBlueprintPropertyPath& InPtr)
	{
		return InPtr;
	}

	static FString DebugDump(FMVVMBlueprintPropertyPath InPtr)
	{
		return FString();
	}

	class SerializerType {};
};
