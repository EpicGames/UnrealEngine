// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MVVMViewBindingMenuContext.generated.h"

class FWidgetBlueprintEditor;
namespace UE::MVVM { class SBindingsPanel; }

UCLASS()
	class UMVVMViewBindingMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor;
	TWeakPtr<UE::MVVM::SBindingsPanel> BindingsPanel;
};
