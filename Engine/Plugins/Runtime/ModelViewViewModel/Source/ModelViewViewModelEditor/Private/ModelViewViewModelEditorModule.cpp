// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelViewViewModelEditorModule.h"

#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "BlueprintModes/WidgetBlueprintApplicationMode.h"
#include "BlueprintModes/WidgetBlueprintApplicationModes.h"
#include "Customizations/MVVMBlueprintViewDesignerExtension.h"
#include "Customizations/MVVMBlueprintViewModelContextCustomization.h"
#include "Customizations/MVVMClipboardExtension.h"
#include "Customizations/MVVMDragDropExtension.h"
#include "Customizations/MVVMListViewBaseExtensionCustomizationExtender.h"
#include "Customizations/MVVMPanelWidgetExtensionCustomizationExtender.h"
#include "Customizations/MVVMPropertyBindingExtension.h"
#include "Customizations/MVVMWidgetBlueprintDiffProvider.h"
#include "Customizations/MVVMWidgetContextMenuExtension.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MessageLogModule.h"
#include "MVVMBlueprintView.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMEditorCommands.h"
#include "MVVMEditorSubsystem.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "PropertyEditorModule.h"
#include "Styling/MVVMEditorStyle.h"
#include "Tabs/MVVMBindingSummoner.h"
#include "Tabs/MVVMViewModelSummoner.h"
#include "ToolMenus.h"
#include "UMGEditorModule.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "WidgetBlueprintEditor.h"
#include "WidgetDrawerConfig.h"
#include "Widgets/SMVVMPopoutEditor.h"
#include "Widgets/SMVVMViewBindingPanel.h"
#include "Widgets/SMVVMViewModelPanel.h"

#define LOCTEXT_NAMESPACE "ModelViewViewModelModule"

/** Rather than expose and depend on private include which won't have _API, we just duplicate anim tab string here */
const FName AnimationTabSummonerTabID(TEXT("Animations"));

void FModelViewViewModelEditorModule::StartupModule()
{
	FMVVMEditorStyle::CreateInstance();

	IUMGEditorModule& UMGEditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
	UMGEditorModule.OnRegisterTabsForEditor().AddRaw(this, &FModelViewViewModelEditorModule::HandleRegisterBlueprintEditorTab);

	PropertyBindingExtension = MakeShared<UE::MVVM::FMVVMPropertyBindingExtension>();
	UMGEditorModule.GetPropertyBindingExtensibilityManager()->AddExtension(PropertyBindingExtension.ToSharedRef());

	ClipboardExtension = MakeShared<UE::MVVM::FClipboardExtension>();
	UMGEditorModule.GetClipboardExtensibilityManager()->AddExtension(ClipboardExtension.ToSharedRef());

	DragDropExtension = MakeShared<UE::MVVM::FWidgetDragDropExtension>();
	UMGEditorModule.GetWidgetDragDropExtensibilityManager()->AddExtension(DragDropExtension.ToSharedRef());

	WidgetContextMenuCustomization = MakeShared<UE::MVVM::FWidgetContextMenuExtension>();
	UMGEditorModule.GetWidgetContextMenuExtensibilityManager()->AddExtension(WidgetContextMenuCustomization.ToSharedRef());

	ListViewBaseCustomizationExtender = UE::MVVM::FMVVMListViewBaseExtensionCustomizationExtender::MakeInstance();
	UMGEditorModule.AddWidgetCustomizationExtender(ListViewBaseCustomizationExtender.ToSharedRef());

	PanelWidgetCustomizationExtender = UE::MVVM::FMVVMPanelWidgetExtensionCustomizationExtender::MakeInstance();
	UMGEditorModule.AddWidgetCustomizationExtender(PanelWidgetCustomizationExtender.ToSharedRef());

	BlueprintViewDesignerExtensionFactory = MakeShared<UE::MVVM::FBlueprintViewDesignerExtensionFactory>();
	UMGEditorModule.GetDesignerExtensibilityManager()->AddDesignerExtensionFactory(BlueprintViewDesignerExtensionFactory.ToSharedRef());

	UMGEditorModule.RegisterInstancedCustomPropertyTypeLayout(
		FMVVMBlueprintViewModelContext::StaticStruct()->GetStructPathName()
		, IUMGEditorModule::FOnGetInstancePropertyTypeCustomizationInstance::CreateStatic(UE::MVVM::FBlueprintViewModelContextDetailCustomization::MakeInstance)
		);

	FBlueprintEditorUtils::OnRenameVariableReferencesEvent.AddRaw(this, &FModelViewViewModelEditorModule::HandleRenameFieldReferences);
	FBlueprintEditorUtils::OnRenameFunctionReferencesEvent.AddRaw(this, &FModelViewViewModelEditorModule::HandleRenameFieldReferences);

	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowFilters = true;
		InitOptions.bShowPages = false;
		InitOptions.bAllowClear = true;
		MessageLogModule.RegisterLogListing("Model View Viewmodel", LOCTEXT("MVVMLog", "Model View Viewmodel"), InitOptions);
	}

	FMVVMEditorCommands::Register();
	FWidgetBlueprintDelegates::GetAssetTagsWithContext.AddRaw(this, &FModelViewViewModelEditorModule::HandleWidgetBlueprintAssetTags);
	FWidgetBlueprintGeneratedClassDelegates::GetAssetTagsWithContext.AddRaw(this, &FModelViewViewModelEditorModule::HandleClassBlueprintAssetTags);
	FWidgetBlueprintGeneratedClassDelegates::CollectSaveOverrides.AddRaw(this, &FModelViewViewModelEditorModule::HandleCollectSaveOverrides);

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FModelViewViewModelEditorModule::HandleRegisterMenus));

	UE::MVVM::FMVVMWidgetBlueprintDiff::RegisterCustomDiff(MakeShared<FMVVMWidgetBlueprintDiffProvider>());
}

void FModelViewViewModelEditorModule::ShutdownModule()
{
	UE::MVVM::FMVVMWidgetBlueprintDiff::UnregisterCustomDiff();

	UnregisterMenus();

	FWidgetBlueprintGeneratedClassDelegates::CollectSaveOverrides.RemoveAll(this);
	FWidgetBlueprintGeneratedClassDelegates::GetAssetTagsWithContext.RemoveAll(this);
	FWidgetBlueprintDelegates::GetAssetTagsWithContext.RemoveAll(this);
	if (FMessageLogModule* MessageLogModule = FModuleManager::GetModulePtr<FMessageLogModule>("MessageLog"))
	{
		MessageLogModule->UnregisterLogListing("Model View Viewmodel");
	}

	FBlueprintEditorUtils::OnRenameVariableReferencesEvent.RemoveAll(this);
	FBlueprintEditorUtils::OnRenameFunctionReferencesEvent.RemoveAll(this);

	if (IUMGEditorModule* UMGEditorModule = FModuleManager::GetModulePtr<IUMGEditorModule>("UMGEditor"))
	{
		UMGEditorModule->GetDesignerExtensibilityManager()->RemoveDesignerExtensionFactory(BlueprintViewDesignerExtensionFactory.ToSharedRef());
		UMGEditorModule->OnRegisterTabsForEditor().RemoveAll(this);
		UMGEditorModule->GetWidgetDragDropExtensibilityManager()->RemoveExtension(DragDropExtension.ToSharedRef());
		UMGEditorModule->GetWidgetContextMenuExtensibilityManager()->RemoveExtension(WidgetContextMenuCustomization.ToSharedRef());
		UMGEditorModule->GetClipboardExtensibilityManager()->RemoveExtension(ClipboardExtension.ToSharedRef());
		UMGEditorModule->GetPropertyBindingExtensibilityManager()->RemoveExtension(PropertyBindingExtension.ToSharedRef());

		if (UObjectInitialized())
		{
			UMGEditorModule->UnregisterInstancedCustomPropertyTypeLayout(FMVVMBlueprintViewModelContext::StaticStruct()->GetStructPathName());
		}
	}
	PropertyBindingExtension.Reset();

	FMVVMEditorStyle::DestroyInstance();

	FMVVMEditorCommands::Unregister();
}

void FModelViewViewModelEditorModule::OpenPopoutEditor(TNotNull<UObject*> InObject, bool bInReadyOnly)
{
	UE::MVVM::SMVVMPopoutEditor::OpenPopoutEditor(InObject, bInReadyOnly);
}


void FModelViewViewModelEditorModule::HandleRegisterBlueprintEditorTab(const FWidgetBlueprintApplicationMode& ApplicationMode, FWorkflowAllowedTabSet& TabFactories)
{
	if (ApplicationMode.GetModeName() == FWidgetBlueprintApplicationModes::DesignerMode)
	{
		TabFactories.RegisterFactory(MakeShared<FMVVMBindingSummoner>(ApplicationMode.GetBlueprintEditor()));
		TabFactories.RegisterFactory(MakeShared<UE::MVVM::FViewModelSummoner>(ApplicationMode.GetBlueprintEditor()));

		if (ApplicationMode.LayoutExtender)
		{
			FTabManager::FTab NewTab(FTabId(FMVVMBindingSummoner::TabID, ETabIdFlags::SaveLayout), ETabState::ClosedTab);
			ApplicationMode.LayoutExtender->ExtendLayout(AnimationTabSummonerTabID, ELayoutExtensionPosition::After, NewTab);

			ApplicationMode.OnPostActivateMode.AddRaw(this, &FModelViewViewModelEditorModule::HandleActivateMode);
			ApplicationMode.OnPreDeactivateMode.AddRaw(this, &FModelViewViewModelEditorModule::HandleDeactiveMode);
		}

		if (TSharedPtr<FWidgetBlueprintEditor> BP = ApplicationMode.GetBlueprintEditor())
		{
			if (UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(BP->GetWidgetBlueprintObj()))
			{
				ExtensionView->SetFilterSettings(GetDefault<UMVVMDeveloperProjectSettings>()->FilterSettings);
			}
		}
	}
}


void FModelViewViewModelEditorModule::HandleRenameFieldReferences(UBlueprint* Blueprint, UClass* FieldOwnerClass, const FName& OldVarName, const FName& NewVarName)
{
	if (UWidgetBlueprint* WidgetBlueprint = Cast<UWidgetBlueprint>(Blueprint))
	{
		if (UMVVMWidgetBlueprintExtension_View* ViewExtension = UWidgetBlueprintExtension::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint))
		{
			if (UMVVMBlueprintView* BlueprintView = ViewExtension->GetBlueprintView())
			{
				BlueprintView->OnFieldRenamed(FieldOwnerClass, OldVarName, NewVarName);
			}

			ViewExtension->OnFieldRenamed(FieldOwnerClass, OldVarName, NewVarName);
		}
	}
}

void FModelViewViewModelEditorModule::HandleDeactiveMode(FWidgetBlueprintApplicationMode& InDesignerMode)
{
	TSharedPtr<FWidgetBlueprintEditor> BP = InDesignerMode.GetBlueprintEditor();
	if (BP && BP->IsEditorClosing())
	{
		InDesignerMode.OnPostActivateMode.RemoveAll(this);
		InDesignerMode.OnPreDeactivateMode.RemoveAll(this);
	}
}

void FModelViewViewModelEditorModule::HandleActivateMode(FWidgetBlueprintApplicationMode& InDesignerMode)
{
	if (TSharedPtr<FWidgetBlueprintEditor> BP = InDesignerMode.GetBlueprintEditor())
	{
		if (!BP->GetExternalEditorWidget(FMVVMBindingSummoner::DrawerID))
		{
			bool bIsDrawerTab = true;
			FMVVMBindingSummoner MVVMDrawerSummoner(BP, bIsDrawerTab);
			FWorkflowTabSpawnInfo SpawnInfo;
			BP->AddExternalEditorWidget(FMVVMBindingSummoner::DrawerID, MVVMDrawerSummoner.CreateTabBody(SpawnInfo));
		}

		// Add MVVM Drawer
		{
			FWidgetDrawerConfig MVVMDrawer(FMVVMBindingSummoner::DrawerID);
			TWeakPtr<FWidgetBlueprintEditor> WeakBP = BP;
			MVVMDrawer.GetDrawerContentDelegate.BindLambda([WeakBP]()
			{
				if (TSharedPtr<FWidgetBlueprintEditor> BP = WeakBP.Pin())
				{
					TSharedPtr<SWidget> DrawerWidgetContent = BP->GetExternalEditorWidget(FMVVMBindingSummoner::DrawerID);
					if (DrawerWidgetContent)
					{
						return DrawerWidgetContent.ToSharedRef();
					}
				}

				return SNullWidget::NullWidget;
			});
			MVVMDrawer.OnDrawerOpenedDelegate.BindLambda([WeakBP](FName StatusBarWithDrawerName)
			{
				if (TSharedPtr<FWidgetBlueprintEditor> BP = WeakBP.Pin())
				{
					FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), BP->GetExternalEditorWidget(FMVVMBindingSummoner::DrawerID));
				}
			});
			MVVMDrawer.OnDrawerDismissedDelegate.BindLambda([WeakBP](const TSharedPtr<SWidget>& NewlyFocusedWidget)
			{
				if (TSharedPtr<FWidgetBlueprintEditor> BP = WeakBP.Pin())
				{
					BP->SetKeyboardFocus();
				}
			});
			MVVMDrawer.ButtonText = LOCTEXT("StatusBar_MVVM", "View Bindings");
			MVVMDrawer.ToolTipText = LOCTEXT("StatusBar_MVVMToolTip", "Opens MVVM Bindings (Ctrl+Shift+B).");
			MVVMDrawer.Icon = FMVVMEditorStyle::Get().GetBrush("BlueprintView.TabIcon");
			BP->RegisterDrawer(MoveTemp(MVVMDrawer), 1);
		}

		BP->GetToolkitCommands()->MapAction(FMVVMEditorCommands::Get().ToggleMVVMDrawer,
			FExecuteAction::CreateStatic(&FMVVMBindingSummoner::ToggleMVVMDrawer)
		);
	}
}

void FModelViewViewModelEditorModule::HandleWidgetBlueprintAssetTags(const UWidgetBlueprint* WidgetBlueprint, FAssetRegistryTagsContext Context)
{
	if (WidgetBlueprint && GEditor)
	{
		if (UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>())
		{
			if (UMVVMBlueprintView* BlueprintView = Subsystem->GetView(WidgetBlueprint))
			{
				BlueprintView->AddAssetTags(Context);
			}
		}
	}
}

void FModelViewViewModelEditorModule::HandleClassBlueprintAssetTags(const UWidgetBlueprintGeneratedClass* GeneratedClass, FAssetRegistryTagsContext Context)
{
	if (GeneratedClass && GEditor && GeneratedClass->ClassGeneratedBy)
	{
		if (UWidgetBlueprint* WidgetBlueprint = Cast<UWidgetBlueprint>(GeneratedClass->ClassGeneratedBy))
		{
			HandleWidgetBlueprintAssetTags(WidgetBlueprint, Context);
		}
	}
}

namespace UE::MVVM::Private
{
	bool GAutogeneratedFunctionsAreForceEditorTransient = true;
	static FAutoConsoleVariableRef CVarAutogeneratedFunctionsAreForceEditorTransient(
		TEXT("MVVM.AutogeneratedFunctionsAreForceEditorTransient"),
		GAutogeneratedFunctionsAreForceEditorTransient,
		TEXT("Is the autogenerated function are mark as transient in editor but still cooked."),
		ECVF_ReadOnly
	);
}

void FModelViewViewModelEditorModule::HandleCollectSaveOverrides(const UWidgetBlueprintGeneratedClass* GeneratedClass, FObjectCollectSaveOverridesContext SaveContext)
{
	// The UWidgetBlueprintGeneratedClass::CollectSaveOverrides calls this callback. It give us the opotunity to remove object/properties from a package.
	//Do not save the UFunction auto generated by MVVM. Can be a viewmodel setter, a conversion function, an event or a inner struct setter.
	//The function are still alive and will be generated again on the next editor load, when a BP compiles. They are saved in a cook package.
	if (UE::MVVM::Private::GAutogeneratedFunctionsAreForceEditorTransient && !SaveContext.IsCooking() && GeneratedClass && GeneratedClass->ClassGeneratedBy)
	{
		if (UWidgetBlueprint* WidgetBlueprint = Cast<UWidgetBlueprint>(GeneratedClass->ClassGeneratedBy))
		{
			if (UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint))
			{
				// If we can't auto-generate function add the transient flags
				FObjectSaveOverride ObjectSaveOverride;
				ObjectSaveOverride.bForceTransient = true;
				for (FName FunctionName : ExtensionView->GetGeneratedFunctions())
				{
					UFunction* Function = GeneratedClass->FindFunctionByName(FunctionName);
					if (Function)
					{
						SaveContext.AddSaveOverride(Function, ObjectSaveOverride);
					}
				}
			}
		}
	}
}


void FModelViewViewModelEditorModule::HandleRegisterMenus()
{
	// Allow cleanup when module unloads
	FToolMenuOwnerScoped OwnerScoped(this);

	UE::MVVM::SMVVMViewModelPanel::RegisterMenu();
	UE::MVVM::SBindingsPanel::RegisterSettingsMenu();
}


void FModelViewViewModelEditorModule::UnregisterMenus()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);
}


IMPLEMENT_MODULE(FModelViewViewModelEditorModule, ModelViewViewModelEditor);

#undef LOCTEXT_NAMESPACE
