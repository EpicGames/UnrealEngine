// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IModelViewViewModelEditorModule.h"
#include "UObject/Object.h"

namespace UE::MVVM
{
	class FAssetTypeActions_ViewModelBlueprint;
	class FBlueprintViewDesignerExtensionFactory;
	class FClipboardExtension;
	class FWidgetDragDropExtension;
	class FWidgetContextMenuExtension;
	class FMVVMListViewBaseExtensionCustomizationExtender;
	class FMVVMPanelWidgetExtensionCustomizationExtender;
	class FMVVMPropertyBindingExtension;
}

class FMVVMBindPropertiesDetailView;
class FWidgetBlueprintApplicationMode;
class FWorkflowAllowedTabSet;
class UBlueprint;
class UWidgetBlueprint;
class UWidgetBlueprintGeneratedClass;

/**
 *
 */
class FModelViewViewModelEditorModule : public IModelViewViewModelEditorModule
{
public:
	FModelViewViewModelEditorModule() = default;

	//~ Begin IModuleInterface interface
	 virtual void StartupModule() override;
	 virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	//~ Begin IModelViewViewModelEditorModule
	virtual void OpenPopoutEditor(TNotNull<UObject*> InObject, bool bInReadyOnly) override;
	//~ End IModelViewViewModelEditorModule

private:
	void HandleRegisterBlueprintEditorTab(const FWidgetBlueprintApplicationMode& ApplicationMode, FWorkflowAllowedTabSet& TabFactories);
	void HandleRenameFieldReferences(UBlueprint* Blueprint, UClass* FieldOwnerClass, const FName& OldVarName, const FName& NewVarName);
	void HandleDeactiveMode(FWidgetBlueprintApplicationMode& InDesignerMode);
	void HandleActivateMode(FWidgetBlueprintApplicationMode& InDesignerMode);
	void HandleWidgetBlueprintAssetTags(const UWidgetBlueprint* Widget, FAssetRegistryTagsContext Context);
	void HandleClassBlueprintAssetTags(const UWidgetBlueprintGeneratedClass* GeneratedClass, FAssetRegistryTagsContext Context);
	void HandleCollectSaveOverrides(const UWidgetBlueprintGeneratedClass* GeneratedClass, FObjectCollectSaveOverridesContext SaveContext);
	void HandleRegisterMenus();
	void UnregisterMenus();

private:
	TSharedPtr<UE::MVVM::FMVVMPropertyBindingExtension> PropertyBindingExtension;
	TSharedPtr<UE::MVVM::FClipboardExtension> ClipboardExtension;
	TSharedPtr<UE::MVVM::FWidgetDragDropExtension> DragDropExtension;
	TSharedPtr<UE::MVVM::FWidgetContextMenuExtension> WidgetContextMenuCustomization;
	TSharedPtr<UE::MVVM::FAssetTypeActions_ViewModelBlueprint> ViewModelBlueprintActions;
	TSharedPtr<UE::MVVM::FMVVMListViewBaseExtensionCustomizationExtender> ListViewBaseCustomizationExtender;
	TSharedPtr<UE::MVVM::FMVVMPanelWidgetExtensionCustomizationExtender> PanelWidgetCustomizationExtender;
	TSharedPtr<UE::MVVM::FBlueprintViewDesignerExtensionFactory> BlueprintViewDesignerExtensionFactory;
};
