// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/MVVMEditorStyle.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleMacros.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Interfaces/IPluginManager.h"


TUniquePtr<FMVVMEditorStyle, FMVVMEditorStyle::FCustomDeleter> FMVVMEditorStyle::Instance;


FMVVMEditorStyle::FMVVMEditorStyle()
	: FSlateStyleSet("MVVMEditorStyle")
{
	using CoreStyleConstants::Icon16x16;
	using CoreStyleConstants::Icon64x64;

	TSharedPtr<IPlugin> MVVMPlugin = IPluginManager::Get().FindPlugin("ModelViewViewModel");
	if (ensure(MVVMPlugin))
	{
		SetContentRoot(MVVMPlugin->GetContentDir() / TEXT("Editor"));
		SetCoreContentRoot(FPaths::EngineContentDir());
	}

	// Class Icons
	{
		Set("ClassIcon.MVVMBlueprintView", new IMAGE_BRUSH_SVG("Slate/ViewModel", Icon16x16));
		Set("ClassIcon.MVVMBlueprintViewModel", new IMAGE_BRUSH_SVG("Slate/ViewModel", Icon16x16));
		Set("ClassThumbnail.MVVMViewModelBase", new IMAGE_BRUSH_SVG("Slate/ViewModel", Icon64x64));
	}

	Set("BlueprintView.TabIcon", new IMAGE_BRUSH_SVG("Slate/ViewModel", Icon16x16));

	Set("BindingView.AddBinding", new IMAGE_BRUSH_SVG("Slate/ViewModel_AddBinding", Icon16x16));
	Set("BindingView.Background", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 0.0f, FStyleColors::Panel, 4.0f));
	Set("BindingView.ViewModelWarning", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::Hover, 1.0f));
	Set("BindingView.WidgetRow", FTableRowStyle(FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
		.SetEvenRowBackgroundBrush(*FAppStyle::Get().GetBrush("Brushes.Header"))
		.SetOddRowBackgroundBrush(*FAppStyle::Get().GetBrush("Brushes.Header"))
		.SetEvenRowBackgroundHoveredBrush(*FAppStyle::Get().GetBrush("Brushes.Dropdown"))
		.SetOddRowBackgroundHoveredBrush(*FAppStyle::Get().GetBrush("Brushes.Dropdown"))
	);

	Set("BindingView.BindingRow", FTableRowStyle(FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
		.SetEvenRowBackgroundBrush(*FAppStyle::Get().GetBrush("Brushes.Panel"))
		.SetOddRowBackgroundBrush(*FAppStyle::Get().GetBrush("Brushes.Panel"))
		.SetEvenRowBackgroundHoveredBrush(*FAppStyle::Get().GetBrush("Brushes.Header"))
		.SetOddRowBackgroundHoveredBrush(*FAppStyle::Get().GetBrush("Brushes.Header"))
	);

	Set("BindingView.ParameterRow", FTableRowStyle(FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
		.SetEvenRowBackgroundBrush(*FAppStyle::Get().GetBrush("Brushes.Panel"))
		.SetOddRowBackgroundBrush(*FAppStyle::Get().GetBrush("Brushes.Panel"))
		.SetEvenRowBackgroundHoveredBrush(*FAppStyle::Get().GetBrush("Brushes.Header"))
		.SetOddRowBackgroundHoveredBrush(*FAppStyle::Get().GetBrush("Brushes.Header"))
	);
	Set("BindingView.VerseBinding", new IMAGE_BRUSH("Slate/VerseBinding_16", Icon16x16));

	FButtonStyle NoStyleComboButtonButtonStyle = FButtonStyle()
		.SetNormal(FSlateBoxBrush(RootToCoreContentDir("Editor/Slate/Common/ButtonHoverHint.png"), FMargin(4.0f / 16.0f), FLinearColor(1.0f, 1.0f, 1.0f, 0.15f)))
		.SetHovered(FSlateBoxBrush(RootToCoreContentDir("Editor/Slate/Common/ButtonHoverHint.png"), FMargin(4.0f / 16.0f), FLinearColor(1.0f, 1.0f, 1.0f, 0.25f)))
		.SetPressed(FSlateBoxBrush(RootToCoreContentDir("Editor/Slate/Common/ButtonHoverHint.png"), FMargin(4.0f / 16.0f), FLinearColor(1.0f, 1.0f, 1.0f, 0.30f)))
		.SetNormalPadding(FMargin(0.0f, 0.0f, 0.0f, 1.0f))
		.SetPressedPadding(FMargin(0.0f, 1.0f, 0.0f, 0.0f));
	Set("NoStyleComboButton", FComboButtonStyle()
		.SetButtonStyle(NoStyleComboButtonButtonStyle)
		.SetDownArrowImage(FSlateNoResource())
		.SetMenuBorderBrush(FSlateNoResource())
		.SetMenuBorderPadding(FMargin(0.0f)));

	// ViewModelSelectionWidget Icons
	{
		Set("ViewModelSelection.AddIcon", new IMAGE_BRUSH("Slate/MVVMViewModelSelection_Add_16x", Icon16x16));
		Set("ViewModelSelection.RemoveIcon", new IMAGE_BRUSH("Slate/MVVMViewModelSelection_Remove_16x", Icon16x16));
		Set("ViewModelSelection.TagIcon", new IMAGE_BRUSH("Slate/MVVMViewModelSelection_Tag_16x", Icon16x16));
	}

	{
		Set("PropertyPath.ContextText", FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetFont(DEFAULT_FONT("Bold", 8))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.9f))
		.SetShadowOffset(FVector2D(1, 1))
		.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.7f)));
	}

	Set("BindingMode.OneWay", new IMAGE_BRUSH_SVG("Slate/BindingMode_OneWay", Icon16x16));
	Set("BindingMode.OneWayToSource", new IMAGE_BRUSH_SVG("Slate/BindingMode_OneWayToSource", Icon16x16));
	Set("BindingMode.TwoWay", new IMAGE_BRUSH_SVG("Slate/BindingMode_TwoWay", Icon16x16));
	Set("BindingMode.OneTime", new IMAGE_BRUSH_SVG("Slate/BindingMode_OneTime", Icon16x16));
	Set("BindingMode.OneTimeOneWay", new IMAGE_BRUSH_SVG("Slate/BindingMode_OneTimeOneWayToSource", Icon16x16));
	Set("BindingMode.OneTimeOneWayToSource", new IMAGE_BRUSH_SVG("Slate/BindingMode_OneTimeOneWay", Icon16x16));
	Set("BindingMode.OneTimeTwoWay", new IMAGE_BRUSH_SVG("Slate/BindingMode_OneTimeTwoWay", Icon16x16));

	Set("ConversionFunction.DestToSource", new IMAGE_BRUSH_SVG("Slate/ConversionFunction_DestToSource", Icon16x16));
	Set("ConversionFunction.SourceToDest", new IMAGE_BRUSH_SVG("Slate/ConversionFunction_SourceToDest", Icon16x16));

	Set("Icon.Ellipsis", new CORE_IMAGE_BRUSH_SVG("Slate/Starship/Common/ellipsis-vertical-narrow", FVector2D(6.0, 24.0)));

	Set("FieldSelector.ComboButton", 
		FComboButtonStyle(FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton"))
		.SetDownArrowImage(*FCoreStyle::Get().GetBrush("Icons.Edit")));

	Set("FunctionParameter.Border", new FSlateColorBrush(FStyleColors::Input));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}


FMVVMEditorStyle::~FMVVMEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

void FMVVMEditorStyle::CreateInstance()
{
	Instance.Reset(new FMVVMEditorStyle);
}


void FMVVMEditorStyle::DestroyInstance()
{
	Instance.Reset();
}
