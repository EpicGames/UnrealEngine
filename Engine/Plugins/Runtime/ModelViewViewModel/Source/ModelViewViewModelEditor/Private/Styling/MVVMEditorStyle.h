// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/**
 * 
 */
class FMVVMEditorStyle : public FSlateStyleSet
{

public:
	static void CreateInstance();
	static void DestroyInstance();

	static FMVVMEditorStyle& Get()
	{
		check(Instance);
		return *Instance;
	}

private:
	FMVVMEditorStyle();
	FMVVMEditorStyle(const FMVVMEditorStyle&) = delete;
	FMVVMEditorStyle& operator=(const FMVVMEditorStyle&) = delete;
	~FMVVMEditorStyle();
	struct FCustomDeleter
	{
		void operator()(FMVVMEditorStyle* Ptr) const 
		{
			delete Ptr;
		}
	};
	static TUniquePtr<FMVVMEditorStyle, FCustomDeleter> Instance;
};
