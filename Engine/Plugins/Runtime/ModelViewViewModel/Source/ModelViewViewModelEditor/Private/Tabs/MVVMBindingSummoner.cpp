// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tabs/MVVMBindingSummoner.h"

#include "WidgetBlueprintEditor.h"
#include "Styling/MVVMEditorStyle.h"
#include "Widgets/SMVVMViewBindingPanel.h"
#include "StatusBarSubsystem.h"

#define LOCTEXT_NAMESPACE "TabSummoner"

const FName FMVVMBindingSummoner::TabID(TEXT("MVVMTab"));
const FName FMVVMBindingSummoner::DrawerID(TEXT("MVVMDrawer"));

FMVVMBindingSummoner::FMVVMBindingSummoner(TSharedPtr<FWidgetBlueprintEditor> BlueprintEditor, bool bInIsDrawerTab)
	: FWorkflowTabFactory(TabID, BlueprintEditor)
	, WeakWidgetBlueprintEditor(BlueprintEditor)
	, bIsDrawerTab(bInIsDrawerTab)
{
	TabLabel = LOCTEXT("ViewBinding_ViewMenu_Label", "View Bindings");
	TabIcon = FSlateIcon(FMVVMEditorStyle::Get().GetStyleSetName(), "BlueprintView.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("ViewBinding_ViewMenu_Desc", "View Bindings");
	ViewMenuTooltip = LOCTEXT("ViewBinding_ViewMenu_ToolTip", "Show the View Bindings tab");
}

TSharedRef<SWidget> FMVVMBindingSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(UE::MVVM::SBindingsPanel, WeakWidgetBlueprintEditor.Pin(), bIsDrawerTab);
}

void FMVVMBindingSummoner::ToggleMVVMDrawer()
{
	GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->TryToggleDrawer(FMVVMBindingSummoner::DrawerID);
}


#undef LOCTEXT_NAMESPACE