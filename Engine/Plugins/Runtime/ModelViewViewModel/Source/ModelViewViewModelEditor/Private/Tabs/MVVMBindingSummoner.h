// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FWidgetBlueprintEditor;

struct FMVVMBindingSummoner : public FWorkflowTabFactory
{
	static const FName TabID;
	static const FName DrawerID;

	FMVVMBindingSummoner(TSharedPtr<FWidgetBlueprintEditor> BlueprintEditor, bool bInIsDrawerTab = false);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	static void ToggleMVVMDrawer();

private:
	TWeakPtr<FWidgetBlueprintEditor> WeakWidgetBlueprintEditor;
	bool bIsDrawerTab;
};
