// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tabs/MVVMViewModelSummoner.h"

#include "Styling/MVVMEditorStyle.h"
#include "WidgetBlueprintEditor.h"
#include "Widgets/SMVVMViewModelPanel.h"

#define LOCTEXT_NAMESPACE "ViewModelPanel"

namespace UE::MVVM
{

const FName FViewModelSummoner::TabID(TEXT("ViewModelPanel"));

FViewModelSummoner::FViewModelSummoner(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
		: FWorkflowTabFactory(TabID, InBlueprintEditor)
		, BlueprintEditor(InBlueprintEditor)
{
	TabLabel = LOCTEXT("ViewmodelTabLabel", "Viewmodels");
	TabIcon = FSlateIcon(FMVVMEditorStyle::Get().GetStyleSetName(), "BlueprintView.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("Viewmodel_Desc", "Viewmodels");
	ViewMenuTooltip = LOCTEXT("Viewmodel_ToolTip", "Show the viewmodels panel");
}

TSharedRef<SWidget> FViewModelSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FWidgetBlueprintEditor>(BlueprintEditor.Pin());

	return SNew(SMVVMViewModelPanel, BlueprintEditorPtr)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewmodelPanel")));
}

} // namespace

#undef LOCTEXT_NAMESPACE 
