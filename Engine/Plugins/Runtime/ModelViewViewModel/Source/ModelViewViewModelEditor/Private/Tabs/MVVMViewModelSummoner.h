// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FWidgetBlueprintEditor;

namespace UE::MVVM
{

struct FViewModelSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	
public:
	FViewModelSummoner(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	
protected:
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;
};

} // namespace
