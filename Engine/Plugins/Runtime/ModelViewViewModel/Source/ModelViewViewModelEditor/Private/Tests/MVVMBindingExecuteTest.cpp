// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/MVVMBindingExecuteTest.h"
#include "Misc/AutomationTest.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Types/MVVMFieldContext.h"
#include "Types/MVVMFunctionContext.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBindingExecuteTest)


namespace UE::MVVM::Private
{
	int32 GMVVMBindingExecTextCounter = 0;
}


FMVVMBindingExecTextCounter::FMVVMBindingExecTextCounter()
	: Value(99)
{
	++UE::MVVM::Private::GMVVMBindingExecTextCounter;
}


FMVVMBindingExecTextCounter::FMVVMBindingExecTextCounter(const FMVVMBindingExecTextCounter& Other)
	: Value(Other.Value)
{
	++UE::MVVM::Private::GMVVMBindingExecTextCounter;
}

FMVVMBindingExecTextCounter::FMVVMBindingExecTextCounter(FMVVMBindingExecTextCounter&& Other)
	: Value(Other.Value)
{
	++UE::MVVM::Private::GMVVMBindingExecTextCounter;
}


FMVVMBindingExecTextCounter& FMVVMBindingExecTextCounter::operator=(const FMVVMBindingExecTextCounter& Other)
{
	Value = Other.Value;
	return *this;
}


FMVVMBindingExecTextCounter& FMVVMBindingExecTextCounter::operator=(FMVVMBindingExecTextCounter&& Other)
{
	Value = Other.Value;
	return *this;
}


FMVVMBindingExecTextCounter::~FMVVMBindingExecTextCounter()
{
	--UE::MVVM::Private::GMVVMBindingExecTextCounter;
}


int32 UMVVMViewModelBindingExecTest::ConversionStructToInt(FMVVMBindingExecTextCounter Value)
{
	return Value.Value;
}


int32 UMVVMViewModelBindingExecTest::ConversionConstStructToInt(const FMVVMBindingExecTextCounter& Value)
{
	return Value.Value;
}


TArray<int32> UMVVMViewModelBindingExecTest::ConversionArrayStructToArrayInt(const TArray<FMVVMBindingExecTextCounter>& Values)
{
	TArray<int32> Result;
	for (const FMVVMBindingExecTextCounter& Value : Values)
	{
		Result.Add(Value.Value);
	}
	return Result;
}


FMVVMBindingExecTextCounter UMVVMViewModelBindingExecTest::ConversionIntToStruct(int32 Value)
{
	FMVVMBindingExecTextCounter Tmp;
	Tmp.Value = Value;
	return Tmp;
}


FMVVMBindingExecTextCounter UMVVMViewModelBindingExecTest::ConversionConstIntToStruct(const int32& Value)
{
	FMVVMBindingExecTextCounter Tmp;
	Tmp.Value = Value;
	return Tmp;
}


TArray<FMVVMBindingExecTextCounter> UMVVMViewModelBindingExecTest::ConversionArrayIntToArrayStruct(const TArray<int32>& Values)
{
	TArray<FMVVMBindingExecTextCounter> Result;
	for (const int32 Value : Values)
	{
		FMVVMBindingExecTextCounter Tmp;
		Tmp.Value = Value;
		Result.Add(Tmp);
	}
	return Result;
}

float UMVVMViewModelBindingExecTest::ConversionIncFloat(float Value)
{
	return Value + 10.f;
}

float UMVVMViewModelBindingExecTest::ConversionIncDouble(double Value)
{
	return (float)(Value + 10.0);
}


#if WITH_AUTOMATION_WORKER

#define LOCTEXT_NAMESPACE "MVVMBindingExecuteTest"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMVVMBindingExecuteTest, "System.Plugins.MVVM.BindingExecution", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)


bool FMVVMBindingExecuteTest::RunTest(const FString& Parameters)
{
	UMVVMViewModelBindingExecTest* SourceObj = NewObject<UMVVMViewModelBindingExecTest>(GetTransientPackage(), TEXT("My Object Name Is The Best"));
	UMVVMViewModelBindingExecTest* DestinationObj = NewObject<UMVVMViewModelBindingExecTest>();
	SourceObj->AddToRoot();
	DestinationObj->AddToRoot();

	{
		auto AssignPropertyA = [this, SourceObj, DestinationObj](UE::MVVM::FMVVMFieldVariant SourceBinding, UE::MVVM::FMVVMFieldVariant DestinationBinding, const TCHAR* Msg)
		{
			SourceObj->PropertyA.Value = 4;
			DestinationObj->PropertyA.Value = 99;
			UE::MVVM::Private::GMVVMBindingExecTextCounter = 0;

			UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBinding);
			UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBinding);
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination);

			if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547 
			{
				AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
			}
			if (SourceObj->PropertyA != DestinationObj->PropertyA)
			{
				AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
			}
			if (SourceObj->PropertyA.Value != 4)
			{
				AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
			}
		};

		FProperty* PropA = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyA));
		UFunction* GetA = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterA));
		UFunction* SetA = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterA));
		AssignPropertyA(UE::MVVM::FMVVMFieldVariant(PropA), UE::MVVM::FMVVMFieldVariant(PropA), TEXT("PropertyA = PropertyA"));
		AssignPropertyA(UE::MVVM::FMVVMFieldVariant(PropA), UE::MVVM::FMVVMFieldVariant(SetA), TEXT("SetPropertyA(PropertyA)"));
		AssignPropertyA(UE::MVVM::FMVVMFieldVariant(GetA), UE::MVVM::FMVVMFieldVariant(SetA), TEXT("SetPropertyA(GetPropertyA())"));
		AssignPropertyA(UE::MVVM::FMVVMFieldVariant(GetA), UE::MVVM::FMVVMFieldVariant(PropA), TEXT("PropertyA = GetPropertyA()"));
	}

	{
		auto AssignPropertyB = [this, SourceObj, DestinationObj](UE::MVVM::FMVVMFieldVariant SourceBinding, UE::MVVM::FMVVMFieldVariant DestinationBinding, const TCHAR* Msg)
		{
			SourceObj->PropertyB.Reset();
			DestinationObj->PropertyB.Reset();

			for (int32 Index = 0; Index < 4; ++Index)
			{
				SourceObj->PropertyB[SourceObj->PropertyB.Add(FMVVMBindingExecTextCounter())].Value = Index+5;
			}
			UE::MVVM::Private::GMVVMBindingExecTextCounter = 0;

			UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBinding);
			UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBinding);
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination);

			if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 4) // -V547 
			{
				AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
			}
			if (SourceObj->PropertyB != DestinationObj->PropertyB)
			{
				AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
			}
			if (SourceObj->PropertyB[0].Value != 5)
			{
				AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
			}
		};

		FProperty* PropB = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyB));
		UFunction* GetB = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterB));
		UFunction* SetB = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterB));
		AssignPropertyB(UE::MVVM::FMVVMFieldVariant(PropB), UE::MVVM::FMVVMFieldVariant(PropB), TEXT("PropertyB = PropertyB"));
		AssignPropertyB(UE::MVVM::FMVVMFieldVariant(PropB), UE::MVVM::FMVVMFieldVariant(SetB), TEXT("SetPropertyB(PropertyB)"));
		AssignPropertyB(UE::MVVM::FMVVMFieldVariant(GetB), UE::MVVM::FMVVMFieldVariant(SetB), TEXT("SetPropertyB(GetPropertyB())"));
		AssignPropertyB(UE::MVVM::FMVVMFieldVariant(GetB), UE::MVVM::FMVVMFieldVariant(PropB), TEXT("PropertyB = GetPropertyB()"));
	}

	{
		auto AssignPropertyC = [this, SourceObj, DestinationObj](UE::MVVM::FMVVMFieldVariant SourceBinding, UE::MVVM::FMVVMFieldVariant DestinationBinding, const TCHAR* Msg)
		{
			SourceObj->PropertyC = 88;
			DestinationObj->PropertyC = 99;
			UE::MVVM::Private::GMVVMBindingExecTextCounter = 0;

			UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBinding);
			UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBinding);
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination);

			if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547 
			{
				AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
			}
			if (SourceObj->PropertyC != DestinationObj->PropertyC)
			{
				AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
			}
			if (SourceObj->PropertyC != 88)
			{
				AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
			}
		};

		FProperty* PropC = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyC));
		UFunction* GetC = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterC));
		UFunction* SetC = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterC));
		AssignPropertyC(UE::MVVM::FMVVMFieldVariant(PropC), UE::MVVM::FMVVMFieldVariant(PropC), TEXT("PropertyC = PropertyC"));
		AssignPropertyC(UE::MVVM::FMVVMFieldVariant(PropC), UE::MVVM::FMVVMFieldVariant(SetC), TEXT("SetPropertyC(PropertyC)"));
		AssignPropertyC(UE::MVVM::FMVVMFieldVariant(GetC), UE::MVVM::FMVVMFieldVariant(SetC), TEXT("SetPropertyC(GetPropertyC())"));
		AssignPropertyC(UE::MVVM::FMVVMFieldVariant(GetC), UE::MVVM::FMVVMFieldVariant(PropC), TEXT("PropertyC = GetPropertyC()"));
	}

	{
		auto AssignPropertyD = [this, SourceObj, DestinationObj](UE::MVVM::FMVVMFieldVariant SourceBinding, UE::MVVM::FMVVMFieldVariant DestinationBinding, const TCHAR* Msg)
		{
			SourceObj->PropertyD.Reset();
			DestinationObj->PropertyD.Reset();

			for (int32 Index = 0; Index < 4; ++Index)
			{
				SourceObj->PropertyD.Add(Index+10);
			}
			UE::MVVM::Private::GMVVMBindingExecTextCounter = 0;

			UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBinding);
			UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBinding);
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination);

			if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547
			{
				AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
			}
			if (SourceObj->PropertyD != DestinationObj->PropertyD)
			{
				AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
			}
			if (SourceObj->PropertyD[0] != 10)
			{
				AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
			}
		};

		FProperty* PropD = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyD));
		UFunction* GetD = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterD));
		UFunction* SetD = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterD));
		AssignPropertyD(UE::MVVM::FMVVMFieldVariant(PropD), UE::MVVM::FMVVMFieldVariant(PropD), TEXT("PropertyD = PropertyD"));
		AssignPropertyD(UE::MVVM::FMVVMFieldVariant(PropD), UE::MVVM::FMVVMFieldVariant(SetD), TEXT("SetPropertyD(PropertyD)"));
		AssignPropertyD(UE::MVVM::FMVVMFieldVariant(GetD), UE::MVVM::FMVVMFieldVariant(SetD), TEXT("SetPropertyD(GetPropertyD())"));
		AssignPropertyD(UE::MVVM::FMVVMFieldVariant(GetD), UE::MVVM::FMVVMFieldVariant(PropD), TEXT("PropertyD = GetPropertyD()"));
	}
	{
		auto AssignPropertyAToC = [this, SourceObj, DestinationObj](UE::MVVM::FMVVMFieldVariant SourceBinding, UE::MVVM::FMVVMFieldVariant DestinationBinding, UFunction* Conversion, const TCHAR* Msg)
		{
			SourceObj->PropertyA.Value = 8;
			DestinationObj->PropertyC = 99;
			UE::MVVM::Private::GMVVMBindingExecTextCounter = 0;

			UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBinding);
			UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBinding);
			UE::MVVM::FFunctionContext Function = UE::MVVM::FFunctionContext::MakeStaticFunction(Conversion);
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination, Function);

			if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547
			{
				AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
			}
			if (SourceObj->PropertyA.Value != DestinationObj->PropertyC)
			{
				AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
			}
			if (SourceObj->PropertyA.Value != 8)
			{
				AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
			}
		};

		FProperty* PropA = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyA));
		FProperty* PropC = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyC));
		UFunction* GetA = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterA));
		UFunction* SetC = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterC));
		UFunction* Conversion = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, ConversionStructToInt));
		UFunction* ConversionRef = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, ConversionConstStructToInt));
		AssignPropertyAToC(UE::MVVM::FMVVMFieldVariant(PropA), UE::MVVM::FMVVMFieldVariant(PropC), Conversion, TEXT("PropertyC = ConversionStructToInt(PropertyA)"));
		AssignPropertyAToC(UE::MVVM::FMVVMFieldVariant(PropA), UE::MVVM::FMVVMFieldVariant(SetC), Conversion, TEXT("SetPropertyC(ConversionStructToInt(PropertyA))"));
		AssignPropertyAToC(UE::MVVM::FMVVMFieldVariant(GetA), UE::MVVM::FMVVMFieldVariant(SetC), Conversion, TEXT("SetPropertyC(ConversionStructToInt(GetPropertyA()))"));
		AssignPropertyAToC(UE::MVVM::FMVVMFieldVariant(GetA), UE::MVVM::FMVVMFieldVariant(PropC), Conversion, TEXT("PropertyC = ConversionStructToInt(GetPropertyA())"));
		AssignPropertyAToC(UE::MVVM::FMVVMFieldVariant(PropA), UE::MVVM::FMVVMFieldVariant(PropC), ConversionRef, TEXT("PropertyC = ConversionConstStructToInt(PropertyA)"));
		AssignPropertyAToC(UE::MVVM::FMVVMFieldVariant(PropA), UE::MVVM::FMVVMFieldVariant(SetC), ConversionRef, TEXT("SetPropertyC(ConversionConstStructToInt(PropertyA))"));
		AssignPropertyAToC(UE::MVVM::FMVVMFieldVariant(GetA), UE::MVVM::FMVVMFieldVariant(SetC), ConversionRef, TEXT("SetPropertyC(ConversionConstStructToInt(GetPropertyA()))"));
		AssignPropertyAToC(UE::MVVM::FMVVMFieldVariant(GetA), UE::MVVM::FMVVMFieldVariant(PropC), ConversionRef, TEXT("PropertyC = ConversionConstStructToInt(GetPropertyA())"));
	}
	{
		auto AssignPropertyCToA = [this, SourceObj, DestinationObj](UE::MVVM::FMVVMFieldVariant SourceBinding, UE::MVVM::FMVVMFieldVariant DestinationBinding, UFunction* Conversion, const TCHAR* Msg)
		{
			SourceObj->PropertyC = 159;
			DestinationObj->PropertyA.Value = 999;
			UE::MVVM::Private::GMVVMBindingExecTextCounter = 0;

			UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBinding);
			UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBinding);
			UE::MVVM::FFunctionContext Function = UE::MVVM::FFunctionContext::MakeStaticFunction(Conversion);
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination, Function);

			if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547
			{
				AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
			}
			if (SourceObj->PropertyA.Value != DestinationObj->PropertyC)
			{
				AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
			}
			if (SourceObj->PropertyC != 159)
			{
				AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
			}
		};

		FProperty* PropA = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyA));
		FProperty* PropC = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyC));
		UFunction* SetA = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterA));
		UFunction* GetC = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterC));
		UFunction* Conversion = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, ConversionIntToStruct));
		UFunction* ConversionRef = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, ConversionConstIntToStruct));
		AssignPropertyCToA(UE::MVVM::FMVVMFieldVariant(PropC), UE::MVVM::FMVVMFieldVariant(PropA), Conversion, TEXT("PropertyA = ConversionIntToStruct(PropertyC)"));
		AssignPropertyCToA(UE::MVVM::FMVVMFieldVariant(PropC), UE::MVVM::FMVVMFieldVariant(SetA), Conversion, TEXT("SetPropertyA(ConversionIntToStruct(PropertyC))"));
		AssignPropertyCToA(UE::MVVM::FMVVMFieldVariant(GetC), UE::MVVM::FMVVMFieldVariant(SetA), Conversion, TEXT("SetPropertyA(ConversionIntToStruct(GetPropertyC()))"));
		AssignPropertyCToA(UE::MVVM::FMVVMFieldVariant(GetC), UE::MVVM::FMVVMFieldVariant(PropA), Conversion, TEXT("PropertyA = ConversionIntToStruct(GetPropertyC())"));
		AssignPropertyCToA(UE::MVVM::FMVVMFieldVariant(PropC), UE::MVVM::FMVVMFieldVariant(PropA), ConversionRef, TEXT("PropertyA = ConversionConstIntToStruct(PropertyC)"));
		AssignPropertyCToA(UE::MVVM::FMVVMFieldVariant(PropC), UE::MVVM::FMVVMFieldVariant(SetA), ConversionRef, TEXT("SetPropertyA(ConversionConstIntToStruct(PropertyC))"));
		AssignPropertyCToA(UE::MVVM::FMVVMFieldVariant(GetC), UE::MVVM::FMVVMFieldVariant(SetA), ConversionRef, TEXT("SetPropertyA(ConversionConstIntToStruct(GetPropertyC()))"));
		AssignPropertyCToA(UE::MVVM::FMVVMFieldVariant(GetC), UE::MVVM::FMVVMFieldVariant(PropA), ConversionRef, TEXT("PropertyA = ConversionConstIntToStruct(GetPropertyC())"));
	}
	{
		auto AssignPropertyBToD = [this, SourceObj, DestinationObj](UE::MVVM::FMVVMFieldVariant SourceBinding, UE::MVVM::FMVVMFieldVariant DestinationBinding, UFunction* Conversion, const TCHAR* Msg)
		{
			SourceObj->PropertyB.Reset();
			DestinationObj->PropertyD.Reset();

			for (int32 Index = 0; Index < 4; ++Index)
			{
				SourceObj->PropertyB[SourceObj->PropertyB.Add(FMVVMBindingExecTextCounter())].Value = Index + 898;
			}
			UE::MVVM::Private::GMVVMBindingExecTextCounter = 0;

			UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBinding);
			UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBinding);
			UE::MVVM::FFunctionContext Function = UE::MVVM::FFunctionContext::MakeStaticFunction(Conversion);
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination, Function);

			if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547
			{
				AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
			}
			bool bIsSame = SourceObj->PropertyB.Num() == DestinationObj->PropertyB.Num();
			if (bIsSame)
			{
				for (int32 Index = 0; Index < SourceObj->PropertyB.Num(); ++Index)
				{
					bIsSame = bIsSame && SourceObj->PropertyB[Index] == DestinationObj->PropertyD[Index];
				}
			}
			if (!bIsSame)
			{
				AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
			}
			if (SourceObj->PropertyB[0].Value != 898)
			{
				AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
			}
		};

		FProperty* PropB = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyB));
		FProperty* PropD = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyD));
		UFunction* GetB = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterB));
		UFunction* SetD = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterD));
		UFunction* Conversion = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, ConversionArrayStructToArrayInt));
		AssignPropertyBToD(UE::MVVM::FMVVMFieldVariant(PropB), UE::MVVM::FMVVMFieldVariant(PropD), Conversion, TEXT("PropertyD = ConversionArrayStructToArrayInt(PropertyB)"));
		AssignPropertyBToD(UE::MVVM::FMVVMFieldVariant(PropB), UE::MVVM::FMVVMFieldVariant(SetD), Conversion, TEXT("SetPropertyD(ConversionArrayStructToArrayInt(PropertyB))"));
		AssignPropertyBToD(UE::MVVM::FMVVMFieldVariant(GetB), UE::MVVM::FMVVMFieldVariant(SetD), Conversion, TEXT("SetPropertyD(ConversionArrayStructToArrayInt(GetPropertyB()))"));
		AssignPropertyBToD(UE::MVVM::FMVVMFieldVariant(GetB), UE::MVVM::FMVVMFieldVariant(PropD), Conversion, TEXT("PropertyD = ConversionArrayStructToArrayInt(GetPropertyB())"));
	}
	{
		auto AssignPropertyDToB = [this, SourceObj, DestinationObj](UE::MVVM::FMVVMFieldVariant SourceBinding, UE::MVVM::FMVVMFieldVariant DestinationBinding, UFunction* Conversion, const TCHAR* Msg)
		{
			SourceObj->PropertyD.Reset();
			DestinationObj->PropertyB.Reset();

			for (int32 Index = 0; Index < 4; ++Index)
			{
				SourceObj->PropertyD.Add(Index + 987);
			}
			UE::MVVM::Private::GMVVMBindingExecTextCounter = 0;

			UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBinding);
			UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBinding);
			UE::MVVM::FFunctionContext Function = UE::MVVM::FFunctionContext::MakeStaticFunction(Conversion);
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination, Function);

			if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 4) // -V547
			{
				AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
			}
			bool bIsSame = SourceObj->PropertyD.Num() == DestinationObj->PropertyB.Num();
			if (bIsSame)
			{
				for (int32 Index = 0; Index < SourceObj->PropertyD.Num(); ++Index)
				{
					bIsSame = bIsSame && DestinationObj->PropertyB[Index] == SourceObj->PropertyD[Index];
				}
			}
			if (!bIsSame)
			{
				AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
			}
			if (SourceObj->PropertyD[0] != 987)
			{
				AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
			}
		};

		FProperty* PropB = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyB));
		FProperty* PropD = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyD));
		UFunction* SetB = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterB));
		UFunction* GetD = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterD));
		UFunction* Conversion = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, ConversionArrayIntToArrayStruct));
		AssignPropertyDToB(UE::MVVM::FMVVMFieldVariant(PropD), UE::MVVM::FMVVMFieldVariant(PropB), Conversion, TEXT("PropertyB = ConversionArrayIntToArrayStruct(PropertyD)"));
		AssignPropertyDToB(UE::MVVM::FMVVMFieldVariant(PropD), UE::MVVM::FMVVMFieldVariant(SetB), Conversion, TEXT("SetPropertyB(ConversionArrayIntToArrayStruct(PropertyD))"));
		AssignPropertyDToB(UE::MVVM::FMVVMFieldVariant(GetD), UE::MVVM::FMVVMFieldVariant(SetB), Conversion, TEXT("SetPropertyB(ConversionArrayIntToArrayStruct(GetPropertyD()))"));
		AssignPropertyDToB(UE::MVVM::FMVVMFieldVariant(GetD), UE::MVVM::FMVVMFieldVariant(PropB), Conversion, TEXT("PropertyB = ConversionArrayIntToArrayStruct(GetPropertyD())"));
	}
	{
		auto AssignPropertyFloatDouble = [this, SourceObj, DestinationObj](
			UE::MVVM::FMVVMFieldVariant SourceBindingFloat, UE::MVVM::FMVVMFieldVariant SourceBindingDouble
			, UE::MVVM::FMVVMFieldVariant DestinationBindingFloat, UE::MVVM::FMVVMFieldVariant DestinationBindingDouble
			, UFunction* Conversion
			, const TCHAR* Msg)
		{
			UE::MVVM::Private::GMVVMBindingExecTextCounter = 0;
			SourceObj->PropertyFloat = 4.f;
			SourceObj->PropertyDouble = 4.0;
			DestinationObj->PropertyFloat = 99.f;
			DestinationObj->PropertyDouble = 99.0;

			float ConversionModifyFloat = Conversion ? 10.f : 0.f;
			float ConversionModifyDouble = Conversion ? 10.0 : 0.0;

			{// float to float
				UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBindingFloat);
				UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBindingFloat);
				if (Conversion)
				{
					UE::MVVM::FFunctionContext Function = UE::MVVM::FFunctionContext::MakeStaticFunction(Conversion);
					UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination, Function);
				}
				else
				{
					UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination);
				}

				if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547
				{
					AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
				}
				if (!FMath::IsNearlyEqual(SourceObj->PropertyFloat + ConversionModifyFloat, DestinationObj->PropertyFloat))
				{
					AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
				}
				if (!FMath::IsNearlyEqual(SourceObj->PropertyFloat, 4.f))
				{
					AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
				}
			}
			{//float to double
				UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBindingFloat);
				UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBindingDouble);
				if (Conversion)
				{
					UE::MVVM::FFunctionContext Function = UE::MVVM::FFunctionContext::MakeStaticFunction(Conversion);
					UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination, Function);
				}
				else
				{
					UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination);
				}

				if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547
				{
					AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
				}
				if (!FMath::IsNearlyEqual(SourceObj->PropertyFloat + ConversionModifyFloat, DestinationObj->PropertyDouble))
				{
					AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
				}
				if (!FMath::IsNearlyEqual(SourceObj->PropertyFloat, 4.f))
				{
					AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
				}
			}
			DestinationObj->PropertyFloat = 99.f;
			DestinationObj->PropertyDouble = 99.0;
			{//double to float
				UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBindingDouble);
				UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBindingFloat);
				if (Conversion)
				{
					UE::MVVM::FFunctionContext Function = UE::MVVM::FFunctionContext::MakeStaticFunction(Conversion);
					UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination, Function);
				}
				else
				{
					UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination);
				}

				if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547
				{
					AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
				}
				if (!FMath::IsNearlyEqual(SourceObj->PropertyDouble + ConversionModifyDouble, DestinationObj->PropertyFloat))
				{
					AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
				}
				if (!FMath::IsNearlyEqual(SourceObj->PropertyDouble, 4.0))
				{
					AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
				}
			}
			{//double to double
				UE::MVVM::FFieldContext Source(UE::MVVM::FObjectVariant(SourceObj), SourceBindingDouble);
				UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBindingDouble);
				if (Conversion)
				{
					UE::MVVM::FFunctionContext Function = UE::MVVM::FFunctionContext::MakeStaticFunction(Conversion);
					UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination, Function);
				}
				else
				{
					UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Source, Destination);
				}

				if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547
				{
					AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
				}
				if (!FMath::IsNearlyEqual(SourceObj->PropertyDouble + ConversionModifyDouble, DestinationObj->PropertyDouble))
				{
					AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
				}
				if (!FMath::IsNearlyEqual(SourceObj->PropertyDouble, 4.0))
				{
					AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
				}
			}
		};

		FProperty* PropFloat = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyFloat));
		FProperty* PropertyFloatAccessor = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyFloatAccessor));
		FProperty* PropDouble = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyDouble));
		UFunction* GetFloat = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterFloat));
		UFunction* GetDouble = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterDouble));
		UFunction* SetFloat = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterFloat));
		UFunction* SetDouble = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterDouble));
		UFunction* ConversionIncFloatFuntion = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, ConversionIncFloat));
		UFunction* ConversionIncDoubleFuntion = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, ConversionIncDouble));

		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble)
			, UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble), nullptr, TEXT("PropertyReal = PropertyReal"));
		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble)
			, UE::MVVM::FMVVMFieldVariant(SetFloat), UE::MVVM::FMVVMFieldVariant(SetDouble), nullptr, TEXT("SetPropertyRead(PropertyRead)"));
		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(GetFloat), UE::MVVM::FMVVMFieldVariant(GetDouble)
			,UE::MVVM::FMVVMFieldVariant(SetFloat), UE::MVVM::FMVVMFieldVariant(SetDouble), nullptr, TEXT("SetPropertyReal(GetPropertyReal())"));
		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(GetFloat), UE::MVVM::FMVVMFieldVariant(GetDouble)
			, UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble), nullptr, TEXT("PropertyReal = GetPropertyReal()"));

		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble)
			, UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble), ConversionIncFloatFuntion, TEXT("PropertyReal = PropertyReal"));
		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble)
			, UE::MVVM::FMVVMFieldVariant(SetFloat), UE::MVVM::FMVVMFieldVariant(SetDouble), ConversionIncFloatFuntion, TEXT("SetPropertyRead(PropertyRead)"));
		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(GetFloat), UE::MVVM::FMVVMFieldVariant(GetDouble)
			, UE::MVVM::FMVVMFieldVariant(SetFloat), UE::MVVM::FMVVMFieldVariant(SetDouble), ConversionIncFloatFuntion, TEXT("SetPropertyReal(GetPropertyReal())"));
		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(GetFloat), UE::MVVM::FMVVMFieldVariant(GetDouble)
			, UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble), ConversionIncFloatFuntion, TEXT("PropertyReal = GetPropertyReal()"));

		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble)
			, UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble), ConversionIncDoubleFuntion, TEXT("PropertyReal = PropertyReal"));
		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble)
			, UE::MVVM::FMVVMFieldVariant(SetFloat), UE::MVVM::FMVVMFieldVariant(SetDouble), ConversionIncDoubleFuntion, TEXT("SetPropertyRead(PropertyRead)"));
		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(GetFloat), UE::MVVM::FMVVMFieldVariant(GetDouble)
			, UE::MVVM::FMVVMFieldVariant(SetFloat), UE::MVVM::FMVVMFieldVariant(SetDouble), ConversionIncDoubleFuntion, TEXT("SetPropertyReal(GetPropertyReal())"));
		AssignPropertyFloatDouble(UE::MVVM::FMVVMFieldVariant(GetFloat), UE::MVVM::FMVVMFieldVariant(GetDouble)
			, UE::MVVM::FMVVMFieldVariant(PropFloat), UE::MVVM::FMVVMFieldVariant(PropDouble), ConversionIncDoubleFuntion, TEXT("PropertyReal = GetPropertyReal()"));
	}
	{
		auto AssignPropertyAFromConversionFunction = [this, SourceObj, DestinationObj](UE::MVVM::FMVVMFieldVariant DestinationBinding, UFunction* Conversion, const TCHAR* Msg)
		{
			SourceObj->PropertyA.Value = 448;
			DestinationObj->PropertyA.Value = 999;
			UE::MVVM::Private::GMVVMBindingExecTextCounter = 0;

			UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBinding);
			UE::MVVM::FFunctionContext Function = UE::MVVM::FFunctionContext(SourceObj, Conversion);
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Destination, Function);

			if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547
			{
				AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
			}
			if (SourceObj->PropertyA != DestinationObj->PropertyA)
			{
				AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
			}
			if (SourceObj->PropertyA.Value != 448)
			{
				AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
			}
		};

		FProperty* PropA = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyA));
		UFunction* SetA = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterA));
		UFunction* GetA = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterA));
		AssignPropertyAFromConversionFunction(UE::MVVM::FMVVMFieldVariant(PropA), GetA, TEXT("PropertyA = GetterA()"));
		AssignPropertyAFromConversionFunction(UE::MVVM::FMVVMFieldVariant(SetA), GetA, TEXT("SetPropertyA(GetterA())"));
	}
	{
		auto AssignPropertyCFromConversionFunction = [this, SourceObj, DestinationObj](UE::MVVM::FMVVMFieldVariant DestinationBinding, UFunction* Conversion, const TCHAR* Msg)
		{
			SourceObj->PropertyC = 458;
			DestinationObj->PropertyC = 999;
			UE::MVVM::Private::GMVVMBindingExecTextCounter = 0;

			UE::MVVM::FFieldContext Destination(UE::MVVM::FObjectVariant(DestinationObj), DestinationBinding);
			UE::MVVM::FFunctionContext Function = UE::MVVM::FFunctionContext(SourceObj, Conversion);
			UE::MVVM::BindingHelper::ExecuteBinding_NoCheck(Destination, Function);

			if (UE::MVVM::Private::GMVVMBindingExecTextCounter != 0) // -V547
			{
				AddError(FString::Printf(TEXT("%s failed to release the reources."), Msg));
			}
			if (SourceObj->PropertyC != DestinationObj->PropertyC)
			{
				AddError(FString::Printf(TEXT("%s failed the assignement."), Msg));
			}
			if (SourceObj->PropertyC != 458)
			{
				AddError(FString::Printf(TEXT("%s the source value changes."), Msg));
			}
		};

		FProperty* PropC = UMVVMViewModelBindingExecTest::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingExecTest, PropertyC));
		UFunction* SetC = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, SetterC));
		UFunction* GetC = UMVVMViewModelBindingExecTest::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UMVVMViewModelBindingExecTest, GetterC));
		AssignPropertyCFromConversionFunction(UE::MVVM::FMVVMFieldVariant(PropC), GetC, TEXT("PropertyC = GetterC()"));
		AssignPropertyCFromConversionFunction(UE::MVVM::FMVVMFieldVariant(SetC), GetC, TEXT("SetPropertyC(GetterC())"));
	}

	SourceObj->RemoveFromRoot();
	DestinationObj->RemoveFromRoot();

	return true;
}

#undef LOCTEXT_NAMESPACE 
#endif //WITH_AUTOMATION_WORKER
