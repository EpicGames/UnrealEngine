// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MVVMBindingExecuteTest.generated.h"


USTRUCT(BlueprintType, BlueprintInternalUseOnly)
struct FMVVMBindingExecTextCounter
{
	GENERATED_BODY()

	FMVVMBindingExecTextCounter();
	FMVVMBindingExecTextCounter(const FMVVMBindingExecTextCounter&);
	FMVVMBindingExecTextCounter(FMVVMBindingExecTextCounter&&);
	FMVVMBindingExecTextCounter& operator=(const FMVVMBindingExecTextCounter&);
	FMVVMBindingExecTextCounter& operator=(FMVVMBindingExecTextCounter&&);
	~FMVVMBindingExecTextCounter();

	bool operator==(const FMVVMBindingExecTextCounter& Other) const
	{
		return Value == Other.Value;
	}
	bool operator!=(const FMVVMBindingExecTextCounter& Other) const
	{
		return Value != Other.Value;
	}
	bool operator==(int32 Other) const
	{
		return Value == Other;
	}
	bool operator!=(int32 Other) const
	{
		return Value != Other;
	}

	UPROPERTY()
	int32 Value;
};

template<>
struct TStructOpsTypeTraits<FMVVMBindingExecTextCounter> : public TStructOpsTypeTraitsBase2<FMVVMBindingExecTextCounter>
{
	enum
	{
		WithCopy = true,
	};
};


UCLASS(MinimalAPI, Transient, hidedropdown, Hidden)
class UMVVMViewModelBindingExecTest : public UObject
{
	GENERATED_BODY()

public:	
	UPROPERTY(BlueprintReadWrite, Category = "MVVM")
	FMVVMBindingExecTextCounter PropertyA;

	UPROPERTY(BlueprintReadWrite, Category = "MVVM")
	TArray<FMVVMBindingExecTextCounter> PropertyB;

	UPROPERTY(BlueprintReadWrite, Category = "MVVM")
	int32 PropertyC;
	
	UPROPERTY(BlueprintReadWrite, Category = "MVVM")
	TArray<int32> PropertyD;

public:
	UPROPERTY(BlueprintReadWrite, Category = "MVVM")
	float PropertyFloat;

	UPROPERTY(BlueprintReadWrite, Setter, Getter, Category = "MVVM")
	float PropertyFloatAccessor;

	float GetPropertyFloatAccessor() const { return PropertyFloatAccessor; }
	void SetPropertyFloatAccessor(float InValue) { PropertyFloatAccessor = InValue; }

	UPROPERTY(BlueprintReadWrite, Category = "MVVM")
	double PropertyDouble;

public:
	UFUNCTION(BlueprintCallable, Category = "MVVM")
	FMVVMBindingExecTextCounter GetterA() const
	{
		return PropertyA;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	TArray<FMVVMBindingExecTextCounter> GetterB() const
	{
		return PropertyB;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	int32 GetterC() const
	{
		return PropertyC;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	TArray<int32> GetterD() const
	{
		return PropertyD;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	float GetterFloat() const
	{
		return PropertyFloat;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	double GetterDouble() const
	{
		return PropertyDouble;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void SetterA(FMVVMBindingExecTextCounter Value)
	{
		PropertyA = Value;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void SetterB(TArray<FMVVMBindingExecTextCounter> Value)
	{
		PropertyB = Value;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void SetterC(int32 Value)
	{
		PropertyC = Value;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void SetterD(TArray<int32> Value)
	{
		PropertyD = Value;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void SetterFloat(float Value)
	{
		PropertyFloat = Value;
	}
	
	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void SetterDouble(double Value)
	{
		PropertyDouble = Value;
	}

public:
	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static int32 ConversionStructToInt(FMVVMBindingExecTextCounter Value);

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static int32 ConversionConstStructToInt(const FMVVMBindingExecTextCounter& Value);

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static TArray<int32> ConversionArrayStructToArrayInt(const TArray<FMVVMBindingExecTextCounter>& Values);

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static FMVVMBindingExecTextCounter ConversionIntToStruct(int32 Value);

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static FMVVMBindingExecTextCounter ConversionConstIntToStruct(const int32& Value);

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static TArray<FMVVMBindingExecTextCounter> ConversionArrayIntToArrayStruct(const TArray<int32>& Values);

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static float ConversionIncFloat(float Value);

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static float ConversionIncDouble(double Value);
};
