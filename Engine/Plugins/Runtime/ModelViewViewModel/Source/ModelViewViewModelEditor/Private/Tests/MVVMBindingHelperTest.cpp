// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMBindingHelperTest.h"
#include "Misc/AutomationTest.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Misc/TVariantMeta.h"
#include "Templates/ValueOrError.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMBindingHelperTest)


#if WITH_AUTOMATION_WORKER

#define LOCTEXT_NAMESPACE "MVVMBindingHelperTest"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMVVMBindingHelperTest, "System.Plugins.MVVM.BindingHelper", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)


bool FMVVMBindingHelperTest::RunTest(const FString& Parameters)
{
	auto TestProperty = [this](FName PropertyName
		, bool bIsValidForSourceBinding, bool bIsValidForDestinationBinding
		, bool bIsAccessibleDirectlyForSourceBinding, bool bIsAccessibleDirectlyForDestinationBinding
		, bool bIsAccessibleWithGetterForSourceBinding, bool bIsAccessibleWithSetterForDestinationBinding
		, bool bTryGetPropertyTypeForSourceBinding, bool bTryGetPropertyTypeForDestinationBinding)
	{
		FProperty* Property = UMVVMViewModelBindingHelperTest::StaticClass()->FindPropertyByName(PropertyName);
		check(Property);

		if (::UE::MVVM::BindingHelper::IsValidForSourceBinding(Property) != bIsValidForSourceBinding)
		{
			AddError(FString::Printf(TEXT("IsValidForSourceBinding failed for '%s'."), *PropertyName.ToString()));
		}
		if (::UE::MVVM::BindingHelper::IsValidForDestinationBinding(Property) != bIsValidForDestinationBinding)
		{
			AddError(FString::Printf(TEXT("IsValidForDestinationBinding failed for '%s'."), *PropertyName.ToString()));
		}
		if (::UE::MVVM::BindingHelper::IsAccessibleDirectlyForSourceBinding(Property) != bIsAccessibleDirectlyForSourceBinding)
		{
			AddError(FString::Printf(TEXT("IsAccessibleDirectlyForSourceBinding failed for '%s'."), *PropertyName.ToString()));
		}
		if (::UE::MVVM::BindingHelper::IsAccessibleDirectlyForDestinationBinding(Property) != bIsAccessibleDirectlyForDestinationBinding)
		{
			AddError(FString::Printf(TEXT("IsAccessibleDirectlyForDestinationBinding failed for '%s'."), *PropertyName.ToString()));
		}
		if (::UE::MVVM::BindingHelper::IsAccessibleWithGetterForSourceBinding(Property) != bIsAccessibleWithGetterForSourceBinding)
		{
			AddError(FString::Printf(TEXT("IsAccessibleWithGetterForSourceBinding failed for '%s'."), *PropertyName.ToString()));
		}
		if (::UE::MVVM::BindingHelper::IsAccessibleWithSetterForDestinationBinding(Property) != bIsAccessibleWithSetterForDestinationBinding)
		{
			AddError(FString::Printf(TEXT("IsAccessibleWithSetterForDestinationBinding failed for '%s'."), *PropertyName.ToString()));
		}

		if (::UE::MVVM::BindingHelper::TryGetPropertyTypeForSourceBinding(Property).HasValue() != bTryGetPropertyTypeForSourceBinding)
		{
			AddError(FString::Printf(TEXT("TryGetPropertyTypeForSourceBinding failed for '%s'."), *PropertyName.ToString()));
		}
		if (::UE::MVVM::BindingHelper::TryGetPropertyTypeForDestinationBinding(Property).HasValue() != bTryGetPropertyTypeForDestinationBinding)
		{
			AddError(FString::Printf(TEXT("TryGetPropertyTypeForDestinationBinding failed for '%s'."), *PropertyName.ToString()));
		}
	};

	TestProperty(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingHelperTest, PropertyA), false, false, false, false, false, false, false, false);
	TestProperty(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingHelperTest, PropertyB), true, true, true, true, false, false, true, true);
	TestProperty(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingHelperTest, PropertyC), true, false, true, false, false, false, true, false);
	TestProperty(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingHelperTest, PropertyD), true, true, true, true, true, true, true, true);
	TestProperty(GET_MEMBER_NAME_CHECKED(UMVVMViewModelBindingHelperTest, PropertyE), true, false, true, false, true, false, true, false);

	TestProperty("PropertyI", false, false, false, false, false, false, false, false);
	TestProperty("PropertyJ", true, true, true, true, false, false, true, true);
	TestProperty("PropertyK", true, false, true, false, false, false, true, false);
	TestProperty("PropertyL", true, true, true, true, true, true, true, true);
	TestProperty("PropertyM", true, false, true, false, true, false, true, false);

	TestProperty("PropertyX", false, false, false, false, false, false, false, false);


	auto TestFunction = [this](FName FunctionName
		, bool bIsValidForSourceBinding, bool bIsValidForDestinationBinding, bool bIsValidForSimpleRuntimeConversion, bool bIsValidForComplexRuntimeConversion
		, FFieldClass* PropertyTypeForSourceBinding, FFieldClass* PropertyTypeForDestinationBinding)
	{
		UFunction* Function = UMVVMViewModelBindingHelperTest::StaticClass()->FindFunctionByName(FunctionName);
		check(Function);

		if (::UE::MVVM::BindingHelper::IsValidForSourceBinding(Function) != bIsValidForSourceBinding)
		{
			AddError(FString::Printf(TEXT("IsValidForSourceBinding failed for '%s'."), *FunctionName.ToString()));
		}
		if (::UE::MVVM::BindingHelper::IsValidForDestinationBinding(Function) != bIsValidForDestinationBinding)
		{
			AddError(FString::Printf(TEXT("IsValidForDestinationBinding failed for '%s'."), *FunctionName.ToString()));
		}
		if (::UE::MVVM::BindingHelper::IsValidForSimpleRuntimeConversion(Function) != bIsValidForSimpleRuntimeConversion)
		{
			AddError(FString::Printf(TEXT("IsValidForSimpleRuntimeConversion failed for '%s'."), *FunctionName.ToString()));
		}
		if (::UE::MVVM::BindingHelper::IsValidForComplexRuntimeConversion(Function) != bIsValidForComplexRuntimeConversion)
		{
			AddError(FString::Printf(TEXT("IsValidForComplexRuntimeConversion failed for '%s'."), *FunctionName.ToString()));
		}

		{
			TValueOrError<const FProperty*, FText> Result = ::UE::MVVM::BindingHelper::TryGetPropertyTypeForSourceBinding(Function);
			if ((PropertyTypeForSourceBinding != nullptr) != Result.HasValue())
			{
				AddError(FString::Printf(TEXT("TryGetPropertyTypeForSourceBinding failed for '%s'."), *FunctionName.ToString()));
			}
			else if (PropertyTypeForSourceBinding && Result.GetValue()->GetClass() != PropertyTypeForSourceBinding)
			{
				AddError(FString::Printf(TEXT("TryGetPropertyTypeForSourceBinding failed for '%s'."), *FunctionName.ToString()));
			}
		}
		{
			TValueOrError<const FProperty*, FText> Result = ::UE::MVVM::BindingHelper::TryGetPropertyTypeForDestinationBinding(Function);
			if ((PropertyTypeForDestinationBinding != nullptr) != Result.HasValue())
			{
				AddError(FString::Printf(TEXT("TryGetPropertyTypeForDestinationBinding failed for '%s'."), *FunctionName.ToString()));
			}
			else if (PropertyTypeForDestinationBinding && Result.GetValue()->GetClass() != PropertyTypeForDestinationBinding)
			{
				AddError(FString::Printf(TEXT("TryGetPropertyTypeForDestinationBinding failed for '%s'."), *FunctionName.ToString()));
			}
		}
	};

	TestFunction("FunctionGetA", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionGetB", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionGetC", true, false, false, true, FIntProperty::StaticClass(), nullptr);
	TestFunction("FunctionGetD", false, false, true, false, nullptr, nullptr);
	TestFunction("FunctionGetE", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionGetF", true, false, false, true, FArrayProperty::StaticClass(), nullptr);
	TestFunction("FunctionGetG", true, false, false, true, FArrayProperty::StaticClass(), nullptr);
	TestFunction("FunctionGetH", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionGetI", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionGetJ", false, false, true, false, nullptr, nullptr);

	TestFunction("FunctionSetA", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionSetB", false, true, false, false, nullptr, FIntProperty::StaticClass());
	TestFunction("FunctionSetC", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionSetD", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionSetE", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionSetF", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionSetG", false, true, false, false, nullptr, FArrayProperty::StaticClass());
	TestFunction("FunctionSetH", false, false, false, false, nullptr, nullptr);

	TestFunction("FunctionGetProtected", true, false, false, true, FIntProperty::StaticClass(), nullptr);
	TestFunction("FunctionSetProtected", false, true, false, false, nullptr, FIntProperty::StaticClass());
	TestFunction("FunctionGetter", true, false, false, true, FIntProperty::StaticClass(), nullptr);
	TestFunction("FunctionSetter", false, true, false, false, nullptr, FIntProperty::StaticClass());

	TestFunction("FunctionConversionA", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionConversionB", false, false, false, true, nullptr, nullptr);
	TestFunction("FunctionConversionC", false, false, true, false, nullptr, nullptr);
	TestFunction("FunctionConversionD", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionConversionE", false, true, false, false, nullptr, FIntProperty::StaticClass());
	TestFunction("FunctionConversionF", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionConversionG", false, false, true, false, nullptr, nullptr);
	TestFunction("FunctionConversionH", false, false, true, false, nullptr, nullptr);
	TestFunction("FunctionConversionI", false, false, false, false, nullptr, nullptr);
	TestFunction("FunctionConversionJ", false, false, false, false, nullptr, nullptr);

	auto TestConversionFunction = [this](FName FunctionName, FFieldClass* ExpectedReturnProperty, FFieldClass* ExpectedFirstProperty)
	{
		UFunction* Function = UMVVMViewModelBindingHelperTest::StaticClass()->FindFunctionByName(FunctionName);
		check(Function);

		const FProperty* ReturnProperty = ::UE::MVVM::BindingHelper::GetReturnProperty(Function);
		if (ExpectedReturnProperty)
		{
			if (!ReturnProperty || ReturnProperty->GetClass() != ExpectedReturnProperty)
			{
				AddError(FString::Printf(TEXT("GetReturnProperty failed for '%s'."), *FunctionName.ToString()));
			}
		}
		else
		{
			if (ReturnProperty)
			{
				AddError(FString::Printf(TEXT("GetReturnProperty failed for '%s'."), *FunctionName.ToString()));
			}
		}

		const FProperty* FirstProperty = ::UE::MVVM::BindingHelper::GetFirstArgumentProperty(Function);
		if (ExpectedFirstProperty)
		{
			if (!FirstProperty || FirstProperty->GetClass() != ExpectedFirstProperty)
			{
				AddError(FString::Printf(TEXT("GetFirstArgumentProperty failed for '%s'."), *FunctionName.ToString()));
			}
		}
		else
		{
			if (FirstProperty)
			{
				AddError(FString::Printf(TEXT("GetFirstArgumentProperty failed for '%s'."), *FunctionName.ToString()));
			}
		}
	};

	TestConversionFunction("FunctionConversionA", nullptr, nullptr);
	TestConversionFunction("FunctionConversionB", FIntProperty::StaticClass(), nullptr);
	TestConversionFunction("FunctionConversionC", FIntProperty::StaticClass(), FIntProperty::StaticClass());
	TestConversionFunction("FunctionConversionD", FIntProperty::StaticClass(), FIntProperty::StaticClass());
	TestConversionFunction("FunctionConversionE", nullptr, FIntProperty::StaticClass());
	TestConversionFunction("FunctionConversionF", nullptr, FIntProperty::StaticClass());
	TestConversionFunction("FunctionConversionG", FArrayProperty::StaticClass(), FSetProperty::StaticClass());
	TestConversionFunction("FunctionConversionH", FArrayProperty::StaticClass(), FSetProperty::StaticClass());
	TestConversionFunction("FunctionConversionI", nullptr, FSetProperty::StaticClass());
	TestConversionFunction("FunctionConversionJ", nullptr, FMapProperty::StaticClass());

	return true;
}

#undef LOCTEXT_NAMESPACE 
#endif //WITH_AUTOMATION_WORKER
