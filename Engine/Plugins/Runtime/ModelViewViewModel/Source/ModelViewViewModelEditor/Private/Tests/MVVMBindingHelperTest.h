// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMViewModelBase.h"


#include "MVVMBindingHelperTest.generated.h"

UCLASS(MinimalAPI, Transient, hidedropdown, Abstract, Hidden)
class UMVVMViewModelBindingHelperTest : public UMVVMViewModelBase
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 PropertyA; //~No, No

	UPROPERTY(BlueprintReadWrite, Category = "MVVM")
	int32 PropertyB; //~Yes, Yes

	UPROPERTY(BlueprintReadOnly, Category = "MVVM")
	int32 PropertyC; //~Yes, No

	UPROPERTY(BlueprintReadWrite, Category = "MVVM", BlueprintGetter = "FunctionGetter", BlueprintSetter = "FunctionSetter")
	int32 PropertyD; //~Yes, Yes

	UPROPERTY(BlueprintReadOnly, Category = "MVVM", BlueprintGetter = "FunctionGetter")
	int32 PropertyE; //~Yes, No

protected:
	UPROPERTY()
	int32 PropertyI; //~No, No

	UPROPERTY(BlueprintReadWrite, Category = "MVVM")
	int32 PropertyJ; //~Yes, Yes

	UPROPERTY(BlueprintReadOnly, Category = "MVVM")
	int32 PropertyK; //~Yes, No

	UPROPERTY(BlueprintReadWrite, Category = "MVVM", BlueprintGetter = "FunctionGetter", BlueprintSetter = "FunctionSetter")
	int32 PropertyL; //~Yes, Yes

	UPROPERTY(BlueprintReadOnly, Category = "MVVM", BlueprintGetter = "FunctionGetter")
	int32 PropertyM; //~Yes, No

private:
	UPROPERTY()
	int32 PropertyX; //~No, No

public:
	UFUNCTION()
	int32 FunctionGetA() const
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	int32 FunctionGetB()
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	int32 FunctionGetC() const
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	int32 FunctionGetD(int32 Param1) const
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	int32 FunctionGetE(int32 Param1, int32 Param2) const
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	TArray<int32> FunctionGetF() const
	{
		return TArray<int32>();
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void FunctionGetG(TArray<int32>& Result) const
	{
		Result = TArray<int32>();
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void FunctionGetH(const TArray<int32>& Result) const
	{
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	TArray<int32> FunctionGetI(TArray<int32>& Result) const
	{
		Result = TArray<int32>();
		return Result;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void FunctionGetJ(TArray<int32>& Result, const TArray<int32>& Value) const
	{
		Result = Value;
	}

public:
	UFUNCTION()
	void FunctionSetA(int32 Value)
	{
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void FunctionSetB(int32 Value)
	{
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	int32 FunctionSetC(int32 Param1)
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void FunctionSetD(int32 Param1) const
	{
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void FunctionSetE(int32 Param1, int32 Param2) const
	{
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void FunctionSetF(TArray<int32>& Param1)
	{
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void FunctionSetG(const TArray<int32>& Param1)
	{
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void FunctionSetH(TArray<int32>& Param1, const TArray<int32>& Param2)
	{
	}

public:
	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static void FunctionConversionA()
	{
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static int32 FunctionConversionB()
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static int32 FunctionConversionC(int32 Param1)
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static int32 FunctionConversionD(int32 Param1, int32 Param2)
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static void FunctionConversionE(int32 Param1)
	{
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static void FunctionConversionF(int32 Param1, int32 Param2)
	{
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static void FunctionConversionG(TArray<int32>& Param1, const TSet<int32>& Param2)
	{
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static TArray<int32> FunctionConversionH(const TSet<int32>& Param1)
	{
		return TArray<int32>();
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static TArray<int32> FunctionConversionI(const TSet<int32>& Param1, TMap<int32, int32>& Param2)
	{
		return TArray<int32>();
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	static TArray<int32> FunctionConversionJ(TSet<int32>& Param1, const TMap<int32, int32>& Param2)
	{
		return TArray<int32>();
	}

protected:
	UFUNCTION(BlueprintCallable, Category = "MVVM")
	int32 FunctionGetProtected() const
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void FunctionSetProtected(int32 Value)
	{
	}

private:
	UFUNCTION(BlueprintCallable, Category = "MVVM")
	int32 FunctionGetter() const
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, Category = "MVVM")
	void FunctionSetter(int32 Value)
	{
	}
};
