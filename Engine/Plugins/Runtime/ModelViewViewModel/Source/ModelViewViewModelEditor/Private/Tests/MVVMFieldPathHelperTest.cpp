// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/MVVMFieldPathHelperTest.h"

#include "Bindings/MVVMFieldPathHelper.h"
#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "Misc/AutomationTest.h"
#include "Templates/ValueOrError.h"
#include "Types/MVVMFieldVariant.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMFieldPathHelperTest)

#if WITH_AUTOMATION_WORKER

#define LOCTEXT_NAMESPACE "MVVMBindingHelperTest"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMVVMFieldPathHelperTestGenerateFieldPathList, "System.Plugins.MVVM.GetNotifyBindingInfoFromFieldPath", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FMVVMFieldPathHelperTestGenerateFieldPathList::RunTest(const FString& Parameters)
{
	using namespace UE::MVVM;

	auto CompareResult = [this](const FieldPathHelper::FParsedNotifyBindingInfo& Result, const FieldPathHelper::FParsedNotifyBindingInfo& Expected, FStringView FieldPath)
	{
		if (Result.NotifyFieldClass != Expected.NotifyFieldClass)
		{
			AddError(FString::Printf(TEXT("Wrong NotifyFieldClass for '%s'"), FieldPath.GetData()));
		}
		if (Result.NotifyFieldId != Expected.NotifyFieldId)
		{
			AddError(FString::Printf(TEXT("Wrong NotifyFieldId for '%s'"), FieldPath.GetData()));
		}
		if (Result.ViewModelIndex != Expected.ViewModelIndex)
		{
			AddError(FString::Printf(TEXT("Wrong NotifyFieldId for '%s'"), FieldPath.GetData()));
		}
	};

	auto Test = [this, &CompareResult](UClass* Accessor, FStringView FieldPath, TOptional<FieldPathHelper::FParsedNotifyBindingInfo> ExpectedResult)
	{
		TValueOrError<TArray<FMVVMConstFieldVariant>, FText> Fields = FieldPathHelper::GenerateFieldPathList(UMVVMWidgetFieldPathHelperTest::StaticClass(), FieldPath, true);
		if (Fields.HasError())
		{
			AddError(FString::Printf(TEXT("The path for '%s' could not be generated."), FieldPath.GetData()));
		}
		else
		{
			TValueOrError<FieldPathHelper::FParsedNotifyBindingInfo, FText> BindingInfo = FieldPathHelper::GetNotifyBindingInfoFromFieldPath(UMVVMWidgetFieldPathHelperTest::StaticClass(), Fields.GetValue());
			if (ExpectedResult.IsSet() && BindingInfo.HasError())
			{
				AddError(FString::Printf(TEXT("GetNotifyBindingInfoFromFieldPath failed for '%s'. %s."), FieldPath.GetData(), *BindingInfo.GetError().ToString()));
			}
			else if (!ExpectedResult.IsSet() && BindingInfo.HasValue())
			{
				AddError(FString::Printf(TEXT("GetNotifyBindingInfoFromFieldPath succeeed when it should had failed for '%s'"), FieldPath.GetData()));
			}
			else if (ExpectedResult.IsSet() && BindingInfo.HasValue())
			{
				CompareResult(ExpectedResult.GetValue(), BindingInfo.GetValue(), FieldPath);
			}
		}
	};


	//Always from the UserWidget point of view
	{
		FieldPathHelper::FParsedNotifyBindingInfo ExpectedInvalidInfo;
		TOptional<FieldPathHelper::FParsedNotifyBindingInfo> ExpectedErrorInfo;

		ExpectedInvalidInfo.NotifyFieldClass = nullptr;
		ExpectedInvalidInfo.NotifyFieldId = FFieldNotificationId();
		ExpectedInvalidInfo.ViewModelIndex = INDEX_NONE;

		auto BuildInfo = [](UClass* Class, FName Notification, int32 Index)
		{
			FieldPathHelper::FParsedNotifyBindingInfo Result;
			Result.NotifyFieldClass = Class;
			Result.NotifyFieldId = FFieldNotificationId(Notification);
			Result.ViewModelIndex = Index;
			return Result;
		};

		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyIntNotify"), BuildInfo(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyIntNotify"), -1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), BuildInfo(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), -1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify.X"), BuildInfo(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), -1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyObject"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyObject.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyObject.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyObject.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyObject.PropertyObject"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyObject.PropertyObject.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyObject.PropertyObject.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyObject.PropertyObject.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyObject.PropertyViewModel"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyIntNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyVectorNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyVectorNotify.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyObject"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyViewModel.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyViewModel.PropertyIntNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyViewModel.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyViewModel.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyViewModel.PropertyVectorNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyStruct.PropertyViewModel.PropertyViewModel.PropertyVectorNotify.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyObject"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyObject.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyObject.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyObject.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyObject.PropertyObject"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyObject.PropertyViewModel"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyIntNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyVectorNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyVectorNotify.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyObject"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyViewModel.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyViewModel.PropertyIntNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyViewModel.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyViewModel.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyViewModel.PropertyVectorNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyObject.PropertyViewModel.PropertyViewModel.PropertyVectorNotify.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyIntNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyIntNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyVectorNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyVectorNotify.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyStructNotify.PropertyInt"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyStructNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyStructNotify.PropertyVector"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyStructNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyStructNotify.PropertyVector.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyStructNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyStructNotify.PropertyObject"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyStructNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyStructNotify.PropertyObject.PropertyVector.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyStructNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyStructNotify.PropertyViewModel"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyStructNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject.PropertyObject"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject.PropertyViewModel"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject.PropertyViewModel.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject.PropertyViewModel.PropertyIntNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject.PropertyViewModel.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject.PropertyViewModel.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject.PropertyViewModel.PropertyVectorNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyObject.PropertyViewModel.PropertyVectorNotify.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyIntNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyVectorNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyVectorNotify.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyObject"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyViewModel.PropertyInt"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyViewModel.PropertyIntNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyViewModel.PropertyVector"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyViewModel.PropertyVector.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyViewModel.PropertyVectorNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModel.PropertyViewModel.PropertyVectorNotify.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyInt"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyIntNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyIntNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyVector"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyVector.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyVectorNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyVectorNotify.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyObject"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModel.PropertyInt"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModel.PropertyIntNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModel.PropertyVector"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModel.PropertyVector.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModel.PropertyVectorNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModel.PropertyVectorNotify.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModelNotify.PropertyInt"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModelNotify.PropertyIntNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyIntNotify"), 2));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModelNotify.PropertyVector"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModelNotify.PropertyVector.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModelNotify.PropertyVectorNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), 2));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModel.PropertyViewModelNotify.PropertyViewModelNotify.PropertyVectorNotify.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), 2));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyInt"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyIntNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyIntNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyVector"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyVector.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyVectorNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyVectorNotify.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyObject"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModel.PropertyInt"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModel.PropertyIntNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModel.PropertyVector"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModel.PropertyVector.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 0));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModel.PropertyVectorNotify"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModel.PropertyVectorNotify.X"), ExpectedInvalidInfo);
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModelNotify.PropertyInt"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModelNotify.PropertyIntNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyIntNotify"), 2));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModelNotify.PropertyVector"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModelNotify.PropertyVector.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify"), 1));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModelNotify.PropertyVectorNotify"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), 2));
		Test(UMVVMWidgetFieldPathHelperTest::StaticClass(), TEXT("PropertyViewModelNotify.PropertyViewModelNotify.PropertyViewModelNotify.PropertyVectorNotify.X"), BuildInfo(UMVVMViewModelFieldPathHelperTest::StaticClass(), TEXT("PropertyVectorNotify"), 2));
	}

	return true;
}

#undef LOCTEXT_NAMESPACE 
#endif //WITH_AUTOMATION_WORKER
