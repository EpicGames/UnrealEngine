// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Blueprint/UserWidget.h"
#include "MVVMViewModelBase.h"

#include "MVVMFieldPathHelperTest.generated.h"

class UMVVMObjectFieldPathHelperTest;
class UMVVMViewModelFieldPathHelperTest;
class UMVVMWidgetFieldPathHelperTest;

/** */
USTRUCT(BlueprintInternalUseOnly)
struct FMVVMStructFieldPathHelperTest
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 PropertyInt = 0;

	UPROPERTY()
	FVector PropertyVector = FVector::ZeroVector;

	UPROPERTY()
	TObjectPtr<UMVVMObjectFieldPathHelperTest> PropertyObject;

	UPROPERTY()
	TObjectPtr<UMVVMViewModelFieldPathHelperTest> PropertyViewModel;
};

/** */
UCLASS(MinimalAPI, Transient, hidedropdown, Abstract, Hidden)
class UMVVMObjectFieldPathHelperTest : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 PropertyInt;
	UFUNCTION()
	int32 GetPropertyInt() const { return 0; }
	UFUNCTION()
	void SetPropertyInt(int32 Value) { }

	UPROPERTY(Getter, Setter)
	int32 PropertyIntWithGetterSetter;
	UFUNCTION()
	int32 GetPropertyIntWithGetterSetter() const { return 0; }
	UFUNCTION()
	void SetPropertyIntWithGetterSetter(int32 Value) {}

	UPROPERTY(Getter, Setter, BlueprintGetter="GetPropertyIntWithGetterSetterAndBP", BlueprintSetter="SetPropertyIntWithGetterSetterAndBP", Category="Test")
	int32 PropertyIntWithGetterSetterAndBP;
	UFUNCTION(BlueprintPure, Category = "Test")
	int32 GetPropertyIntWithGetterSetterAndBP() const { return 0;}
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyIntWithGetterSetterAndBP(int32 Value){}

	UPROPERTY(BlueprintGetter="GetPropertyIntWithBPGetterSetter", BlueprintSetter="SetPropertyIntWithBPGetterSetter", Category = "Test")
	int32 PropertyIntWithBPGetterSetter;
	UFUNCTION(BlueprintPure, Category = "Test")
	int32 GetPropertyIntWithBPGetterSetter() const { return 0; }
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyIntWithBPGetterSetter(int32 Value) { }

public:
	UPROPERTY()
	FVector PropertyVector;
	UFUNCTION()
	FVector GetPropertyVector() const { return FVector(); }
	UFUNCTION()
	void SetPropertyVector(FVector Value) { }

	UPROPERTY(Getter, Setter)
	FVector PropertyVectorWithGetterSetter;
	UFUNCTION()
	FVector GetPropertyVectorWithGetterSetter() const { return FVector(); }
	UFUNCTION()
	void SetPropertyVectorWithGetterSetter(FVector Value) {}

	UPROPERTY(Getter, Setter, BlueprintGetter="GetPropertyVectorWithGetterSetterAndBP", BlueprintSetter="SetPropertyVectorWithGetterSetterAndBP", Category = "Test")
	FVector PropertyVectorWithGetterSetterAndBP;
	UFUNCTION(BlueprintPure, Category = "Test")
	FVector GetPropertyVectorWithGetterSetterAndBP() const { return FVector();}
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyVectorWithGetterSetterAndBP(FVector Value){}

	UPROPERTY(BlueprintGetter="GetPropertyVectorWithBPGetterSetter", BlueprintSetter="SetPropertyVectorWithBPGetterSetter", Category = "Test")
	FVector PropertyVectorWithBPGetterSetter;
	UFUNCTION(BlueprintPure, Category = "Test")
	FVector GetPropertyVectorWithBPGetterSetter() const { return FVector(); }
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyVectorWithBPGetterSetter(FVector Value) { }

public:
	UPROPERTY()
	FMVVMStructFieldPathHelperTest PropertyStruct;
	UFUNCTION()
	FMVVMStructFieldPathHelperTest GetPropertyStruct() const { return FMVVMStructFieldPathHelperTest(); }
	UFUNCTION()
	void SetPropertyStruct(FMVVMStructFieldPathHelperTest Value) { }

	UPROPERTY(Getter, Setter)
	FMVVMStructFieldPathHelperTest PropertyStructWithGetterSetter;
	UFUNCTION()
	FMVVMStructFieldPathHelperTest GetPropertyStructWithGetterSetter() const { return FMVVMStructFieldPathHelperTest(); }
	UFUNCTION()
	void SetPropertyStructWithGetterSetter(FMVVMStructFieldPathHelperTest Value) {}

	UPROPERTY(BlueprintReadWrite, Getter, Setter, BlueprintGetter="GetPropertyStructWithGetterSetterAndBP", BlueprintSetter="SetPropertyStructWithGetterSetterAndBP", Category = "Test")
	FMVVMStructFieldPathHelperTest PropertyStructWithGetterSetterAndBP;
	UFUNCTION(BlueprintPure, Category = "Test")
	FMVVMStructFieldPathHelperTest GetPropertyStructWithGetterSetterAndBP() const { return FMVVMStructFieldPathHelperTest();}
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyStructWithGetterSetterAndBP(FMVVMStructFieldPathHelperTest Value){}

	UPROPERTY(BlueprintReadWrite, BlueprintGetter="GetPropertyStructWithBPGetterSetter", BlueprintSetter="SetPropertyStructWithBPGetterSetter", Category = "Test")
	FMVVMStructFieldPathHelperTest PropertyStructWithBPGetterSetter;
	UFUNCTION(BlueprintPure, Category = "Test")
	FMVVMStructFieldPathHelperTest GetPropertyStructWithBPGetterSetter() const { return FMVVMStructFieldPathHelperTest(); }
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyStructWithBPGetterSetter(FMVVMStructFieldPathHelperTest Value) { }

public:
	UPROPERTY()
	TObjectPtr<UMVVMObjectFieldPathHelperTest> PropertyObject;
	UFUNCTION()
	UMVVMObjectFieldPathHelperTest* GetPropertyObject() const { return nullptr; }
	UFUNCTION()
	void SetPropertyObject(UMVVMObjectFieldPathHelperTest* Value) { }

	UPROPERTY(Getter, Setter)
	TObjectPtr<UMVVMObjectFieldPathHelperTest> PropertyObjectWithGetterSetter;
	UFUNCTION()
	UMVVMObjectFieldPathHelperTest* GetPropertyObjectWithGetterSetter() const { return nullptr; }
	UFUNCTION()
	void SetPropertyObjectWithGetterSetter(UMVVMObjectFieldPathHelperTest* Value) {}

	UPROPERTY(BlueprintReadWrite, Getter, Setter, BlueprintGetter="GetPropertyObjectWithGetterSetterAndBP", BlueprintSetter="SetPropertyObjectWithGetterSetterAndBP", Category = "Test")
	TObjectPtr<UMVVMObjectFieldPathHelperTest> PropertyObjectWithGetterSetterAndBP;
	UFUNCTION(BlueprintPure, Category = "Test")
	UMVVMObjectFieldPathHelperTest* GetPropertyObjectWithGetterSetterAndBP() const { return nullptr;}
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyObjectWithGetterSetterAndBP(UMVVMObjectFieldPathHelperTest* Value){}

	UPROPERTY(BlueprintReadWrite, BlueprintGetter="GetPropertyObjectWithBPGetterSetter", BlueprintSetter="SetPropertyObjectWithBPGetterSetter", Category = "Test")
	TObjectPtr<UMVVMObjectFieldPathHelperTest> PropertyObjectWithBPGetterSetter;
	UFUNCTION(BlueprintPure, Category = "Test")
	UMVVMObjectFieldPathHelperTest* GetPropertyObjectWithBPGetterSetter() const { return nullptr; }
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyObjectWithBPGetterSetter(UMVVMObjectFieldPathHelperTest* Value) { }

public:
	UPROPERTY()
	TObjectPtr<UMVVMViewModelFieldPathHelperTest> PropertyViewModel;
	UFUNCTION()
	UMVVMViewModelFieldPathHelperTest* GetPropertyViewModel() const { return nullptr; }
	UFUNCTION()
	void SetPropertyViewModel(UMVVMViewModelFieldPathHelperTest* Value) { }

	UPROPERTY(Getter, Setter)
	TObjectPtr<UMVVMViewModelFieldPathHelperTest> PropertyViewModelWithGetterSetter;
	UFUNCTION()
	UMVVMViewModelFieldPathHelperTest* GetPropertyViewModelWithGetterSetter() const { return nullptr; }
	UFUNCTION()
	void SetPropertyViewModelWithGetterSetter(UMVVMViewModelFieldPathHelperTest* Value) {}

	UPROPERTY(BlueprintReadWrite, Getter, Setter, BlueprintGetter="GetPropertyViewModelWithGetterSetterAndBP", BlueprintSetter="SetPropertyViewModelWithGetterSetterAndBP", Category = "Test")
	TObjectPtr<UMVVMViewModelFieldPathHelperTest> PropertyViewModelWithGetterSetterAndBP;
	UFUNCTION(BlueprintPure, Category = "Test")
	UMVVMViewModelFieldPathHelperTest* GetPropertyViewModelWithGetterSetterAndBP() const { return nullptr;}
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyViewModelWithGetterSetterAndBP(UMVVMViewModelFieldPathHelperTest* Value){}

	UPROPERTY(BlueprintReadWrite, BlueprintGetter="GetPropertyViewModelWithBPGetterSetter", BlueprintSetter="SetPropertyViewModelWithBPGetterSetter", Category = "Test")
	TObjectPtr<UMVVMViewModelFieldPathHelperTest> PropertyViewModelWithBPGetterSetter;
	UFUNCTION(BlueprintPure, Category = "Test")
	UMVVMViewModelFieldPathHelperTest* GetPropertyViewModelWithBPGetterSetter() const { return nullptr; }
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyViewModelWithBPGetterSetter(UMVVMViewModelFieldPathHelperTest* Value) { }
};

/** */
UCLASS(MinimalAPI, Transient, hidedropdown, Abstract, Hidden)
class UMVVMViewModelFieldPathHelperTestBase : public UMVVMObjectFieldPathHelperTest, public INotifyFieldValueChanged
{
	GENERATED_BODY()

public:
	struct FFieldNotificationClassDescriptor : public ::UE::FieldNotification::IClassDescriptor
	{
	};

	//~ Begin INotifyFieldValueChanged Interface
	virtual FDelegateHandle AddFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FFieldValueChangedDelegate InNewDelegate) override final { return FDelegateHandle(); }
	virtual bool RemoveFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FDelegateHandle InHandle) override final { return false; }
	virtual int32 RemoveAllFieldValueChangedDelegates(FDelegateUserObjectConst InUserObject) override final { return 0; }
	virtual int32 RemoveAllFieldValueChangedDelegates(UE::FieldNotification::FFieldId InFieldId, FDelegateUserObjectConst InUserObject) override final { return 0; }
	virtual const UE::FieldNotification::IClassDescriptor& GetFieldNotificationDescriptor() const override { static FFieldNotificationClassDescriptor Tmp; return Tmp; }
	virtual void BroadcastFieldValueChanged(UE::FieldNotification::FFieldId InFieldId) override {}
	//~ End INotifyFieldValueChanged Interface
};

/** */
UCLASS(MinimalAPI, Transient, hidedropdown, Abstract, Hidden)
class UMVVMViewModelFieldPathHelperTest : public UMVVMViewModelFieldPathHelperTestBase
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, FieldNotify, Category = "Test")
	int32 PropertyIntNotify;
	UFUNCTION()
	int32 GetPropertyIntNotify() const { return 0; }
	UFUNCTION()
	void SetPropertyIntNotify(int32 Value) { }

public:
	UPROPERTY(BlueprintReadWrite, FieldNotify, Category = "Test")
	FVector PropertyVectorNotify;
	UFUNCTION()
	FVector GetPropertyVectorNotify() const { return FVector(); }
	UFUNCTION()
	void SetPropertyVectorNotify(FVector Value) { }

public:
	UPROPERTY(BlueprintReadWrite, FieldNotify, Category = "Test")
	FMVVMStructFieldPathHelperTest PropertyStructNotify;
	UFUNCTION()
	FMVVMStructFieldPathHelperTest GetPropertyStructNotify() const { return FMVVMStructFieldPathHelperTest(); }
	UFUNCTION()
	void SetPropertyStructNotify(FMVVMStructFieldPathHelperTest Value) { }

public:
	UPROPERTY(BlueprintReadWrite, FieldNotify, Category = "Test")
	TObjectPtr<UMVVMObjectFieldPathHelperTest> PropertyObjectNotify;
	UFUNCTION()
	UMVVMObjectFieldPathHelperTest* GetPropertyObjectNotify() const { return nullptr; }
	UFUNCTION()
	void SetPropertyObjectNotify(UMVVMObjectFieldPathHelperTest* Value) { }

public:
	UPROPERTY(BlueprintReadWrite, FieldNotify, Category = "Test")
	TObjectPtr<UMVVMViewModelFieldPathHelperTest> PropertyViewModelNotify;
	UFUNCTION()
	UMVVMViewModelFieldPathHelperTest* GetPropertyViewModelNotify() const { return nullptr; }
	UFUNCTION()
	void SetPropertyViewModelNotify(UMVVMViewModelFieldPathHelperTest* Value) { }
};

/** */
UCLASS(MinimalAPI, Transient, hidedropdown, Abstract, Hidden)
class UMVVMWidgetFieldPathHelperTest : public UUserWidget
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 PropertyInt;
	UFUNCTION()
	int32 GetPropertyInt() const { return 0; }
	UFUNCTION()
	void SetPropertyInt(int32 Value) { }

	UPROPERTY(Getter, Setter)
	int32 PropertyIntWithGetterSetter;
	UFUNCTION()
	int32 GetPropertyIntWithGetterSetter() const { return 0; }
	UFUNCTION()
	void SetPropertyIntWithGetterSetter(int32 Value) {}

	UPROPERTY(Getter, Setter, BlueprintGetter="GetPropertyIntWithGetterSetterAndBP", BlueprintSetter="SetPropertyIntWithGetterSetterAndBP", Category="Test")
	int32 PropertyIntWithGetterSetterAndBP;
	UFUNCTION(BlueprintPure, Category = "Test")
	int32 GetPropertyIntWithGetterSetterAndBP() const { return 0;}
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyIntWithGetterSetterAndBP(int32 Value){}

	UPROPERTY(BlueprintGetter="GetPropertyIntWithBPGetterSetter", BlueprintSetter="SetPropertyIntWithBPGetterSetter", Category = "Test")
	int32 PropertyIntWithBPGetterSetter;
	UFUNCTION(BlueprintPure, Category = "Test")
	int32 GetPropertyIntWithBPGetterSetter() const { return 0; }
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyIntWithBPGetterSetter(int32 Value) { }

	UPROPERTY(BlueprintReadWrite, FieldNotify, Category = "Test")
	int32 PropertyIntNotify;
	UFUNCTION()
	int32 GetPropertyIntNotify() const { return 0; }
	UFUNCTION()
	void SetPropertyIntNotify(int32 Value) { }

public:
	UPROPERTY()
	FVector PropertyVector;
	UFUNCTION()
	FVector GetPropertyVector() const { return FVector(); }
	UFUNCTION()
	void SetPropertyVector(FVector Value) { }

	UPROPERTY(Getter, Setter)
	FVector PropertyVectorWithGetterSetter;
	UFUNCTION()
	FVector GetPropertyVectorWithGetterSetter() const { return FVector(); }
	UFUNCTION()
	void SetPropertyVectorWithGetterSetter(FVector Value) {}

	UPROPERTY(Getter, Setter, BlueprintGetter="GetPropertyVectorWithGetterSetterAndBP", BlueprintSetter="SetPropertyVectorWithGetterSetterAndBP", Category = "Test")
	FVector PropertyVectorWithGetterSetterAndBP;
	UFUNCTION(BlueprintPure, Category = "Test")
	FVector GetPropertyVectorWithGetterSetterAndBP() const { return FVector();}
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyVectorWithGetterSetterAndBP(FVector Value){}

	UPROPERTY(BlueprintGetter="GetPropertyVectorWithBPGetterSetter", BlueprintSetter="SetPropertyVectorWithBPGetterSetter", Category = "Test")
	FVector PropertyVectorWithBPGetterSetter;
	UFUNCTION(BlueprintPure, Category = "Test")
	FVector GetPropertyVectorWithBPGetterSetter() const { return FVector(); }
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyVectorWithBPGetterSetter(FVector Value) { }

	UPROPERTY(BlueprintReadWrite, FieldNotify, Category = "Test")
	FVector PropertyVectorNotify;
	UFUNCTION()
	FVector GetPropertyVectorNotify() const { return FVector(); }
	UFUNCTION()
	void SetPropertyVectorNotify(FVector Value) { }

public:
	UPROPERTY()
	FMVVMStructFieldPathHelperTest PropertyStruct;
	UFUNCTION()
	FMVVMStructFieldPathHelperTest GetPropertyStruct() const { return FMVVMStructFieldPathHelperTest(); }
	UFUNCTION()
	void SetPropertyStruct(FMVVMStructFieldPathHelperTest Value) { }

	UPROPERTY(Getter, Setter)
	FMVVMStructFieldPathHelperTest PropertyStructWithGetterSetter;
	UFUNCTION()
	FMVVMStructFieldPathHelperTest GetPropertyStructWithGetterSetter() const { return FMVVMStructFieldPathHelperTest(); }
	UFUNCTION()
	void SetPropertyStructWithGetterSetter(FMVVMStructFieldPathHelperTest Value) {}

	UPROPERTY(BlueprintReadWrite, Getter, Setter, BlueprintGetter="GetPropertyStructWithGetterSetterAndBP", BlueprintSetter="SetPropertyStructWithGetterSetterAndBP", Category = "Test")
	FMVVMStructFieldPathHelperTest PropertyStructWithGetterSetterAndBP;
	UFUNCTION(BlueprintPure, Category = "Test")
	FMVVMStructFieldPathHelperTest GetPropertyStructWithGetterSetterAndBP() const { return FMVVMStructFieldPathHelperTest();}
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyStructWithGetterSetterAndBP(FMVVMStructFieldPathHelperTest Value){}

	UPROPERTY(BlueprintReadWrite, BlueprintGetter="GetPropertyStructWithBPGetterSetter", BlueprintSetter="SetPropertyStructWithBPGetterSetter", Category = "Test")
	FMVVMStructFieldPathHelperTest PropertyStructWithBPGetterSetter;
	UFUNCTION(BlueprintPure, Category = "Test")
	FMVVMStructFieldPathHelperTest GetPropertyStructWithBPGetterSetter() const { return FMVVMStructFieldPathHelperTest(); }
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyStructWithBPGetterSetter(FMVVMStructFieldPathHelperTest Value) { }

	UPROPERTY(BlueprintReadWrite, FieldNotify, Category = "Test")
	FMVVMStructFieldPathHelperTest PropertyStructNotify;
	UFUNCTION()
	FMVVMStructFieldPathHelperTest GetPropertyStructNotify() const { return FMVVMStructFieldPathHelperTest(); }
	UFUNCTION()
	void SetPropertyStructNotify(FMVVMStructFieldPathHelperTest Value) { }

public:
	UPROPERTY()
	TObjectPtr<UMVVMObjectFieldPathHelperTest> PropertyObject;
	UFUNCTION()
	UMVVMObjectFieldPathHelperTest* GetPropertyObject() const { return nullptr; }
	UFUNCTION()
	void SetPropertyObject(UMVVMObjectFieldPathHelperTest* Value) { }

	UPROPERTY(Getter, Setter)
	TObjectPtr<UMVVMObjectFieldPathHelperTest> PropertyObjectWithGetterSetter;
	UFUNCTION()
	UMVVMObjectFieldPathHelperTest* GetPropertyObjectWithGetterSetter() const { return nullptr; }
	UFUNCTION()
	void SetPropertyObjectWithGetterSetter(UMVVMObjectFieldPathHelperTest* Value) {}

	UPROPERTY(BlueprintReadWrite, Getter, Setter, BlueprintGetter="GetPropertyObjectWithGetterSetterAndBP", BlueprintSetter="SetPropertyObjectWithGetterSetterAndBP", Category = "Test")
	TObjectPtr<UMVVMObjectFieldPathHelperTest> PropertyObjectWithGetterSetterAndBP;
	UFUNCTION(BlueprintPure, Category = "Test")
	UMVVMObjectFieldPathHelperTest* GetPropertyObjectWithGetterSetterAndBP() const { return nullptr;}
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyObjectWithGetterSetterAndBP(UMVVMObjectFieldPathHelperTest* Value){}

	UPROPERTY(BlueprintReadWrite, BlueprintGetter="GetPropertyObjectWithBPGetterSetter", BlueprintSetter="SetPropertyObjectWithBPGetterSetter", Category = "Test")
	TObjectPtr<UMVVMObjectFieldPathHelperTest> PropertyObjectWithBPGetterSetter;
	UFUNCTION(BlueprintPure, Category = "Test")
	UMVVMObjectFieldPathHelperTest* GetPropertyObjectWithBPGetterSetter() const { return nullptr; }
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyObjectWithBPGetterSetter(UMVVMObjectFieldPathHelperTest* Value) { }

	UPROPERTY(BlueprintReadWrite, FieldNotify, Category = "Test")
	TObjectPtr<UMVVMObjectFieldPathHelperTest> PropertyObjectNotify;
	UFUNCTION()
	UMVVMObjectFieldPathHelperTest* GetPropertyObjectNotify() const { return nullptr; }
	UFUNCTION()
	void SetPropertyObjectNotify(UMVVMObjectFieldPathHelperTest* Value) { }

public:
	UPROPERTY()
	TObjectPtr<UMVVMViewModelFieldPathHelperTest> PropertyViewModel;
	UFUNCTION()
	UMVVMViewModelFieldPathHelperTest* GetPropertyViewModel() const { return nullptr; }
	UFUNCTION()
	void SetPropertyViewModel(UMVVMViewModelFieldPathHelperTest* Value) { }

	UPROPERTY(Getter, Setter)
	TObjectPtr<UMVVMViewModelFieldPathHelperTest> PropertyViewModelWithGetterSetter;
	UFUNCTION()
	UMVVMViewModelFieldPathHelperTest* GetPropertyViewModelWithGetterSetter() const { return nullptr; }
	UFUNCTION()
	void SetPropertyViewModelWithGetterSetter(UMVVMViewModelFieldPathHelperTest* Value) {}

	UPROPERTY(BlueprintReadWrite, Getter, Setter, BlueprintGetter="GetPropertyViewModelWithGetterSetterAndBP", BlueprintSetter="SetPropertyViewModelWithGetterSetterAndBP", Category = "Test")
	TObjectPtr<UMVVMViewModelFieldPathHelperTest> PropertyViewModelWithGetterSetterAndBP;
	UFUNCTION(BlueprintPure, Category = "Test")
	UMVVMViewModelFieldPathHelperTest* GetPropertyViewModelWithGetterSetterAndBP() const { return nullptr;}
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyViewModelWithGetterSetterAndBP(UMVVMViewModelFieldPathHelperTest* Value){}

	UPROPERTY(BlueprintReadWrite, BlueprintGetter="GetPropertyViewModelWithBPGetterSetter", BlueprintSetter="SetPropertyViewModelWithBPGetterSetter", Category = "Test")
	TObjectPtr<UMVVMViewModelFieldPathHelperTest> PropertyViewModelWithBPGetterSetter;
	UFUNCTION(BlueprintPure, Category = "Test")
	UMVVMViewModelFieldPathHelperTest* GetPropertyViewModelWithBPGetterSetter() const { return nullptr; }
	UFUNCTION(BlueprintCallable, Category = "Test")
	void SetPropertyViewModelWithBPGetterSetter(UMVVMViewModelFieldPathHelperTest* Value) { }

	UPROPERTY(BlueprintReadWrite, FieldNotify, Category = "Test")
	TObjectPtr<UMVVMViewModelFieldPathHelperTest> PropertyViewModelNotify;
	UFUNCTION()
	UMVVMViewModelFieldPathHelperTest* GetPropertyViewModelNotify() const { return nullptr; }
	UFUNCTION()
	void SetPropertyViewModelNotify(UMVVMViewModelFieldPathHelperTest* Value) { }
};
