// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/MVVMFieldValueChangedTest.h"
#include "Tests/MVVMFieldPathHelperTest.h"
#include "Misc/AutomationTest.h"

#include "UObject/GCObjectScopeGuard.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMFieldValueChangedTest)


#if WITH_AUTOMATION_WORKER

#define LOCTEXT_NAMESPACE "MVVMBindingExecuteTest"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMVVMFieldValueChangedTest, "System.Plugins.MVVM.FieldValueChanged", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)


bool FMVVMFieldValueChangedTest::RunTest(const FString& Parameters)
{
	UMVVMFieldValueChangedTest* SourceObj = NewObject<UMVVMFieldValueChangedTest>();
	TGCObjectScopeGuard<UObject> ScopeObject = TGCObjectScopeGuard<UObject>(SourceObj);

	struct FCallbackHandler
	{
		void OnPropertyIntChanged(UObject* InObject, UE::FieldNotification::FFieldId InFieldId)
		{
			if (InObject != SourceObj)
			{
				Test->AddError(TEXT("Wrong SourceObject in OnPropertyIntChanged"));
			}
			if (InFieldId != UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt)
			{
				Test->AddError(TEXT("Wrong FFieldId in OnPropertyIntChanged"));
			}
			++NumberPropertyIntChanged;
		}

		void OnPropertyFloatChanged(UObject* InObject, UE::FieldNotification::FFieldId InFieldId)
		{
			if (InObject != SourceObj)
			{
				Test->AddError(TEXT("Wrong SourceObject in OnPropertyFloatChanged"));
			}
			if (InFieldId != UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat)
			{
				Test->AddError(TEXT("Wrong FFieldId in OnPropertyFloatChanged"));
			}
			++NumberPropertyFloatChanged;
		}

		void OnFunctionIntChanged(UObject* InObject, UE::FieldNotification::FFieldId InFieldId)
		{
			if (InObject != SourceObj)
			{
				Test->AddError(TEXT("Wrong SourceObject in OnFunctionIntChanged"));
			}
			if (InFieldId != UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt)
			{
				Test->AddError(TEXT("Wrong FFieldId in OnFunctionIntChanged"));
			}
			++NumberFunctionIntChanged;
		}

		void OnFunctionFloatChanged(UObject* InObject, UE::FieldNotification::FFieldId InFieldId)
		{
			if (InObject != SourceObj)
			{
				Test->AddError(TEXT("Wrong SourceObject in OnFunctionFloatChanged"));
			}
			if (InFieldId != UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat)
			{
				Test->AddError(TEXT("Wrong FFieldId in OnFunctionFloatChanged"));
			}
			++NumberFunctionFloatChanged;
		}

		void ResetValues()
		{
			NumberPropertyIntChanged = 0;
			NumberPropertyFloatChanged = 0;
			NumberFunctionIntChanged = 0;
			NumberFunctionFloatChanged = 0;
		}
		FMVVMFieldValueChangedTest* Test = nullptr;
		UMVVMFieldValueChangedTest* SourceObj = nullptr;
		int32 NumberPropertyIntChanged = 0;
		int32 NumberPropertyFloatChanged = 0;
		int32 NumberFunctionIntChanged = 0;
		int32 NumberFunctionFloatChanged = 0;
	};

	const int32 NumInstances = 3;
	TArray<FCallbackHandler> CallbackInstances;
	for (int32 Index = 0; Index < NumInstances; ++Index)
	{
		FCallbackHandler& Instance = CallbackInstances.AddDefaulted_GetRef();
		Instance.Test = this;
		Instance.SourceObj = SourceObj;
	}

	auto ResetAllValues = [&]()
	{
		for (FCallbackHandler& Instance : CallbackInstances)
		{
			Instance.ResetValues();
		}
	};

	auto TestAllExpectedCountValues = [&](int32 InPropertyInt, int32 InPropertyFloat, int32 InFunctionInt, int32 InFunctionFloat)
	{
		int32 NumberPropertyIntChanged = 0;
		int32 NumberPropertyFloatChanged = 0;
		int32 NumberFunctionIntChanged = 0;
		int32 NumberFunctionFloatChanged = 0;
		for (FCallbackHandler& Instance : CallbackInstances)
		{
			NumberPropertyIntChanged += Instance.NumberPropertyIntChanged;
			NumberPropertyFloatChanged += Instance.NumberPropertyFloatChanged;
			NumberFunctionIntChanged += Instance.NumberFunctionIntChanged;
			NumberFunctionFloatChanged += Instance.NumberFunctionFloatChanged;
		}

		if (NumberPropertyIntChanged != InPropertyInt)
		{
			AddError(TEXT("Wrong number of callback called with PropertyInt"));
		}
		if (NumberPropertyFloatChanged != InPropertyFloat)
		{
			AddError(TEXT("Wrong number of callback called with PropertyFloat"));
		}
		if (NumberFunctionIntChanged != InFunctionInt)
		{
			AddError(TEXT("Wrong number of callback called with FunctionInt"));
		}
		if (NumberFunctionFloatChanged != InFunctionFloat)
		{
			AddError(TEXT("Wrong number of callback called with FunctionFloat"));
		}
	};

	auto TestExpectedCountValues = [&](int32 EntryIndex, int32 InPropertyInt, int32 InPropertyFloat, int32 InFunctionInt, int32 InFunctionFloat)
	{
		FCallbackHandler& Instance = CallbackInstances[EntryIndex];
		if (Instance.NumberPropertyIntChanged != InPropertyInt)
		{
			AddError(TEXT("Wrong number of callback called with PropertyInt"));
		}
		if (Instance.NumberPropertyFloatChanged != InPropertyFloat)
		{
			AddError(TEXT("Wrong number of callback called with PropertyFloat"));
		}
		if (Instance.NumberFunctionIntChanged != InFunctionInt)
		{
			AddError(TEXT("Wrong number of callback called with FunctionInt"));
		}
		if (Instance.NumberFunctionFloatChanged != InFunctionFloat)
		{
			AddError(TEXT("Wrong number of callback called with FunctionFloat"));
		}
	};

	auto TestBroadcastAll = [&]()
	{
		SourceObj->BroadcastFieldValueChanged(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt);
		SourceObj->BroadcastFieldValueChanged(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat);
		SourceObj->BroadcastFieldValueChanged(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt);
		SourceObj->BroadcastFieldValueChanged(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat);
	};


	FDelegateHandle HandleA_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnPropertyIntChanged));
	FDelegateHandle HandleB_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnPropertyFloatChanged));
	FDelegateHandle HandleC_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnFunctionIntChanged));
	FDelegateHandle HandleD_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnFunctionFloatChanged));

	TestAllExpectedCountValues(0, 0, 0, 0);
	ResetAllValues();

	SourceObj->BroadcastFieldValueChanged(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt);
	TestAllExpectedCountValues(1, 0, 0, 0);
	TestExpectedCountValues(0, 1, 0, 0, 0);
	ResetAllValues();

	TestBroadcastAll();
	TestAllExpectedCountValues(1, 1, 1, 1);
	TestExpectedCountValues(0, 1, 1, 1, 1);
	ResetAllValues();

	SourceObj->RemoveFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, HandleB_0);
	TestBroadcastAll();
	TestAllExpectedCountValues(1, 1, 1, 1);
	TestExpectedCountValues(0, 1, 1, 1, 1);
	ResetAllValues();
	
	SourceObj->RemoveFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, HandleA_0);
	TestBroadcastAll();
	TestAllExpectedCountValues(0, 1, 1, 1);
	TestExpectedCountValues(0, 0, 1, 1, 1);
	ResetAllValues();

	HandleA_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnPropertyIntChanged));
	TestBroadcastAll();
	TestAllExpectedCountValues(1, 1, 1, 1);
	TestExpectedCountValues(0, 1, 1, 1, 1);
	ResetAllValues();

	FDelegateHandle HandleA_1 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnPropertyIntChanged));
	FDelegateHandle HandleB_1 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnPropertyFloatChanged));
	FDelegateHandle HandleC_1 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnFunctionIntChanged));
	FDelegateHandle HandleD_1 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnFunctionFloatChanged));

	TestBroadcastAll();
	TestAllExpectedCountValues(2, 2, 2, 2);
	TestExpectedCountValues(0, 2, 2, 2, 2);
	ResetAllValues();

	FDelegateHandle HandleA_2 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[1], &FCallbackHandler::OnPropertyIntChanged));
	FDelegateHandle HandleB_2 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[1], &FCallbackHandler::OnPropertyFloatChanged));
	FDelegateHandle HandleC_2 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[1], &FCallbackHandler::OnFunctionIntChanged));
	FDelegateHandle HandleD_2 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[1], &FCallbackHandler::OnFunctionFloatChanged));

	TestBroadcastAll();
	TestAllExpectedCountValues(3, 3, 3, 3);
	TestExpectedCountValues(0, 2, 2, 2, 2);
	ResetAllValues();

	SourceObj->RemoveFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, HandleA_0);
	TestBroadcastAll();
	TestAllExpectedCountValues(3, 3, 3, 3);
	TestExpectedCountValues(0, 2, 2, 2, 2);
	ResetAllValues();

	SourceObj->RemoveFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, HandleC_0);
	TestBroadcastAll();
	TestAllExpectedCountValues(3, 3, 2, 3);
	TestExpectedCountValues(0, 2, 2, 1, 2);
	ResetAllValues();

	SourceObj->RemoveFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, HandleC_0);
	TestBroadcastAll();
	TestAllExpectedCountValues(3, 3, 2, 3);
	TestExpectedCountValues(0, 2, 2, 1, 2);
	ResetAllValues();

	SourceObj->RemoveFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, HandleA_0);
	TestBroadcastAll();
	TestAllExpectedCountValues(2, 3, 2, 3);
	TestExpectedCountValues(0, 1, 2, 1, 2);
	ResetAllValues();

	SourceObj->RemoveAllFieldValueChangedDelegates(&CallbackInstances[0]);
	TestBroadcastAll();
	TestAllExpectedCountValues(1, 1, 1, 1);
	TestExpectedCountValues(0, 0, 0, 0, 0);
	ResetAllValues();

	HandleA_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnPropertyIntChanged));
	HandleB_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnPropertyFloatChanged));
	HandleC_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnFunctionIntChanged));
	HandleD_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnFunctionFloatChanged));

	FDelegateHandle HandleA_3 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[2], &FCallbackHandler::OnPropertyIntChanged));
	FDelegateHandle HandleB_3 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[2], &FCallbackHandler::OnPropertyFloatChanged));
	FDelegateHandle HandleC_3 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[2], &FCallbackHandler::OnFunctionIntChanged));
	FDelegateHandle HandleD_3 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[2], &FCallbackHandler::OnFunctionFloatChanged));

	TestBroadcastAll();
	TestAllExpectedCountValues(3, 3, 3, 3);
	TestExpectedCountValues(0, 1, 1, 1, 1);
	TestExpectedCountValues(1, 1, 1, 1, 1);
	TestExpectedCountValues(2, 1, 1, 1, 1);
	ResetAllValues();

	SourceObj->BroadcastFieldValueChanged(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt);
	TestAllExpectedCountValues(3, 0, 0, 0);
	TestExpectedCountValues(0, 1, 0, 0, 0);
	TestExpectedCountValues(1, 1, 0, 0, 0);
	TestExpectedCountValues(2, 1, 0, 0, 0);
	ResetAllValues();

	SourceObj->RemoveAllFieldValueChangedDelegates(&CallbackInstances[0]);
	TestBroadcastAll();
	TestAllExpectedCountValues(2, 2, 2, 2);
	TestExpectedCountValues(0, 0, 0, 0, 0);
	TestExpectedCountValues(1, 1, 1, 1, 1);
	ResetAllValues();

	SourceObj->RemoveAllFieldValueChangedDelegates(&CallbackInstances[2]);
	TestBroadcastAll();
	TestAllExpectedCountValues(1, 1, 1, 1);
	TestExpectedCountValues(1, 1, 1, 1, 1);
	ResetAllValues();

	SourceObj->RemoveAllFieldValueChangedDelegates(&CallbackInstances[0]);
	TestBroadcastAll();
	TestAllExpectedCountValues(1, 1, 1, 1);
	TestExpectedCountValues(1, 1, 1, 1, 1);
	ResetAllValues();
	
	SourceObj->RemoveAllFieldValueChangedDelegates(&CallbackInstances[1]);
	TestBroadcastAll();
	TestAllExpectedCountValues(0, 0, 0, 0);
	ResetAllValues();

	HandleA_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnPropertyIntChanged));
	HandleB_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnPropertyFloatChanged));
	HandleC_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnFunctionIntChanged));
	HandleD_0 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnFunctionFloatChanged));

	HandleA_1 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnPropertyIntChanged));
	HandleB_1 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnPropertyFloatChanged));
	HandleC_1 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnFunctionIntChanged));
	HandleD_1 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[0], &FCallbackHandler::OnFunctionFloatChanged));

	HandleA_2 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[1], &FCallbackHandler::OnPropertyIntChanged));
	HandleB_2 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[1], &FCallbackHandler::OnPropertyFloatChanged));
	HandleC_2 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[1], &FCallbackHandler::OnFunctionIntChanged));
	HandleD_2 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[1], &FCallbackHandler::OnFunctionFloatChanged));

	HandleA_3 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[2], &FCallbackHandler::OnPropertyIntChanged));
	HandleB_3 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[2], &FCallbackHandler::OnPropertyFloatChanged));
	HandleC_3 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[2], &FCallbackHandler::OnFunctionIntChanged));
	HandleD_3 = SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&CallbackInstances[2], &FCallbackHandler::OnFunctionFloatChanged));

	SourceObj->RemoveAllFieldValueChangedDelegates(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, &CallbackInstances[0]);
	TestBroadcastAll();
	TestAllExpectedCountValues(2, 4, 4, 4);
	TestExpectedCountValues(0, 0, 2, 2, 2);
	TestExpectedCountValues(1, 1, 1, 1, 1);
	ResetAllValues();

	// Remove element while iterating
	struct FRemoveCallbackHandler
	{
		FDelegateHandle ToRemoveHandle;
		UMVVMFieldValueChangedTest* SourceObj = nullptr;
		FCallbackHandler* CallbackHandler = nullptr;
		int32 NumberPropertyIntChanged = 0;
		int32 NumberPropertyFloatChanged = 0;
		int32 NumberFunctionIntChanged = 0;
		int32 NumberFunctionFloatChanged = 0;
		int32 NumberWasAddedWhileIteratingOther = 0;
		int32 NumberWasAddedWhileIterating = 0;
		void OnPropertyIntChanged(UObject* InObject, UE::FieldNotification::FFieldId InFieldId)
		{
			if (SourceObj && ToRemoveHandle.IsValid())
			{
				SourceObj->RemoveFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, ToRemoveHandle);
			}
			++NumberPropertyIntChanged;
		}
		void OnPropertyFloatChanged(UObject* InObject, UE::FieldNotification::FFieldId InFieldId)
		{
			if (SourceObj && CallbackHandler)
			{
				SourceObj->RemoveAllFieldValueChangedDelegates(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, CallbackHandler);
			}
			++NumberPropertyFloatChanged;
		}
		void OnFunctionIntChanged(UObject* InObject, UE::FieldNotification::FFieldId InFieldId)
		{
			if (SourceObj && CallbackHandler)
			{
				SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(this, &FRemoveCallbackHandler::OnWasAddedWhileIteratingOther));
			}
			++NumberFunctionIntChanged;
		}
		void OnFunctionFloatChanged(UObject* InObject, UE::FieldNotification::FFieldId InFieldId)
		{
			if (SourceObj && CallbackHandler)
			{
				SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(this, &FRemoveCallbackHandler::OnWasAddedWhileIterating));
			}
			++NumberFunctionFloatChanged;
		}
		void OnWasAddedWhileIteratingOther(UObject* InObject, UE::FieldNotification::FFieldId InFieldId)
		{
			// this one should be called when OnFunctionFloatChanged is called, not when OnFunctionIntChanged is called
			++NumberWasAddedWhileIteratingOther;
			if (SourceObj && CallbackHandler)
			{
				SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::FunctionFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(this, &FRemoveCallbackHandler::OnWasAddedWhileIterating));
			}
		}
		void OnWasAddedWhileIterating(UObject* InObject, UE::FieldNotification::FFieldId InFieldId)
		{
			// this one should be called when OnFunctionFloatChanged and OnWasAddedWhileIteratingOther are called
			++NumberWasAddedWhileIterating;
		}
		void Reset()
		{
			NumberPropertyIntChanged = 0;
			NumberPropertyFloatChanged = 0;
			NumberFunctionIntChanged = 0;
			NumberFunctionFloatChanged = 0;
			NumberWasAddedWhileIteratingOther = 0;
			NumberWasAddedWhileIterating = 0;
		}
		void TestExpectedCount(FMVVMFieldValueChangedTest* TestOwner, int32 InPropertyInt, int32 InPropertyFloat, int32 InFunctionInt, int32 InFunctionFloat, int32 InAddedWhileIteratingOther, int32 InAddedWhileIterating)
		{
			if (NumberPropertyIntChanged != InPropertyInt)
			{
				TestOwner->AddError(TEXT("Wrong number of callback called with PropertyInt while iterating"));
			}
			if (NumberPropertyFloatChanged != InPropertyFloat)
			{
				TestOwner->AddError(TEXT("Wrong number of callback called with PropertyFloat while iterating"));
			}
			if (NumberFunctionIntChanged != InFunctionInt)
			{
				TestOwner->AddError(TEXT("Wrong number of callback called with FunctionInt while iterating"));
			}
			if (NumberFunctionFloatChanged != InFunctionFloat)
			{
				TestOwner->AddError(TEXT("Wrong number of callback called with FunctionFloat while iterating"));
			}
			if (NumberWasAddedWhileIteratingOther != InAddedWhileIteratingOther)
			{
				TestOwner->AddError(TEXT("Wrong number of callback called with AddedWhileIteratingOther while iterating"));
			}
			if (NumberWasAddedWhileIterating != InAddedWhileIterating)
			{
				TestOwner->AddError(TEXT("Wrong number of callback called with AddedWhileIterating while iterating"));
			}
		}
	};
	FRemoveCallbackHandler RemoveCallbackInstance;

	RemoveCallbackInstance.SourceObj = SourceObj;
	RemoveCallbackInstance.ToRemoveHandle = HandleA_2;
	RemoveCallbackInstance.CallbackHandler = &CallbackInstances[2];

	SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyInt, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&RemoveCallbackInstance, &FRemoveCallbackHandler::OnPropertyIntChanged));
	SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&RemoveCallbackInstance, &FRemoveCallbackHandler::OnPropertyFloatChanged));

	TestBroadcastAll();
	// Do not test the result here because the order is not deterministic.
	//TestAllExpectedCountValues
	RemoveCallbackInstance.TestExpectedCount(this, 1, 1, 0, 0, 0, 0);
	ResetAllValues();
	RemoveCallbackInstance.Reset();

	TestBroadcastAll();
	TestAllExpectedCountValues(1, 3, 4, 4);
	TestExpectedCountValues(0, 0, 2, 2, 2);
	TestExpectedCountValues(1, 0, 1, 1, 1);
	TestExpectedCountValues(2, 1, 0, 1, 1);
	RemoveCallbackInstance.TestExpectedCount(this, 1, 1, 0, 0, 0, 0);
	ResetAllValues();
	RemoveCallbackInstance.Reset();

	SourceObj->AddFieldValueChangedDelegate(UMVVMFieldValueChangedTest::FFieldNotificationClassDescriptor::PropertyFloat, UMVVMFieldValueChangedTest::FFieldValueChangedDelegate::CreateRaw(&RemoveCallbackInstance, &FRemoveCallbackHandler::OnFunctionIntChanged));
	TestBroadcastAll();
	TestAllExpectedCountValues(1, 3, 4, 4);
	TestExpectedCountValues(0, 0, 2, 2, 2);
	TestExpectedCountValues(1, 0, 1, 1, 1);
	TestExpectedCountValues(2, 1, 0, 1, 1);
	RemoveCallbackInstance.TestExpectedCount(this, 1, 1, 1, 0, 1, 1);
	ResetAllValues();
	RemoveCallbackInstance.Reset();

	return true;
}

#undef LOCTEXT_NAMESPACE 
#endif //WITH_AUTOMATION_WORKER
