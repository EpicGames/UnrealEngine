// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMViewModelBase.h"


#include "MVVMFieldValueChangedTest.generated.h"

UCLASS(MinimalAPI, Transient, hidedropdown, Hidden)
class UMVVMFieldValueChangedTest : public UMVVMViewModelBase
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, FieldNotify, Category = "MVVM")
	int32 PropertyInt;

	UPROPERTY(BlueprintReadOnly, FieldNotify, Category = "MVVM")
	float PropertyFloat;

	UFUNCTION(BlueprintCallable, FieldNotify, Category = "MVVM")
	int32 FunctionInt() const
	{
		return 0;
	}

	UFUNCTION(BlueprintCallable, FieldNotify, Category = "MVVM")
	float FunctionFloat() const
	{
		return 1.0f;
	}
};
