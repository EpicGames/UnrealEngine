// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/MVVMBindingEntry.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMBlueprintViewCondition.h"


#define LOCTEXT_NAMESPACE "BindingEntry"

namespace UE::MVVM
{

FMVVMBlueprintViewBinding* FBindingEntry::GetBinding(UMVVMBlueprintView* View) const
{
	return View->GetBinding(BindingId);
}

const FMVVMBlueprintViewBinding* FBindingEntry::GetBinding(const UMVVMBlueprintView* View) const
{
	return View->GetBinding(BindingId);
}

void FBindingEntry::SetGroup(FName WidgetName)
{
	check(RowType == ERowType::None);
	RowType = ERowType::Group;
	Name = WidgetName;
	bGroupIsWidget = true;
}

void FBindingEntry::SetGroup(FName ViewModelName, FGuid ViewModelId)
{
	if (!ViewModelId.IsValid())
	{
		SetGroup(ViewModelName);
	}
	else
	{
		check(RowType == ERowType::None);
		RowType = ERowType::Group;
		Name = ViewModelName;
		BindingId = ViewModelId;
		bGroupIsWidget = false;
	}
}

void FBindingEntry::SetBindingId(FGuid Id)
{
	check(RowType == ERowType::None);
	RowType = ERowType::Binding;
	BindingId = Id;
}

void FBindingEntry::SetBindingParameter(FGuid Id, FMVVMBlueprintPinId Parameter, FEdGraphPinType ParameterType)
{
	check(RowType == ERowType::None);
	RowType = ERowType::BindingParameter;
	BindingId = Id;
	PinId = MoveTemp(Parameter);
	PinType = MoveTemp(ParameterType);
}

UMVVMBlueprintViewEvent* FBindingEntry::GetEvent() const
{
	return Event.Get();
}

void FBindingEntry::SetEvent(UMVVMBlueprintViewEvent* InEvent)
{
	check(RowType == ERowType::None);
	RowType = ERowType::Event;
	Event = InEvent;
}

void FBindingEntry::SetEventParameter(UMVVMBlueprintViewEvent* InEvent, FMVVMBlueprintPinId Parameter, FEdGraphPinType ParameterType)
{
	check(RowType == ERowType::None);
	RowType = ERowType::EventParameter;
	Event = InEvent;
	PinId = MoveTemp(Parameter);
	PinType = MoveTemp(ParameterType);
}

UMVVMBlueprintViewCondition* FBindingEntry::GetCondition() const
{
	return Condition.Get();
}

void FBindingEntry::SetCondition(UMVVMBlueprintViewCondition* InCondition)
{
	check(RowType == ERowType::None);
	RowType = ERowType::Condition;
	Condition = InCondition;
}

void FBindingEntry::SetConditionParameter(UMVVMBlueprintViewCondition* InCondition, FMVVMBlueprintPinId Parameter, FEdGraphPinType ParameterType)
{
	check(RowType == ERowType::None);
	RowType = ERowType::ConditionParameter;
	Condition = InCondition;
	PinId = MoveTemp(Parameter);
	PinType = MoveTemp(ParameterType);
}


void FBindingEntry::AddChild(TSharedPtr<FBindingEntry> Child)
{
	AllChildren.Add(Child);
}
	
void FBindingEntry::AddFilteredChild(TSharedPtr<FBindingEntry> Child)
{
	FilteredChildren.Add(Child);
	bUseFilteredChildren = true;
}

void FBindingEntry::ResetChildren()
{
	AllChildren.Reset();
	FilteredChildren.Reset();
	bUseFilteredChildren = false;
}

void FBindingEntry::ResetFilteredChildren()
{
	FilteredChildren.Reset();
	bUseFilteredChildren = false;
}

void FBindingEntry::SetUseFilteredChildList()
{
	bUseFilteredChildren = true;
}

bool FBindingEntry::operator==(const FBindingEntry& Other) const
{
	return RowType == Other.RowType
		&& Name == Other.Name
		&& BindingId == Other.BindingId
		&& PinId == Other.PinId
		&& PinType == Other.PinType
		&& Event == Other.Event
		&& Condition == Other.Condition;
}

FString FBindingEntry::GetSearchNameString(UMVVMBlueprintView* View, UWidgetBlueprint* WidgetBP)
{
	FString RowToString;
	FString FunctionKeywords;

	switch (RowType)
	{
	case UE::MVVM::FBindingEntry::ERowType::Group:
		RowToString = Name.ToString();
		break;
	case UE::MVVM::FBindingEntry::ERowType::BindingParameter:
	case UE::MVVM::FBindingEntry::ERowType::EventParameter:
	case UE::MVVM::FBindingEntry::ERowType::ConditionParameter:
		RowToString = PinId.ToString();
		break;
	case UE::MVVM::FBindingEntry::ERowType::Binding:
		{
			FMVVMBlueprintViewBinding* BindingInRow = GetBinding(View);
			check(BindingInRow);
			RowToString.Append(BindingInRow->GetSearchableString(WidgetBP));
		}
		break;
	case UE::MVVM::FBindingEntry::ERowType::Event:
		if (UMVVMBlueprintViewEvent* EventInRow = GetEvent())
		{
			RowToString.Append(EventInRow->GetSearchableString());
		}
		break;
	case UE::MVVM::FBindingEntry::ERowType::Condition:
		if (UMVVMBlueprintViewCondition* ConditionInRow = GetCondition())
		{
			RowToString.Append(ConditionInRow->GetSearchableString());
		}
		break;
	default:
		break;
	}

	RowToString.ReplaceInline(TEXT(" "), TEXT(""));

	return RowToString;
}

} // namespace

#undef LOCTEXT_NAMESPACE