// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MVVMBlueprintPin.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UMVVMBlueprintView;
class UMVVMBlueprintViewEvent;
class UMVVMBlueprintViewCondition;
struct FMVVMBlueprintViewBinding;
class UWidgetBlueprint;

namespace UE::MVVM
{

/**
 * A structure for the different entries in the BindingList
 */
struct FBindingEntry
{
	enum class ERowType
	{
		None,
		Group,
		Binding,
		BindingParameter,
		Event,
		EventParameter,
		Condition,
		ConditionParameter
	};

	FMVVMBlueprintViewBinding* GetBinding(UMVVMBlueprintView* View) const;

	const FMVVMBlueprintViewBinding* GetBinding(const UMVVMBlueprintView* View) const;

	ERowType GetRowType() const
	{
		return RowType;
	}

	//~ group
	FName GetGroupName() const
	{
		return Name;
	}
	
	FGuid GetGroupAsViewModel() const
	{
		return BindingId;
	}
	
	bool IsGroupWidget() const
	{
		return bGroupIsWidget;
	}
	
	void SetGroup(FName WidgetName);
	void SetGroup(FName ViewModelName, FGuid ViewModelId);

	//~ binding
	FGuid GetBindingId() const
	{
		return BindingId;
	}

	void SetBindingId(FGuid Id);

	void SetHasPathToVerseVariable(bool bInHasPathToVerseVariable) { bHasPathToVerseVariable = bInHasPathToVerseVariable; }
	void SetHasPathToViewModel(bool bInHasPathToViewModel) { bHasPathToViewModel = bInHasPathToViewModel; }
	bool GetHasPathToVerseVariable() const { return bHasPathToVerseVariable; }
	bool GetHasPathToViewModel() const { return bHasPathToViewModel; }

	//~ binding parameter
	const FMVVMBlueprintPinId& GetBindingParameterId() const
	{
		return PinId;
	}
	
	const FEdGraphPinType& GetBindingParameterType() const
	{
		return PinType;
	}

	void SetBindingParameter(FGuid Id, FMVVMBlueprintPinId Parameter, FEdGraphPinType ParameterType);

	//~ event
	UMVVMBlueprintViewEvent* GetEvent() const;

	void SetEvent(UMVVMBlueprintViewEvent* InEvent);

	//~ event parameter
	const FMVVMBlueprintPinId& GetEventParameterId() const
	{
		return PinId;
	}
	
	const FEdGraphPinType& GetEventParameterType() const
	{
		return PinType;
	}

	void SetEventParameter(UMVVMBlueprintViewEvent* Event, FMVVMBlueprintPinId Parameter, FEdGraphPinType ParameterType);

	//~ condition
	UMVVMBlueprintViewCondition* GetCondition() const;

	void SetCondition(UMVVMBlueprintViewCondition* InCondition);

	//~ condition parameter
	const FMVVMBlueprintPinId& GetConditionParameterId() const
	{
		return PinId;
	}
	
	const FEdGraphPinType& GetConditionParameterType() const
	{
		return PinType;
	}

	void SetConditionParameter(UMVVMBlueprintViewCondition* Condition, FMVVMBlueprintPinId Parameter, FEdGraphPinType ParameterType);


	//~ children
	TConstArrayView<TSharedPtr<FBindingEntry>> GetAllChildren() const
	{
		return AllChildren;
	}

	TConstArrayView<TSharedPtr<FBindingEntry>> GetFilteredChildren() const
	{
		return bUseFilteredChildren ? FilteredChildren : AllChildren;
	}

	void AddChild(TSharedPtr<FBindingEntry> Child);

	void AddFilteredChild(TSharedPtr<FBindingEntry> Child);

	void ResetChildren();
	void ResetFilteredChildren();

	void SetUseFilteredChildList();

	bool operator==(const FBindingEntry& Other) const;

	FString GetSearchNameString(UMVVMBlueprintView* View, UWidgetBlueprint* WidgetBP);
	
private:
	ERowType RowType = ERowType::None;
	FName Name;
	FGuid BindingId;
	FMVVMBlueprintPinId PinId;
	FEdGraphPinType PinType;
	TWeakObjectPtr<UMVVMBlueprintViewEvent> Event;
	TWeakObjectPtr<UMVVMBlueprintViewCondition> Condition;
	TArray<TSharedPtr<FBindingEntry>> AllChildren;
	TArray<TSharedPtr<FBindingEntry>> FilteredChildren;
	bool bGroupIsWidget = false;
	bool bUseFilteredChildren = false;
	bool bHasPathToVerseVariable = false;
	bool bHasPathToViewModel = false;
};

} // namespace
