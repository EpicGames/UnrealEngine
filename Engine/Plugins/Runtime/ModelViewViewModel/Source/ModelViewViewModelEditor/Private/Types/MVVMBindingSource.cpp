// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/MVVMBindingSource.h"
#include "Blueprint/WidgetTree.h"
#include "Editor.h"
#include "Extensions/UIComponent.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewModelContext.h"
#include "MVVMPropertyPath.h"
#include "MVVMEditorSubsystem.h"
#include "Types/MVVMBindingName.h"
#include "WidgetBlueprint.h"


#define LOCTEXT_NAMESPACE "BindingSource"

namespace UE::MVVM
{

const UClass* FBindingSource::GetClass() const
{
	return Class.Get();
}

FText FBindingSource::GetDisplayName() const
{
	return DisplayName;
}

FMVVMBindingName FBindingSource::ToBindingName(const UWidgetBlueprint* WidgetBlueprint) const
{
	switch (Source)
	{
	case EMVVMBlueprintFieldPathSource::SelfContext:
		return FMVVMBindingName(WidgetBlueprint->GetFName());

	case EMVVMBlueprintFieldPathSource::ViewModel:
		if (UMVVMBlueprintView* View = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->GetView(WidgetBlueprint))
		{
			if (const FMVVMBlueprintViewModelContext* ViewModel = View->FindViewModel(ViewModelId))
			{
				return FMVVMBindingName(ViewModel->GetViewModelName());
			}
		}
		return FMVVMBindingName();

	case EMVVMBlueprintFieldPathSource::Widget:
		return FMVVMBindingName(WidgetName);
	}
	return FMVVMBindingName();
}

void FBindingSource::SetSourceTo(FMVVMBlueprintPropertyPath& PropertyPath) const
{
	switch(Source)
	{
	case EMVVMBlueprintFieldPathSource::SelfContext:
		PropertyPath.SetSelfContext();
		break;
	case EMVVMBlueprintFieldPathSource::ViewModel:
		PropertyPath.SetViewModelId(ViewModelId);
		break;
	case EMVVMBlueprintFieldPathSource::Widget:
		PropertyPath.SetWidgetName(WidgetName);
		break;
	default:
		PropertyPath.ResetSource();
	}
}

FBindingSource FBindingSource::CreateForBlueprint(const UWidgetBlueprint* WidgetBlueprint)
{
	FBindingSource Source;

	Source.Source = EMVVMBlueprintFieldPathSource::SelfContext;
	Source.DisplayName = FText::FromString(WidgetBlueprint->GetName());
	Source.Class = WidgetBlueprint->SkeletonGeneratedClass ? WidgetBlueprint->SkeletonGeneratedClass : WidgetBlueprint->GeneratedClass;
	
	return Source;
}

FBindingSource FBindingSource::CreateForWidget(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget)
{
	if (Widget->GetFName() == WidgetBlueprint->GetFName())
	{
		return CreateForBlueprint(WidgetBlueprint);
	}

	FBindingSource Source;

	Source.Source = EMVVMBlueprintFieldPathSource::Widget;
	Source.WidgetName = Widget->GetFName();
	Source.DisplayName = Widget->GetLabelText();
	Source.Class = Widget->GetClass();
	
	return Source;
}

FBindingSource FBindingSource::CreateForWidget(const UWidgetBlueprint* WidgetBlueprint, FName WidgetName)
{
	if (WidgetName == WidgetBlueprint->GetFName())
	{
		return CreateForBlueprint(WidgetBlueprint);
	}

	FBindingSource Source;

	Source.Source = EMVVMBlueprintFieldPathSource::Widget;
	Source.WidgetName = WidgetName;
	if (UWidget* Widget = WidgetBlueprint->WidgetTree->FindWidget(Source.WidgetName))
	{
		Source.DisplayName = Widget->GetLabelText();
		Source.Class = Widget->GetClass();
	}

	return Source;
}

FBindingSource FBindingSource::CreateForWidgetComponent(const UWidgetBlueprint* WidgetBlueprint, FName WidgetName, UUIComponent* Component)
{
	FBindingSource Source;

	Source.Source = EMVVMBlueprintFieldPathSource::Widget;
	Source.WidgetName = WidgetName;
	if (UWidget* Widget = WidgetBlueprint->WidgetTree->FindWidget(Source.WidgetName))
	{
		Source.DisplayName = Widget->GetLabelText();
		Source.Class = Component->GetClass();
	}
	return Source;
}

FBindingSource FBindingSource::CreateForViewModel(const UWidgetBlueprint* WidgetBlueprint, FGuid ViewModelId)
{
	FBindingSource Source;

	Source.Source = EMVVMBlueprintFieldPathSource::ViewModel;
	Source.ViewModelId = ViewModelId;

	UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	if (UMVVMBlueprintView* View = Subsystem->GetView(WidgetBlueprint))
	{
		if (const FMVVMBlueprintViewModelContext* ViewModel = View->FindViewModel(Source.ViewModelId))
		{
			Source.DisplayName = ViewModel->GetDisplayName();
			Source.Class = ViewModel->GetViewModelClass();
		}
	}

	return Source;
}

FBindingSource FBindingSource::CreateForViewModel(const UWidgetBlueprint* WidgetBlueprint, FName ViewModelName)
{
	FBindingSource Source;

	UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	if (UMVVMBlueprintView* View = Subsystem->GetView(WidgetBlueprint))
	{
		if (const FMVVMBlueprintViewModelContext* ViewModel = View->FindViewModel(ViewModelName))
		{
			Source.Source = EMVVMBlueprintFieldPathSource::ViewModel;
			Source.ViewModelId = ViewModel->GetViewModelId();
			Source.DisplayName = ViewModel->GetDisplayName();
			Source.Class = ViewModel->GetViewModelClass();
		}
	}

	return Source;
}

FBindingSource FBindingSource::CreateForViewModel(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewModelContext& ViewModelContext)
{
	FBindingSource Source;

	Source.Source = EMVVMBlueprintFieldPathSource::ViewModel;
	Source.ViewModelId = ViewModelContext.GetViewModelId();
	Source.DisplayName = ViewModelContext.GetDisplayName();
	Source.Class = ViewModelContext.GetViewModelClass();

	return Source;
}

FBindingSource FBindingSource::CreateEmptySource(UClass* ViewModel)
{
	FBindingSource Source;

	Source.DisplayName = ViewModel->GetDisplayNameText();
	Source.Class = ViewModel;

	return Source;
}

bool FBindingSource::Matches(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& PropertyPath) const
{
	if (Source == PropertyPath.GetSource(WidgetBlueprint))
	{
		switch (Source)
		{
		case EMVVMBlueprintFieldPathSource::ViewModel:
			return PropertyPath.GetViewModelId() == ViewModelId;
		case EMVVMBlueprintFieldPathSource::Widget:
			return PropertyPath.GetWidgetName() == WidgetName;
		case EMVVMBlueprintFieldPathSource::SelfContext:
		default:
			return true;
		}
	}
	return false;
}

FBindingSource FBindingSource::CreateFromPropertyPath(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& Path)
{
	switch (Path.GetSource(WidgetBlueprint))
	{
	case EMVVMBlueprintFieldPathSource::SelfContext:
		return FBindingSource::CreateForBlueprint(WidgetBlueprint);
	case EMVVMBlueprintFieldPathSource::ViewModel:
		return FBindingSource::CreateForViewModel(WidgetBlueprint, Path.GetViewModelId());
	case EMVVMBlueprintFieldPathSource::Widget:
		return FBindingSource::CreateForWidget(WidgetBlueprint, Path.GetWidgetName());
	}
	return FBindingSource();
}

} // namespace

#undef LOCTEXT_NAMESPACE