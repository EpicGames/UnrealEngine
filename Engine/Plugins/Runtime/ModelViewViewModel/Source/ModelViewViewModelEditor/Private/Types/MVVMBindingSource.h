// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Views/TableViewTypeTraits.h"
#include "MVVMPropertyPath.h"
#include "UObject/Class.h"

struct FMVVMBindingName;
struct FMVVMBlueprintPropertyPath;
struct FMVVMBlueprintViewModelContext;
class UUIComponent;
class UWidget;
class UWidgetBlueprint;

namespace UE::MVVM
{

struct FBindingSource
{
private:
	FGuid ViewModelId;
	FName WidgetName;
	TWeakObjectPtr<UClass> Class;
	FText DisplayName;
	EMVVMBlueprintFieldPathSource Source = EMVVMBlueprintFieldPathSource::None;

public:
	bool operator==(const FBindingSource& Other) const
	{
		return Source == Other.Source && ViewModelId == Other.ViewModelId && WidgetName == Other.WidgetName && Class == Other.Class;
	}

	bool operator!=(const FBindingSource& Other) const
	{
		return !(operator==(Other));
	}

	friend int32 GetTypeHash(const FBindingSource& InSource)
	{
		uint32 Hash = HashCombine(GetTypeHash(InSource.ViewModelId), GetTypeHash(InSource.WidgetName));
		Hash = HashCombine(Hash, GetTypeHash(InSource.Class));
		Hash = HashCombine(Hash, GetTypeHash(InSource.Source));
		return Hash;
	}

	bool IsValid() const
	{
		return Class != nullptr && Source != EMVVMBlueprintFieldPathSource::None;
	}

	EMVVMBlueprintFieldPathSource GetSource() const
	{
		return Source;
	}

	const UClass* GetClass() const;
	FText GetDisplayName() const;

	FName GetWidgetName() const
	{
		return WidgetName;
	}

	FGuid GetViewModelId() const
	{
		return ViewModelId;
	}

	void Reset()
	{
		ViewModelId = FGuid();
		WidgetName = FName();
		Class = nullptr;
		DisplayName = FText::GetEmpty();
		Source = EMVVMBlueprintFieldPathSource::None;
	}

	FMVVMBindingName ToBindingName(const UWidgetBlueprint* WidgetBlueprint) const;
	void SetSourceTo(FMVVMBlueprintPropertyPath& PropertyPath) const;

	bool Matches(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& PropertyPath) const;

	static FBindingSource CreateForBlueprint(const UWidgetBlueprint* WidgetBlueprint);
	static FBindingSource CreateForWidget(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget);
	static FBindingSource CreateForWidget(const UWidgetBlueprint* WidgetBlueprint, FName WidgetName);
	static FBindingSource CreateForWidgetComponent(const UWidgetBlueprint* WidgetBlueprint, FName WidgetName, UUIComponent* Component);
	static FBindingSource CreateForViewModel(const UWidgetBlueprint* WidgetBlueprint, FGuid ViewModelId);
	static FBindingSource CreateForViewModel(const UWidgetBlueprint* WidgetBlueprint, FName ViewModelName);
	static FBindingSource CreateForViewModel(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewModelContext& ViewModelContext);
	static FBindingSource CreateEmptySource(UClass* ViewModel);
	static FBindingSource CreateFromPropertyPath(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& Path);
};

} // namespace

template <>
struct TIsValidListItem<UE::MVVM::FBindingSource>
{
	enum
	{
		Value = true
	};
};

template <>
struct TListTypeTraits<UE::MVVM::FBindingSource>
{
	typedef UE::MVVM::FBindingSource NullableType;

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<UE::MVVM::FBindingSource, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<UE::MVVM::FBindingSource, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<UE::MVVM::FBindingSource>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector&,
		TArray<UE::MVVM::FBindingSource>&,
		TSet<UE::MVVM::FBindingSource>&,
		TMap<const U*, UE::MVVM::FBindingSource>&)
	{
	}

	static bool IsPtrValid(const UE::MVVM::FBindingSource& InPtr)
	{
		return InPtr.IsValid();
	}

	static void ResetPtr(UE::MVVM::FBindingSource& InPtr)
	{
		InPtr.Reset();
	}

	static UE::MVVM::FBindingSource MakeNullPtr()
	{
		return UE::MVVM::FBindingSource();
	}

	static UE::MVVM::FBindingSource NullableItemTypeConvertToItemType(const UE::MVVM::FBindingSource& InPtr)
	{
		return InPtr;
	}

	static FString DebugDump(UE::MVVM::FBindingSource InPtr)
	{
		return InPtr.GetDisplayName().ToString();
	}

	class SerializerType {};
};
