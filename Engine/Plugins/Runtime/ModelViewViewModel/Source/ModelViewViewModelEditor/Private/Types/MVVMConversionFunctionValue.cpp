// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/MVVMConversionFunctionValue.h"
#include "EdGraphSchema_K2.h"

namespace UE::MVVM
{

FString FConversionFunctionValue::GetName() const
{
	return ConversionFunction ? ConversionFunction->GetName() : ConversionNode ? ConversionNode->GetName() : FString();
}

FName FConversionFunctionValue::GetFName() const
{
	return ConversionFunction ? ConversionFunction->GetFName() : ConversionNode ? ConversionNode->GetFName() : FName();
}

FString FConversionFunctionValue::GetFullGroupName(bool bStartWithOuter) const
{
	return ConversionFunction ? ConversionFunction->GetFullGroupName(bStartWithOuter) : ConversionNode ? ConversionNode->GetFullGroupName(bStartWithOuter) : FString();
}

FText FConversionFunctionValue::GetDisplayName() const
{
	return ConversionFunction ? ConversionFunction->GetDisplayNameText() : ConversionNode ? ConversionNode.GetDefaultObject()->GetNodeTitle(ENodeTitleType::MenuTitle) : FText();
}

FText FConversionFunctionValue::GetTooltip() const
{
	return ConversionFunction ? ConversionFunction->GetToolTipText() : ConversionNode ? ConversionNode->GetToolTipText() : FText();
}

FText FConversionFunctionValue::GetCategory() const
{
	static const FName NAME_Category = "Category";
	return ConversionFunction ? ConversionFunction->GetMetaDataText(NAME_Category) : ConversionNode ? ConversionNode->GetMetaDataText(NAME_Category) : FText();
}

namespace Private
{
	template<typename T>
	void AddResult(TArray<FString>& Result, T* Value)
	{
		Result.Add(Value->GetName());
		const FString& DisplayName = Value->GetMetaData(FBlueprintMetadata::MD_DisplayName);
		if (DisplayName.Len() > 0)
		{
			Result.Add(DisplayName);
		}

		FString MetadataKeywords = Value->GetMetaDataText(FBlueprintMetadata::MD_FunctionKeywords, TEXT("UObjectKeywords"), Value->GetFullGroupName(false)).ToString();
		if (MetadataKeywords.Len() > 0)
		{
			Result.Add(MoveTemp(MetadataKeywords));
		}
	}
}

TArray<FString> FConversionFunctionValue::GetSearchKeywords() const
{
	TArray<FString> Result;
	if (ConversionFunction)
	{
		Private::AddResult(Result, ConversionFunction);
	}
	else if (ConversionNode.Get())
	{
		Private::AddResult(Result, ConversionNode.Get());
	}
	return Result;
}

} // namespace UE::MVVM