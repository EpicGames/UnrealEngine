// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/MVVMLinkedPinValue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMLinkedPinValue)

FMVVMLinkedPinValue::FMVVMLinkedPinValue(FMVVMBlueprintPropertyPath InPath)
	: PropertyPath(MoveTemp(InPath))
{}

FMVVMLinkedPinValue::FMVVMLinkedPinValue(const UBlueprint* InBlueprint, FMVVMBlueprintFunctionReference InConversion)
{
	if (InConversion.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
	{
		ConversionFunction = InConversion.GetFunction(InBlueprint);
	}
	else if (InConversion.GetType() == EMVVMBlueprintFunctionReferenceType::Node)
	{
		ConversionNode = InConversion.GetNode();
	}
}

FMVVMLinkedPinValue::FMVVMLinkedPinValue(UE::MVVM::FConversionFunctionValue Function)
	: ConversionFunction(Function.GetFunction())
	, ConversionNode(Function.GetNode())
{}

FMVVMLinkedPinValue::FMVVMLinkedPinValue(const UFunction* Function)
	: ConversionFunction(Function)
{}

FMVVMLinkedPinValue::FMVVMLinkedPinValue(TSubclassOf<UK2Node> Node)
	: ConversionNode(Node)
{}

bool FMVVMLinkedPinValue::IsValid() const
{
	return ConversionFunction != nullptr || ConversionNode.Get() != nullptr || PropertyPath.IsValid();
}

bool FMVVMLinkedPinValue::IsPropertyPath() const
{
	return PropertyPath.IsValid();
}

const FMVVMBlueprintPropertyPath& FMVVMLinkedPinValue::GetPropertyPath() const
{
	return PropertyPath;
}

bool FMVVMLinkedPinValue::IsConversionFunction() const
{
	return ConversionFunction != nullptr;
}

const UFunction* FMVVMLinkedPinValue::GetConversionFunction() const
{
	return ConversionFunction;
}

bool FMVVMLinkedPinValue::IsConversionNode() const
{
	return ConversionNode.Get() != nullptr;
}

TSubclassOf<UK2Node> FMVVMLinkedPinValue::GetConversionNode() const
{
	return ConversionNode;
}
