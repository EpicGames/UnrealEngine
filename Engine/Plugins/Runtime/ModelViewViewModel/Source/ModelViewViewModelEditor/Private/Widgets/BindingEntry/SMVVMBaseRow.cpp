// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/BindingEntry/SMVVMBaseRow.h"
#include "Widgets/SMVVMViewBindingListView.h"

#include "Framework/MVVMRowHelper.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMDeveloperProjectSettings.h"
#include "Styling/MVVMEditorStyle.h"

#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "BindingListView_BaseRow"

namespace UE::MVVM::BindingEntry
{

void SBaseRow::Construct(const FArguments& Args, SBindingsList* InBindingsList, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FWidgetBlueprintEditor>& InBlueprintEditor, UWidgetBlueprint* InBlueprint, const TSharedPtr<FBindingEntry>& InEntry)
{
	BindingEntry = InEntry;
	WeakBlueprint = InBlueprint;
	WeakBlueprintEditor = InBlueprintEditor;
	BindingsList = InBindingsList;

	STableRow<TSharedPtr<FBindingEntry>>::Construct(
		STableRow<TSharedPtr<FBindingEntry>>::FArguments()
		.Padding(2.0f)
		.Style(FMVVMEditorStyle::Get(), GetTableRowStyle())
		[
			BuildRowWidget()
		],
		OwnerTableView
	);
}

TSharedRef<SWidget> SBaseRow::BuildRowWidget()
{
	return SNullWidget::NullWidget;
}
	
UWidgetBlueprint* SBaseRow::GetBlueprint() const
{
	return WeakBlueprint.Get();
}

UMVVMBlueprintView* SBaseRow::GetBlueprintView() const
{
	return GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->GetView(GetBlueprint());
}

UMVVMEditorSubsystem* SBaseRow::GetEditorSubsystem() const
{
	return GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
}

TSharedRef<SWidget> SBaseRow::BuildContextMenuButton()
{
	return SNew(SComboButton)
		.ContentPadding(FMargin(2.f, 0.0f))
		.ComboButtonStyle(&FMVVMEditorStyle::Get().GetWidgetStyle<FComboButtonStyle>("NoStyleComboButton"))
		.HasDownArrow(false)
		.OnGetMenuContent(this, &SBaseRow::HandleContextMenu)
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FMVVMEditorStyle::Get().GetBrush("Icon.Ellipsis"))
			.DesiredSizeOverride(FVector2D(6.0, 24.0))
		];
}

TSharedRef<SWidget> SBaseRow::HandleContextMenu() const
{
	TSharedPtr<FBindingEntry> Entry = GetEntry();

	// Ensure only this entry is selected when opening context menu. When user has multiple rows selected, opening this menu will only be relevant to the selected entry
	BindingsList->SetSelection(MakeArrayView(&Entry, 1));

	FMenuBuilder MenuBuilder = BindingEntry::FRowHelper::CreateContextMenu(GetBlueprint(), GetBlueprintView(), MakeArrayView(&Entry, 1));
	
	{
		MenuBuilder.BeginSection("Developer", LOCTEXT("Developer", "Developer"));

		if (GetDefault<UMVVMDeveloperProjectSettings>()->bShowDeveloperGenerateGraphSettings)
		{
			bool bCanShowGraph = BindingEntry::FRowHelper::HasBlueprintGraph(GetBlueprintView(), Entry);

			FUIAction ShowGraphAction;
			ShowGraphAction.ExecuteAction = FExecuteAction::CreateSP(this, &SBaseRow::HandleShowBlueprintGraph);
			ShowGraphAction.CanExecuteAction = FCanExecuteAction::CreateLambda([bCanShowGraph]() { return bCanShowGraph; });
			MenuBuilder.AddMenuEntry(LOCTEXT("ShowGraph", "Show binding graph")
				, LOCTEXT("ShowGraphTooltip", "Show the Blueprint graph that represent the binding."
					" The graph is always generated but may not be visible to the user.")
				, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintEditor.FindInBlueprints.MenuIcon")
				, ShowGraphAction);
		}

		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void SBaseRow::HandleShowBlueprintGraph() const
{
	TSharedPtr<FBindingEntry> Entry = GetEntry();
	BindingEntry::FRowHelper::ShowBlueprintGraph(GetBlueprintEditor().Get(), GetBlueprint(), GetBlueprintView(), MakeArrayView(&Entry, 1));
}

} // namespace UE::MVVM::BindingEntry

#undef LOCTEXT_NAMESPACE
