// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/MVVMBindingEntry.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#include "MVVMEditorSubsystem.h"
#include "MVVMBlueprintView.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"

namespace UE::MVVM
{
class SBindingsList;
}

namespace UE::MVVM::BindingEntry
{

/**
 * 
 */
class SBaseRow : public STableRow<TSharedPtr<FBindingEntry>>
{
public:
	SLATE_BEGIN_ARGS(SBaseRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, SBindingsList* BindingsList, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FWidgetBlueprintEditor>& InBlueprintEditor, UWidgetBlueprint* InBlueprint, const TSharedPtr<FBindingEntry>& InEntry);

	TSharedPtr<FBindingEntry> GetEntry() const
	{
		return BindingEntry;
	}

	TSharedPtr<FWidgetBlueprintEditor> GetBlueprintEditor() const
	{
		return WeakBlueprintEditor.Pin();
	}
	
	UWidgetBlueprint* GetBlueprint() const;
	UMVVMBlueprintView* GetBlueprintView() const;
	UMVVMEditorSubsystem* GetEditorSubsystem() const;

protected:
	virtual TSharedRef<SWidget> BuildRowWidget() = 0;
	virtual const ANSICHAR* GetTableRowStyle() const = 0;
	virtual TSharedRef<SWidget> BuildContextMenuButton();

private:
	TSharedRef<SWidget> HandleContextMenu() const;
	void HandleShowBlueprintGraph() const;

	TSharedPtr<FBindingEntry> BindingEntry;
	TWeakObjectPtr<UWidgetBlueprint> WeakBlueprint;
	TWeakPtr<FWidgetBlueprintEditor> WeakBlueprintEditor;
	SBindingsList* BindingsList;
};

} // namespace UE::MVVM
