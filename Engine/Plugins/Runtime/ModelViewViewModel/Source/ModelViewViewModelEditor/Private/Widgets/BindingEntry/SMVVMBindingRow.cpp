// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/BindingEntry/SMVVMBindingRow.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Framework/MVVMRowHelper.h"
#include "MVVMBlueprintViewBinding.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMPropertyPath.h"

#include "Styling/AppStyle.h"
#include "Styling/MVVMEditorStyle.h"

#include "Dialog/SCustomDialog.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMVVMFieldSelector.h"
#include "Widgets/SMVVMSourceSelector.h"
#include "SSimpleButton.h"

#define LOCTEXT_NAMESPACE "BindingListView_BindingRow"

namespace UE::MVVM::BindingEntry
{

namespace Private
{

	TArray<FName>* GetBindingModeNames()
	{
		static TArray<FName> BindingModeNames;

		if (BindingModeNames.IsEmpty())
		{
			UEnum* ModeEnum = StaticEnum<EMVVMBindingMode>();

			BindingModeNames.Reserve(ModeEnum->NumEnums());

			for (int32 BindingIndex = 0; BindingIndex < ModeEnum->NumEnums() - 1; ++BindingIndex)
			{
				const bool bIsHidden = ModeEnum->HasMetaData(TEXT("Hidden"), BindingIndex);
				if (!bIsHidden)
				{
					BindingModeNames.Add(ModeEnum->GetNameByIndex(BindingIndex));
				}
			}
		}

		return &BindingModeNames;
	}

} // namespace

void SBindingRow::Construct(const FArguments& Args, SBindingsList* InBindingsList, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FWidgetBlueprintEditor>& InBlueprintEditor, UWidgetBlueprint* InBlueprint, const TSharedPtr<FBindingEntry>& InEntry)
{
	static IConsoleVariable* CVarDefaultExecutionMode = IConsoleManager::Get().FindConsoleVariable(TEXT("MVVM.DefaultExecutionMode"));
	ensure(CVarDefaultExecutionMode);
	DefaultExecutionMode = CVarDefaultExecutionMode ? (EMVVMExecutionMode)CVarDefaultExecutionMode->GetInt() : EMVVMExecutionMode::DelayedWhenSharedElseImmediate;

	SBaseRow::Construct(SBaseRow::FArguments(), InBindingsList, OwnerTableView, InBlueprintEditor, InBlueprint, InEntry);

	TSharedPtr<SWidget> ChildContent = ChildSlot.DetachWidget();
	ChildSlot
	[
		// Add a single pixel top and bottom border for this widget.
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.Padding(0.0f, 2.0f, 0.0f, 1.0f)
		[
			// Restore the border that we're meant to have that reacts to selection/hover/etc.
			SNew(SBorder)
			.BorderImage(this, &SBindingRow::GetBorderImage)
			.Padding(0.0f)
			[
				ChildContent.ToSharedRef()
			]
		]
	];
}

TSharedRef<SWidget> SBindingRow::BuildRowWidget()
{
	FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding();
	
	return SNew(SBorder)
	.BorderImage(FAppStyle::Get().GetBrush("PlainBorder"))
	.Padding(0)
	.BorderBackgroundColor(this, &SBindingRow::GetErrorBorderColor)
	[
		SNew(SBox)
		.HeightOverride(30)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(2, 0)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SBindingRow::IsBindingCompiled)
				.OnCheckStateChanged(this, &SBindingRow::OnIsBindingCompileChanged)
				.ToolTipText(LOCTEXT("MVVMCompileTooltip", "Is compile"))
			]

			+ SHorizontalBox::Slot()
			.Padding(4.f, 0.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(150)
				[
					SNew(SFieldSelector, GetBlueprint())
					.OnGetLinkedValue(this, &SBindingRow::GetFieldSelectedValue, false)
					.OnSelectionChanged(this, &SBindingRow::HandleFieldSelectionChanged, false)
					.OnGetSelectionContext(this, &SBindingRow::GetSelectedSelectionContext, false)
					.OnDrop(this, &SBindingRow::HandleFieldSelectorDrop, false)
					.OnDragOver(this, &SBindingRow::HandleFieldSelectorDragOver, false)
					.ShowContext(false)
					.CanCreateEvent(true)
					.ToolTipText(FText::Join(FText::FromString("\n"), LOCTEXT("MVVMDestTooltipHeader", "Destination"), LOCTEXT("MVVMDestTooltipBody", "The property to hook data into.")))
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(4.f, 0.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SComboBox<FName>)
				.OptionsSource(Private::GetBindingModeNames())
				.InitiallySelectedItem(StaticEnum<EMVVMBindingMode>()->GetNameByValue((int64)ViewBinding->BindingType))
				.OnSelectionChanged(this, &SBindingRow::OnBindingModeSelectionChanged)
				.OnGenerateWidget(this, &SBindingRow::GenerateBindingModeWidget)
				.ToolTipText(this, &SBindingRow::GetCurrentBindingModeLabel)
				.Content()
				[
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.WidthOverride(16.0f)
					.HeightOverride(16.0f)
					[
						SNew(SImage)
						.Image(this, &SBindingRow::GetCurrentBindingModeBrush)
					]
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(4.f, 0.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(150.0f)
				[
					SNew(SFieldSelector, GetBlueprint())
					.OnGetLinkedValue(this, &SBindingRow::GetFieldSelectedValue, true)
					.OnSelectionChanged(this, &SBindingRow::HandleFieldSelectionChanged, true)
					.OnGetSelectionContext(this, &SBindingRow::GetSelectedSelectionContext, true)
					.OnDrop(this, &SBindingRow::HandleFieldSelectorDrop, true)
					.OnDragOver(this, &SBindingRow::HandleFieldSelectorDragOver, true)
					.ToolTipText(FText::Join(FText::FromString("\n"), LOCTEXT("MVVMSrcTooltipHeader", "Source"), LOCTEXT("MVVMSrcTooltipBody", "The source of data to bind to.")))
				]
			]

			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]

			+ SHorizontalBox::Slot()
			.Padding(2.0f, 1.0f)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SBindingRow::IsExecutionModeOverrideChecked)
				.OnCheckStateChanged(this, &SBindingRow::OnExecutionModeOverrideChanged)
			]

			+ SHorizontalBox::Slot()
			.Padding(2.0f, 1.0f)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SComboButton)
				.ContentPadding(FMargin(4.f, 0.f))
				.OnGetMenuContent(this, &SBindingRow::OnGetExecutionModeMenuContent)
				.IsEnabled(this, &SBindingRow::IsExecutionModeOverridden)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &SBindingRow::GetExecutioModeValue)
					.ToolTipText(this, &SBindingRow::GetExecutioModeValueToolTip)
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SSimpleButton)
				.Icon(FAppStyle::Get().GetBrush("Icons.Error"))
				.Visibility(this, &SBindingRow::GetErrorButtonVisibility)
				.ToolTipText(this, &SBindingRow::GetErrorButtonToolTip)
				.OnClicked(this, &SBindingRow::OnErrorButtonClicked)
			]

			+ SHorizontalBox::Slot()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				BuildContextMenuButton()
			]
		]
	];
}

FMVVMBlueprintViewBinding* SBindingRow::GetThisViewBinding() const
{
	UMVVMBlueprintView* BlueprintView = GetBlueprintView();
	return BlueprintView ? GetEntry()->GetBinding(BlueprintView) : nullptr;
}

FSlateColor SBindingRow::GetErrorBorderColor() const
{
	if (const FMVVMBlueprintViewBinding* ViewModelBinding = GetThisViewBinding())
	{
		UMVVMBlueprintView* BlueprintView = GetBlueprintView();
		if (BlueprintView->HasBindingMessage(ViewModelBinding->BindingId, EBindingMessageType::Error))
		{
			return FStyleColors::Error;
		}
		else if (BlueprintView->HasBindingMessage(ViewModelBinding->BindingId, EBindingMessageType::Warning))
		{
			return FStyleColors::Warning;
		}
	}
	return FStyleColors::Transparent;
}

ECheckBoxState SBindingRow::IsBindingEnabled() const
{
	if (const FMVVMBlueprintViewBinding* ViewModelBinding = GetThisViewBinding())
	{
		return ViewModelBinding->bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

ECheckBoxState SBindingRow::IsBindingCompiled() const
{
	if (const FMVVMBlueprintViewBinding* ViewModelBinding = GetThisViewBinding())
	{
		return ViewModelBinding->bCompile ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

EVisibility SBindingRow::GetErrorButtonVisibility() const
{
	if (const FMVVMBlueprintViewBinding* ViewModelBinding = GetThisViewBinding())
	{
		UMVVMBlueprintView* BlueprintView = GetBlueprintView();
		const bool HasBindingMessage = BlueprintView->HasBindingMessage(ViewModelBinding->BindingId);
		return HasBindingMessage ? EVisibility::Visible : EVisibility::Hidden;
	}
	return EVisibility::Collapsed;
}

FText SBindingRow::GetErrorButtonToolTip() const
{
	// Get error messages of this binding
	if (const FMVVMBlueprintViewBinding* ViewModelBinding = GetThisViewBinding())
	{
		UMVVMBlueprintView* BlueprintView = GetBlueprintView();
		TArray<FText> BindingErrorList = BlueprintView->GetBindingMessages(ViewModelBinding->BindingId, EBindingMessageType::Error);
		TArray<FText> BindingWarningList = BlueprintView->GetBindingMessages(ViewModelBinding->BindingId, EBindingMessageType::Warning);
		TArray<FText> BindingInfoList = BlueprintView->GetBindingMessages(ViewModelBinding->BindingId, EBindingMessageType::Info);
		BindingErrorList.Append(BindingWarningList);
		BindingErrorList.Append(BindingInfoList);

		static const FText NewLineText = FText::FromString(TEXT("\n"));
		FText HintText = LOCTEXT("ErrorButtonText", "Messages: (Click to show in a separate window)");
		FText ErrorsText = FText::Join(NewLineText, BindingErrorList);
		return FText::Join(NewLineText, HintText, ErrorsText);
	}
	return FText();
}

FReply SBindingRow::OnErrorButtonClicked()
{
	ErrorItems.Reset();

	const UWidgetBlueprint* WidgetBlueprint = GetBlueprint();
	const UMVVMBlueprintView* View = GetBlueprintView();
	const FMVVMBlueprintViewBinding* ViewBinding = View ? GetEntry()->GetBinding(View) : nullptr;
	if (WidgetBlueprint && View && ViewBinding)
	{
		for (const FText& ErrorText : View->GetBindingMessages(ViewBinding->BindingId, EBindingMessageType::Error))
		{
			ErrorItems.Add(MakeShared<FText>(ErrorText));
		}

		for (const FText& WarningText : View->GetBindingMessages(ViewBinding->BindingId, EBindingMessageType::Warning))
		{
			ErrorItems.Add(MakeShared<FText>(WarningText));
		}

		for (const FText& InfoText : View->GetBindingMessages(ViewBinding->BindingId, EBindingMessageType::Info))
		{
			ErrorItems.Add(MakeShared<FText>(InfoText));
		}

		const FText BindingDisplayName = FText::FromString(ViewBinding->GetDisplayNameString(WidgetBlueprint));
		TSharedRef<SCustomDialog> ErrorDialog = SNew(SCustomDialog)
			.Title(FText::Format(LOCTEXT("Compilation Messages", "Compilation messages for {0}"), BindingDisplayName))
			.Buttons({
				SCustomDialog::FButton(LOCTEXT("OK", "OK"))
				})
			.Content()
			[
				SNew(SListView<TSharedPtr<FText>>)
				.ListItemsSource(&ErrorItems)
				.OnGenerateRow(this, &SBindingRow::OnGenerateErrorRow)
			];

		ErrorDialog->Show();
	}

	return FReply::Handled();
}

TSharedRef<ITableRow> SBindingRow::OnGenerateErrorRow(TSharedPtr<FText> Text, const TSharedRef<STableViewBase>& TableView) const
{
	return SNew(STableRow<TSharedPtr<FText>>, TableView)
	.Content()
	[
		SNew(SBorder)
		.BorderBackgroundColor(FStyleColors::Background)
		[
			SNew(STextBlock)
			.Text(*Text.Get())
		]
	];
}

TArray<FBindingSource> SBindingRow::GetAvailableViewModels() const
{
	UMVVMEditorSubsystem* EditorSubsystem = GetEditorSubsystem();
	return EditorSubsystem->GetAllViewModels(GetBlueprint());
}

FMVVMLinkedPinValue SBindingRow::GetFieldSelectedValue(bool bSourceToDest) const
{
	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{

		if (UMVVMBlueprintViewConversionFunction* ConversionFunction = ViewBinding->Conversion.GetConversionFunction(bSourceToDest))
		{
			if (ConversionFunction->GetConversionFunction().GetType() == EMVVMBlueprintFunctionReferenceType::Function)
			{
				return FMVVMLinkedPinValue(ConversionFunction->GetConversionFunction().GetFunction(GetBlueprint()));
			}
			else if (ConversionFunction->GetConversionFunction().GetType() == EMVVMBlueprintFunctionReferenceType::Node)
			{
				return FMVVMLinkedPinValue(ConversionFunction->GetConversionFunction().GetNode());
			}
		}
		else
		{
			return FMVVMLinkedPinValue(bSourceToDest ? ViewBinding->SourcePath : ViewBinding->DestinationPath);
		}
	}
	return FMVVMLinkedPinValue();
}

void SBindingRow::HandleFieldSelectionChanged(FMVVMLinkedPinValue Value, SFieldSelectorMenu::ESelectionType SelectionType, bool bSource)
{
	UWidgetBlueprint* WidgetBlueprint = GetBlueprint();
	FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding();
	if (WidgetBlueprint && ViewBinding)
	{
		UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
		if (bSource)
		{
			if (Value.IsPropertyPath())
			{
				Subsystem->SetSourcePathForBinding(WidgetBlueprint, *ViewBinding, Value.GetPropertyPath());
			}
			else if (Value.IsConversionFunction())
			{
				Subsystem->SetSourceToDestinationConversionFunction(WidgetBlueprint, *ViewBinding, FMVVMBlueprintFunctionReference(WidgetBlueprint, Value.GetConversionFunction()));
			}
			else if (Value.IsConversionNode())
			{
				Subsystem->SetSourceToDestinationConversionFunction(WidgetBlueprint, *ViewBinding, FMVVMBlueprintFunctionReference(Value.GetConversionNode()));
			}
			else
			{
				Subsystem->SetSourcePathForBinding(WidgetBlueprint, *ViewBinding, FMVVMBlueprintPropertyPath());
			}
		}
		else
		{
			if (Value.IsPropertyPath())
			{
				const bool bAllowEventConversion = SelectionType == SFieldSelectorMenu::ESelectionType::Event;
				Subsystem->SetDestinationPathForBinding(WidgetBlueprint, *ViewBinding, Value.GetPropertyPath(), bAllowEventConversion);
			}
			else if (Value.IsConversionFunction())
			{
				Subsystem->SetDestinationToSourceConversionFunction(WidgetBlueprint, *ViewBinding, FMVVMBlueprintFunctionReference(WidgetBlueprint, Value.GetConversionFunction()));
			}
			else if (Value.IsConversionNode())
			{
				Subsystem->SetDestinationToSourceConversionFunction(WidgetBlueprint, *ViewBinding, FMVVMBlueprintFunctionReference(Value.GetConversionNode()));
			}
			else
			{
				Subsystem->SetDestinationPathForBinding(WidgetBlueprint, *ViewBinding, FMVVMBlueprintPropertyPath(), false);
			}
		}
	}
}

FFieldSelectionContext SBindingRow::GetSelectedSelectionContext(bool bSource) const
{
	FFieldSelectionContext Result;

	const UWidgetBlueprint* WidgetBlueprint = GetBlueprint();
	const FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding();
	if (WidgetBlueprint == nullptr || ViewBinding == nullptr)
	{
		return Result;
	}

	Result.BindingMode = ViewBinding->BindingType;

	{
		TArray<FMVVMConstFieldVariant> Fields = bSource ? ViewBinding->DestinationPath.GetFields(WidgetBlueprint->SkeletonGeneratedClass) : ViewBinding->SourcePath.GetFields(WidgetBlueprint->SkeletonGeneratedClass);
		if (Fields.Num() > 0)
		{
			FMVVMConstFieldVariant LastField = Fields.Last();
			if (LastField.IsProperty())
			{
				Result.AssignableTo = LastField.GetProperty();
			}
			else if (LastField.IsFunction())
			{
				if (const UFunction* Function = LastField.GetFunction())
				{
					Result.AssignableTo = BindingHelper::GetReturnProperty(Function);
				}
			}
		}
	}

	if (!bSource && !ViewBinding->DestinationPath.GetWidgetName().IsNone())
	{
		Result.FixedBindingSource = FBindingSource::CreateForWidget(WidgetBlueprint, ViewBinding->DestinationPath.GetWidgetName());
	}

	Result.bAllowWidgets = true;
	Result.bAllowViewModels = bSource;
	Result.bAllowConversionFunctions = false;
	bool bIsReadingValue = (IsForwardBinding(ViewBinding->BindingType) && bSource)
		|| (IsBackwardBinding(ViewBinding->BindingType) && !bSource);
	if (!(IsBackwardBinding(ViewBinding->BindingType) && IsForwardBinding(ViewBinding->BindingType)))
	{
		Result.bAllowConversionFunctions = bIsReadingValue;
	}

	Result.bReadable = bIsReadingValue;
	Result.bWritable = (IsForwardBinding(ViewBinding->BindingType) && !bSource)
		|| (IsBackwardBinding(ViewBinding->BindingType) && bSource);
	
	return Result;
}

FReply SBindingRow::HandleFieldSelectorDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bSource)
{
	FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding();
	UWidgetBlueprint* WidgetBlueprint = GetBlueprint();
	if (ViewBinding == nullptr)
	{
		return FReply::Unhandled();
	}

	TOptional<FMVVMBlueprintPropertyPath> PropertyPath = BindingEntry::FRowHelper::DropFieldSelector(WidgetBlueprint, DragDropEvent, bSource);
	if (!PropertyPath.IsSet())
	{
		return FReply::Handled();
	}

	UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
	if (bSource)
	{
		Subsystem->SetSourcePathForBinding(WidgetBlueprint, *ViewBinding, PropertyPath.GetValue());
	}
	else
	{
		Subsystem->SetDestinationPathForBinding(WidgetBlueprint, *ViewBinding, PropertyPath.GetValue(), false);
	}
	return FReply::Handled();
}

FReply SBindingRow::HandleFieldSelectorDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bSource)
{
	return BindingEntry::FRowHelper::DragOverFieldSelector(GetBlueprint(), DragDropEvent, bSource);
}

ECheckBoxState SBindingRow::IsExecutionModeOverrideChecked() const
{
	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{
		return ViewBinding->bOverrideExecutionMode ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void SBindingRow::OnExecutionModeOverrideChanged(ECheckBoxState NewState)
{
	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{
		UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
		if (NewState == ECheckBoxState::Checked)
		{
			Subsystem->OverrideExecutionModeForBinding(GetBlueprint(), *ViewBinding, DefaultExecutionMode);
		}
		else
		{
			Subsystem->ResetExecutionModeForBinding(GetBlueprint(), *ViewBinding);
		}
	}
}

void SBindingRow::OnExecutionModeSelectionChanged(EMVVMExecutionMode Value)
{
	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{
		UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
		Subsystem->OverrideExecutionModeForBinding(GetBlueprint(), *ViewBinding, Value);
	}
}

TSharedRef<SWidget> SBindingRow::OnGetExecutionModeMenuContent()
{
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr, nullptr, true);

	UEnum* Enum = StaticEnum<EMVVMExecutionMode>();
	for (int32 Index = 0; Index < Enum->NumEnums() - 1; Index++)
	{
		EMVVMExecutionMode Mode = static_cast<EMVVMExecutionMode>(Enum->GetValueByIndex(Index));
		if (!GetDefault<UMVVMDeveloperProjectSettings>()->IsExecutionModeAllowed(Mode))
		{
			continue;
		}

		MenuBuilder.AddMenuEntry(
			Enum->GetDisplayNameTextByIndex(Index),
			Enum->GetToolTipTextByIndex(Index),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateLambda([this, Mode]()
					{
						OnExecutionModeSelectionChanged(Mode);
					})
			)
		);
	}

	return MenuBuilder.MakeWidget();
}

FText SBindingRow::GetExecutioModeValue() const
{
	EMVVMExecutionMode ExecutionMode = DefaultExecutionMode;
	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{
		if (ViewBinding->bOverrideExecutionMode)
		{
			ExecutionMode = ViewBinding->OverrideExecutionMode;
		}
	}
	return StaticEnum<EMVVMExecutionMode>()->GetDisplayNameTextByValue(static_cast<int64>(ExecutionMode));
}

FText SBindingRow::GetExecutioModeValueToolTip() const
{
	EMVVMExecutionMode ExecutionMode = DefaultExecutionMode;
	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{
		if (ViewBinding->bOverrideExecutionMode)
		{
			ExecutionMode = ViewBinding->OverrideExecutionMode;
		}
	}
	return StaticEnum<EMVVMExecutionMode>()->GetToolTipTextByIndex(static_cast<int64>(ExecutionMode));
}

bool SBindingRow::IsExecutionModeOverridden() const
{
	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{
		return ViewBinding->bOverrideExecutionMode;
	}
	return false;
}

void SBindingRow::OnIsBindingEnableChanged(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Undetermined)
	{
		return;
	}

	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{
		UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
		Subsystem->SetEnabledForBinding(GetBlueprint(), *ViewBinding, NewState == ECheckBoxState::Checked);
	}
}

void SBindingRow::OnIsBindingCompileChanged(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Undetermined)
	{
		return;
	}

	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{
		UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
		Subsystem->SetCompileForBinding(GetBlueprint(), *ViewBinding, NewState == ECheckBoxState::Checked);
	}
}

const FSlateBrush* SBindingRow::GetBindingModeBrush(EMVVMBindingMode BindingMode) const
{
	switch (BindingMode)
	{
	case EMVVMBindingMode::OneTimeToDestination:
		return FMVVMEditorStyle::Get().GetBrush("BindingMode.OneTimeOneWay");
	case EMVVMBindingMode::OneWayToDestination:
		return FMVVMEditorStyle::Get().GetBrush("BindingMode.OneWayToSource");
	case EMVVMBindingMode::OneWayToSource:
		return FMVVMEditorStyle::Get().GetBrush("BindingMode.OneWay");
	case EMVVMBindingMode::OneTimeToSource:
		return nullptr;
	case EMVVMBindingMode::TwoWay:
		return FMVVMEditorStyle::Get().GetBrush("BindingMode.TwoWay");
	default:
		return nullptr;
	}
}

const FSlateBrush* SBindingRow::GetCurrentBindingModeBrush() const
{
	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{
		return GetBindingModeBrush(ViewBinding->BindingType);
	}
	return nullptr;
}

FText SBindingRow::GetCurrentBindingModeLabel() const
{
	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{
		return FText::Join(FText::FromString(TEXT("\n")), LOCTEXT("BindingMode", "Mode"), GetBindingModeLabel(ViewBinding->BindingType));
	}
	return FText::GetEmpty();
}

FText SBindingRow::GetBindingModeLabel(EMVVMBindingMode BindingMode) const
{
	static FText OneTimeToDestinationLabel = LOCTEXT("OneTimeToDestinationLabel", "One Time To Widget");
	static FText OneWayToDestinationLabel = LOCTEXT("OneWayToDestinationLabel", "One Way To Widget");
	static FText OneWayToSourceLabel = LOCTEXT("OneWayToSourceLabel", "One Way To View Model");
	static FText OneTimeToSourceLabel = LOCTEXT("OneTimeToSourceLabel", "One Time To View Model");
	static FText TwoWayLabel = LOCTEXT("TwoWayLabel", "Two Way");

	switch (BindingMode)
	{
	case EMVVMBindingMode::OneTimeToDestination:
		return OneTimeToDestinationLabel;
	case EMVVMBindingMode::OneWayToDestination:
		return OneWayToDestinationLabel;
	case EMVVMBindingMode::OneWayToSource:
		return OneWayToSourceLabel;
	case EMVVMBindingMode::OneTimeToSource:
		return OneTimeToSourceLabel;
	case EMVVMBindingMode::TwoWay:
		return TwoWayLabel;
	default:
		return FText::GetEmpty();
	}
}

TSharedRef<SWidget> SBindingRow::GenerateBindingModeWidget(FName ValueName) const
{
	const UEnum* ModeEnum = StaticEnum<EMVVMBindingMode>();
	int32 Index = ModeEnum->GetIndexByName(ValueName);
	EMVVMBindingMode MVVMBindingMode = EMVVMBindingMode(Index);
	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	[
		SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.WidthOverride(16.0f)
		.HeightOverride(16.0f)
		[
			SNew(SImage)
			.Image(GetBindingModeBrush(MVVMBindingMode))
		]
	]
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(2.0f, 0.0f, 0.0f, 0.0f)
	[
		SNew(STextBlock)
		.Text(GetBindingModeLabel(MVVMBindingMode))
		.ToolTipText(ModeEnum->GetToolTipTextByIndex(Index))
	];
}

void SBindingRow::OnBindingModeSelectionChanged(FName ValueName, ESelectInfo::Type)
{
	if (FMVVMBlueprintViewBinding* ViewBinding = GetThisViewBinding())
	{
		const UEnum* ModeEnum = StaticEnum<EMVVMBindingMode>();
		EMVVMBindingMode NewMode = (EMVVMBindingMode)ModeEnum->GetValueByName(ValueName);

		UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
		Subsystem->SetBindingTypeForBinding(GetBlueprint(), *ViewBinding, NewMode);
	}
}

} // namespace

#undef LOCTEXT_NAMESPACE
