// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Types/MVVMBindingSource.h"
#include "Types/MVVMExecutionMode.h"
#include "Widgets/BindingEntry/SMVVMBaseRow.h"
#include "Widgets/SMVVMFieldSelectorMenu.h"

struct FMVVMBlueprintViewBinding;

namespace UE::MVVM
{
class SBindingsList;
}

namespace UE::MVVM::BindingEntry
{

/**
 *
 */
class SBindingRow : public SBaseRow
{
public:
	SLATE_BEGIN_ARGS(SBindingRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, SBindingsList* BindingsList, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FWidgetBlueprintEditor>& InBlueprintEditor, UWidgetBlueprint* InBlueprint, const TSharedPtr<FBindingEntry>& InEntry);

protected:
	virtual TSharedRef<SWidget> BuildRowWidget() override;
	virtual const ANSICHAR* GetTableRowStyle() const override
	{
		return "BindingView.BindingRow";
	}

private:
	FMVVMBlueprintViewBinding* GetThisViewBinding() const;

	FSlateColor GetErrorBorderColor() const;

	ECheckBoxState IsBindingEnabled() const;

	ECheckBoxState IsBindingCompiled() const;

	EVisibility GetErrorButtonVisibility() const;

	FText GetErrorButtonToolTip() const;

	FReply OnErrorButtonClicked();

	TSharedRef<ITableRow> OnGenerateErrorRow(TSharedPtr<FText> Text, const TSharedRef<STableViewBase>& TableView) const;

	TArray<FBindingSource> GetAvailableViewModels() const;

	FMVVMLinkedPinValue GetFieldSelectedValue(bool bSource) const;

	void HandleFieldSelectionChanged(FMVVMLinkedPinValue Value, SFieldSelectorMenu::ESelectionType SelectionType, bool bSource);

	FFieldSelectionContext GetSelectedSelectionContext(bool bSource) const;

	FReply HandleFieldSelectorDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bSource);

	FReply HandleFieldSelectorDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bSource);

	ECheckBoxState IsExecutionModeOverrideChecked() const;

	void OnExecutionModeOverrideChanged(ECheckBoxState NewState);

	void OnExecutionModeSelectionChanged(EMVVMExecutionMode Value);

	TSharedRef<SWidget> OnGetExecutionModeMenuContent();

	FText GetExecutioModeValue() const;

	FText GetExecutioModeValueToolTip() const;

	bool IsExecutionModeOverridden() const;

	void OnIsBindingEnableChanged(ECheckBoxState NewState);

	void OnIsBindingCompileChanged(ECheckBoxState NewState);

	const FSlateBrush* GetBindingModeBrush(EMVVMBindingMode BindingMode) const;

	const FSlateBrush* GetCurrentBindingModeBrush() const;

	FText GetCurrentBindingModeLabel() const;

	FText GetBindingModeLabel(EMVVMBindingMode BindingMode) const;

	TSharedRef<SWidget> GenerateBindingModeWidget(FName ValueName) const;

	void OnBindingModeSelectionChanged(FName ValueName, ESelectInfo::Type);

private:
	TArray<TSharedPtr<FText>> ErrorItems;
	EMVVMExecutionMode DefaultExecutionMode = EMVVMExecutionMode::DelayedWhenSharedElseImmediate;
};

} // namespace UE::MVVM
