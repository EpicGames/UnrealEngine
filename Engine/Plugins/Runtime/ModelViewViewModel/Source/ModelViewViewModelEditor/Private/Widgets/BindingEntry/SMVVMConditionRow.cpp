// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/BindingEntry/SMVVMConditionRow.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Framework/MVVMRowHelper.h"
#include "MVVMBlueprintViewCondition.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMPropertyPath.h"

#include "Styling/AppStyle.h"
#include "Styling/MVVMEditorStyle.h"

#include "Dialog/SCustomDialog.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMVVMFieldSelector.h"
#include "Widgets/SMVVMSourceSelector.h"
#include "SSimpleButton.h"

#define LOCTEXT_NAMESPACE "BindingListView_ConditionRow"

namespace UE::MVVM::BindingEntry
{
	namespace Private
	{
		TArray<FName>* GetConditionOperationNames()
		{
			static TArray<FName> ConditionOperationNames;

			if (ConditionOperationNames.IsEmpty())
			{
				UEnum* OperationEnum = StaticEnum<EMVVMConditionOperation>();

				ConditionOperationNames.Reserve(OperationEnum->NumEnums());

				for (int32 Index = 0; Index < OperationEnum->NumEnums() - 1; ++Index)
				{
					ConditionOperationNames.Add(OperationEnum->GetNameByIndex(Index));
				}
			}

			return &ConditionOperationNames;
		}
	}

void SConditionRow::Construct(const FArguments& Args, SBindingsList* InBindingsList, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FWidgetBlueprintEditor>& InBlueprintEditor, UWidgetBlueprint* InBlueprint, const TSharedPtr<FBindingEntry>& InEntry)
{
	SBaseRow::Construct(SBaseRow::FArguments(), InBindingsList, OwnerTableView, InBlueprintEditor, InBlueprint, InEntry);

	TSharedPtr<SWidget> ChildContent = ChildSlot.DetachWidget();
	ChildSlot
	[
		// Add a single pixel top and bottom border for this widget.
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.Padding(0.0f, 2.0f, 0.0f, 1.0f)
		[
			// Restore the border that we're meant to have that reacts to selection/hover/etc.
			SNew(SBorder)
			.BorderImage(this, &SConditionRow::GetBorderImage)
			.Padding(0.0f)
			[
				ChildContent.ToSharedRef()
			]
		]
	];
}

TSharedRef<SWidget> SConditionRow::BuildRowWidget()
{
	UMVVMBlueprintViewCondition* Condition = GetCondition();

	return SNew(SBorder)
	.BorderImage(FAppStyle::Get().GetBrush("PlainBorder"))
	.Padding(0.0f)
	.BorderBackgroundColor(this, &SConditionRow::GetErrorBorderColor)
	[
		SNew(SBox)
		.HeightOverride(30.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SConditionRow::IsConditionCompiled)
				.OnCheckStateChanged(this, &SConditionRow::OnIsConditionCompileChanged)
			]

			+ SHorizontalBox::Slot()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("WhenTextBlock",
						"When"))
			]

			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(150.0f)
				[
					SNew(SFieldSelector, GetBlueprint())
						.OnGetLinkedValue(this, &SConditionRow::GetFieldSelectedValue, true)
						.OnSelectionChanged(this, &SConditionRow::HandleFieldSelectionChanged, true)
						.OnGetSelectionContext(this, &SConditionRow::GetSelectedSelectionContext, true)
						.OnDrop(this, &SConditionRow::HandleFieldSelectorDrop, true)
						.OnDragOver(this, &SConditionRow::HandleFieldSelectorOver, true)
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("IsTextBlock",
						"is"))
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(150.0f)
				[
					SNew(SComboBox<FName>)
						.OptionsSource(Private::GetConditionOperationNames())
						.InitiallySelectedItem(StaticEnum<EMVVMConditionOperation>()->GetNameByValue((int64)Condition->GetOperation()))
						.OnSelectionChanged(this, &SConditionRow::OnConditionOperationSelectionChanged)
						.OnGenerateWidget(this, &SConditionRow::GenerateConditionOperationWidget)
						.ToolTipText(this, &SConditionRow::GetCurrentConditionOperationLabel)
						.Content()
						[
							SNew(STextBlock)
								.Text(this, &SConditionRow::GetCurrentConditionOperationLabel)
								.ToolTipText(this, &SConditionRow::GetCurrentConditionOperationLabel)
						]
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SNumericEntryBox<float>)
						.AllowSpin(false)
						.Value(this, &SConditionRow::GetValue)
						.OnValueChanged(this, &SConditionRow::OnValueChanged)
						.ToolTipText(this, &SConditionRow::GetValueTooltipFromOperation)
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
					.Visibility(this, &SConditionRow::GetMaxValueVisibilityFromOperation)
					.WidthOverride(100.0f)
					[
						SNew(SNumericEntryBox<float>)
							.AllowSpin(false)
							.Value(this, &SConditionRow::GetMaxValue)
							.OnValueChanged(this, &SConditionRow::OnMaxValueChanged)
							.ToolTipText(LOCTEXT("UpperBoundTooltip","Between upper bound value"))
					]
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.WidthOverride(16.0f)
				.HeightOverride(16.0f)
				[
					SNew(SImage)
					.Image(FMVVMEditorStyle::Get().GetBrush("BindingMode.OneWay"))
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(150.0f)
				[
					SNew(SFieldSelector, GetBlueprint())
					.OnGetLinkedValue(this, &SConditionRow::GetFieldSelectedValue, false)
					.OnSelectionChanged(this, &SConditionRow::HandleFieldSelectionChanged, false)
					.OnGetSelectionContext(this, &SConditionRow::GetSelectedSelectionContext, false)
					.OnDrop(this, &SConditionRow::HandleFieldSelectorDrop, false)
					.OnDragOver(this, &SConditionRow::HandleFieldSelectorOver, false)
					.IsBindingToEvent(true)
				]
			]

			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]

			+ SHorizontalBox::Slot()
			.Padding(0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SSimpleButton)
				.Icon(FAppStyle::Get().GetBrush("Icons.Error"))
				.Visibility(this, &SConditionRow::GetErrorButtonVisibility)
				.ToolTipText(this, &SConditionRow::GetErrorButtonToolTip)
				.OnClicked(this, &SConditionRow::OnErrorButtonClicked)
			]

			+ SHorizontalBox::Slot()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				BuildContextMenuButton()
			]
		]
	];
}

TOptional<float> SConditionRow::GetValue() const
{
	if (UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		return Condition->GetOperationValue();
	}
	return TOptional<float>();
}

TOptional<float> SConditionRow::GetMaxValue() const
{
	if (UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		return Condition->GetOperationMaxValue();
	}
	return TOptional<float>();
}

void SConditionRow::OnValueChanged(float InValue)
{
	if (UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		if (Condition->GetOperationValue() != InValue)
		{
			UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
			Subsystem->SetConditionOperationValue(Condition, InValue);
		}
	}
}

void SConditionRow::OnMaxValueChanged(float InValue)
{
	if (UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		if (Condition->GetOperationMaxValue() != InValue)
		{
			UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
			Subsystem->SetConditionOperationMaxValue(Condition, InValue);
		}
	}
}

EVisibility SConditionRow::GetMaxValueVisibilityFromOperation() const
{
	if (UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		EMVVMConditionOperation Operation = Condition->GetOperation();
		if (Operation == EMVVMConditionOperation::BetweenInclusive || Operation == EMVVMConditionOperation::BetweenExclusive)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

FText SConditionRow::GetValueTooltipFromOperation() const
{
	if (UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		EMVVMConditionOperation Operation = Condition->GetOperation();
		if (Operation == EMVVMConditionOperation::BetweenInclusive || Operation == EMVVMConditionOperation::BetweenExclusive)
		{
			return FText(LOCTEXT("LowerBoundTooltip", "Between lower bound value"));
		}
	}
	return FText(LOCTEXT("ComparisonValueTooltip", "Compared value"));
}


EMVVMConditionOperation SConditionRow::GetConditionOperationFromValueName(FName ValueName)
{
	const UEnum* Operation = StaticEnum<EMVVMConditionOperation>();
	int32 Index = Operation->GetIndexByName(ValueName);
	return  EMVVMConditionOperation(Index);
}

TSharedRef<SWidget> SConditionRow::GenerateConditionOperationWidget(FName ValueName) const
{
	const UEnum* Operation = StaticEnum<EMVVMConditionOperation>();
	int32 Index = Operation->GetIndexByName(ValueName);
	EMVVMConditionOperation MVVMOperation = EMVVMConditionOperation(Index);
	return SNew(SBox)
		[
			SNew(STextBlock)
				.Text(GetConditionOperationLabel(MVVMOperation))
				.ToolTipText(Operation->GetToolTipTextByIndex(Index))
		];
}

FText SConditionRow::GetConditionOperationLabel(EMVVMConditionOperation Operation) const
{
	static FText EqualLabel = LOCTEXT("EqualLabel", "Equal (==)");
	static FText NotEqualLabel = LOCTEXT("NotEqualLabel", "Not Equal (!=)");
	static FText MoreThanLabel = LOCTEXT("MoreThanLabel", "More Than (>)");
	static FText MoreThanOrEqualLabel = LOCTEXT("MoreThanOrEqualLabel", "More Than or Equal (>=)");
	static FText LessThanLabel = LOCTEXT("LessThanLabel", "Less Than (<)");
	static FText LessThanOrEqualLabel = LOCTEXT("LessThanOrEqualLabel", "Less Than or Equal (<=)");
	static FText BetweenInclusiveLabel = LOCTEXT("BetweenInclusiveLabel", "Between (Including)");
	static FText BetweenExclusiveLabel = LOCTEXT("BetweenExclusiveLabel", "Between (Excluding)");

	switch (Operation)
	{
	case EMVVMConditionOperation::Equal:
		return EqualLabel;
	case EMVVMConditionOperation::NotEqual:
		return NotEqualLabel;
	case EMVVMConditionOperation::MoreThan:
		return MoreThanLabel;
	case EMVVMConditionOperation::MoreThanOrEqual:
		return MoreThanOrEqualLabel;
	case EMVVMConditionOperation::LessThan:
		return LessThanLabel;
	case EMVVMConditionOperation::LessThanOrEqual:
		return LessThanOrEqualLabel;
	case EMVVMConditionOperation::BetweenInclusive:
		return BetweenInclusiveLabel;
	case EMVVMConditionOperation::BetweenExclusive:
		return BetweenExclusiveLabel;
	}
	return FText::GetEmpty();
}

FText SConditionRow::GetCurrentConditionOperationLabel() const
{
	if (UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		return GetConditionOperationLabel(Condition->GetOperation());
	}
	return FText::GetEmpty();
}

void SConditionRow::OnConditionOperationSelectionChanged(FName ValueName, ESelectInfo::Type)
{
	if (UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		EMVVMConditionOperation Operation = GetConditionOperationFromValueName(ValueName);

		if (Condition->GetOperation() != Operation)
		{
			UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
			Subsystem->SetConditionOperation(Condition, Operation);
		}
	}
}


UMVVMBlueprintViewCondition* SConditionRow::GetCondition() const
{
	return GetEntry()->GetCondition();
}

FSlateColor SConditionRow::GetErrorBorderColor() const
{
	if (const UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		if (Condition->HasCompilationMessage(UMVVMBlueprintViewCondition::EMessageType::Error))
		{
			return FStyleColors::Error;
		}
		else if (Condition->HasCompilationMessage(UMVVMBlueprintViewCondition::EMessageType::Warning))
		{
			return FStyleColors::Warning;
		}
	}
	return FStyleColors::Transparent;
}

EVisibility SConditionRow::GetErrorButtonVisibility() const
{
	if (const UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		bool bHasBindingError = Condition->HasCompilationMessage(UMVVMBlueprintViewCondition::EMessageType::Error);
		bool bHasBindingWarning = Condition->HasCompilationMessage(UMVVMBlueprintViewCondition::EMessageType::Warning);
		return bHasBindingError || bHasBindingWarning ? EVisibility::Visible : EVisibility::Hidden;
	}
	return EVisibility::Collapsed;
}

FText SConditionRow::GetErrorButtonToolTip() const
{
	if (const UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		TArray<FText> BindingErrorList = Condition->GetCompilationMessages(UMVVMBlueprintViewCondition::EMessageType::Error);
		TArray<FText> BindingWarningList = Condition->GetCompilationMessages(UMVVMBlueprintViewCondition::EMessageType::Warning);
		BindingErrorList.Append(BindingWarningList);

		static const FText NewLineText = FText::FromString(TEXT("\n"));
		FText HintText = LOCTEXT("ErrorButtonText", "Errors and Warnings: (Click to show in a separate window)");
		FText ErrorsText = FText::Join(NewLineText, BindingErrorList);
		return FText::Join(NewLineText, HintText, ErrorsText);
	}
	return FText();
}

FReply SConditionRow::OnErrorButtonClicked()
{
	ErrorItems.Reset();

	if (const UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		for (const FText& ErrorText : Condition->GetCompilationMessages(UMVVMBlueprintViewCondition::EMessageType::Error))
		{
			ErrorItems.Add(MakeShared<FText>(ErrorText));
		}

		for (const FText& WarningText : Condition->GetCompilationMessages(UMVVMBlueprintViewCondition::EMessageType::Warning))
		{
			ErrorItems.Add(MakeShared<FText>(WarningText));
		}

		const FText BindingDisplayName = Condition->GetDisplayName(true);
		TSharedRef<SCustomDialog> ErrorDialog = SNew(SCustomDialog)
			.Title(FText::Format(LOCTEXT("Compilation Errors and Warnings", "Compilation Errors and Warnings for {0}"), BindingDisplayName))
			.Buttons({
				SCustomDialog::FButton(LOCTEXT("OK", "OK"))
				})
			.Content()
			[
				SNew(SListView<TSharedPtr<FText>>)
					.ListItemsSource(&ErrorItems)
					.OnGenerateRow(this, &SConditionRow::OnGenerateErrorRow)
			];

		ErrorDialog->Show();
	}

	return FReply::Handled();
}

TSharedRef<ITableRow> SConditionRow::OnGenerateErrorRow(TSharedPtr<FText> Text, const TSharedRef<STableViewBase>& TableView) const
{
	return SNew(STableRow<TSharedPtr<FText>>, TableView)
	.Content()
	[
		SNew(SBorder)
		.BorderBackgroundColor(FStyleColors::Background)
		[
			SNew(STextBlock)
			.Text(*Text.Get())
		]
	];
}

ECheckBoxState SConditionRow::IsConditionEnabled() const
{
	if (const UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		return Condition->bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void SConditionRow::OnIsConditionEnableChanged(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Undetermined)
	{
		return;
	}

	if (UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		GetEditorSubsystem()->SetEnabledForCondition(Condition, NewState == ECheckBoxState::Checked);
	}
}

ECheckBoxState SConditionRow::IsConditionCompiled() const
{
	if (const UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		return Condition->bCompile ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void SConditionRow::OnIsConditionCompileChanged(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Undetermined)
	{
		return;
	}

	if (UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		GetEditorSubsystem()->SetCompileForCondition(Condition, NewState == ECheckBoxState::Checked);
	}
}

FMVVMLinkedPinValue SConditionRow::GetFieldSelectedValue(bool bCondition) const
{
	if (const UMVVMBlueprintViewCondition* Condition = GetCondition())
	{
		return bCondition ? FMVVMLinkedPinValue(Condition->GetConditionPath()) : FMVVMLinkedPinValue(Condition->GetDestinationPath());
	}
	return FMVVMLinkedPinValue();
}

void SConditionRow::HandleFieldSelectionChanged(FMVVMLinkedPinValue Value, SFieldSelectorMenu::ESelectionType SelectionType, bool bCondition)
{
	UWidgetBlueprint* WidgetBlueprint = GetBlueprint();
	UMVVMBlueprintViewCondition* Condition = GetCondition();
	if (WidgetBlueprint && Condition)
	{
		UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
		if (bCondition)
		{
			const bool bRequestBindingConversion = SelectionType == SFieldSelectorMenu::ESelectionType::Binding;
			Subsystem->SetConditionPath(Condition, Value.IsPropertyPath() ? Value.GetPropertyPath() : FMVVMBlueprintPropertyPath(), bRequestBindingConversion);
		}
		else
		{
			Subsystem->SetConditionDestinationPath(Condition, Value.IsPropertyPath() ? Value.GetPropertyPath() : FMVVMBlueprintPropertyPath());
		}
	}
}

FFieldSelectionContext SConditionRow::GetSelectedSelectionContext(bool bCondition) const
{
	FFieldSelectionContext Result;
	const UWidgetBlueprint* WidgetBlueprintPtr = GetBlueprint();
	UMVVMBlueprintViewCondition* Condition = GetCondition();
	if (WidgetBlueprintPtr == nullptr || Condition == nullptr)
	{
		return Result;
	}

	Result.BindingMode = EMVVMBindingMode::OneTimeToDestination;
	if (bCondition && !Condition->GetConditionPath().GetWidgetName().IsNone())
	{
		Result.FixedBindingSource = FBindingSource::CreateForWidget(WidgetBlueprintPtr, Condition->GetConditionPath().GetWidgetName());
	}

	const UMVVMDeveloperProjectSettings* MVVMSettings = GetDefault<UMVVMDeveloperProjectSettings>();
	Result.bAllowWidgets = MVVMSettings->bAllowWidgetInConditionSource || !bCondition;
	Result.bAllowViewModels = true;
	Result.bAllowConversionFunctions = false;
	Result.bReadable = bCondition;
	Result.bWritable = !bCondition;

	return Result;
}

FReply SConditionRow::HandleFieldSelectorDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bCondition)
{
	UMVVMBlueprintViewCondition* Condition = GetCondition();
	if (Condition == nullptr)
	{
		return FReply::Unhandled();
	}

	TOptional<FMVVMBlueprintPropertyPath> PropertyPath = BindingEntry::FRowHelper::DropFieldSelector(GetBlueprint(), DragDropEvent, bCondition);
	if (!PropertyPath.IsSet())
	{
		return FReply::Handled();
	}

	if (bCondition)
	{
		GetEditorSubsystem()->SetConditionPath(Condition, PropertyPath.GetValue(), true);
	}
	else
	{
		GetEditorSubsystem()->SetConditionDestinationPath(Condition, PropertyPath.GetValue());
	}
	return FReply::Handled();
}

FReply SConditionRow::HandleFieldSelectorOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bCondition)
{
	return BindingEntry::FRowHelper::DragOverFieldSelector(GetBlueprint(), DragDropEvent, bCondition);
}

} // namespace

#undef LOCTEXT_NAMESPACE
