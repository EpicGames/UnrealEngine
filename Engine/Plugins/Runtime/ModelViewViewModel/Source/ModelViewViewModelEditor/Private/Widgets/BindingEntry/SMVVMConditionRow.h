// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Types/MVVMBindingSource.h"
#include "Types/MVVMExecutionMode.h"
#include "Widgets/BindingEntry/SMVVMBaseRow.h"
#include "Widgets/SMVVMFieldSelectorMenu.h"

namespace UE::MVVM
{
class SBindingsList;
}

namespace UE::MVVM::BindingEntry
{

/**
 *
 */
class SConditionRow : public SBaseRow
{
public:
	SLATE_BEGIN_ARGS(SConditionRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, SBindingsList* BindingsList, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FWidgetBlueprintEditor>& InBlueprintEditor, UWidgetBlueprint* InBlueprint, const TSharedPtr<FBindingEntry>& InEntry);

protected:
	virtual TSharedRef<SWidget> BuildRowWidget() override;
	virtual const ANSICHAR* GetTableRowStyle() const override
	{
		return "BindingView.BindingRow";
	}

private:
	UMVVMBlueprintViewCondition* GetCondition() const;

	FSlateColor GetErrorBorderColor() const;

	EVisibility GetErrorButtonVisibility() const;

	FText GetErrorButtonToolTip() const;

	FReply OnErrorButtonClicked();

	TSharedRef<ITableRow> OnGenerateErrorRow(TSharedPtr<FText> Text, const TSharedRef<STableViewBase>& TableView) const;

	ECheckBoxState IsConditionEnabled() const;

	void OnIsConditionEnableChanged(ECheckBoxState NewState);

	ECheckBoxState IsConditionCompiled() const;

	void OnIsConditionCompileChanged(ECheckBoxState NewState);

	FMVVMLinkedPinValue GetFieldSelectedValue(bool bCondition) const;

	void HandleFieldSelectionChanged(FMVVMLinkedPinValue Value, SFieldSelectorMenu::ESelectionType SelectionType, bool bCondition);

	FFieldSelectionContext GetSelectedSelectionContext(bool bCondition) const;

	FReply HandleFieldSelectorDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bCondition);

	FReply HandleFieldSelectorOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bCondition);

	void OnConditionOperationSelectionChanged(FName ValueName, ESelectInfo::Type);
	FText GetCurrentConditionOperationLabel() const;
	TSharedRef<SWidget> GenerateConditionOperationWidget(FName ValueName) const;
	FText GetConditionOperationLabel(EMVVMConditionOperation Operation) const;
	EMVVMConditionOperation GetConditionOperationFromValueName(FName ValueName);

	TOptional<float> GetValue() const;
	TOptional<float> GetMaxValue() const;
	void OnValueChanged(float InValue);
	void OnMaxValueChanged(float InValue);
	EVisibility GetMaxValueVisibilityFromOperation() const;
	FText GetValueTooltipFromOperation() const;

private:
	TArray<TSharedPtr<FText>> ErrorItems;
};

} // namespace UE::MVVM
