// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/BindingEntry/SMVVMEventRow.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Framework/MVVMRowHelper.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMPropertyPath.h"

#include "Styling/AppStyle.h"
#include "Styling/MVVMEditorStyle.h"

#include "Dialog/SCustomDialog.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMVVMFieldSelector.h"
#include "Widgets/SMVVMSourceSelector.h"
#include "SSimpleButton.h"

#define LOCTEXT_NAMESPACE "BindingListView_EventRow"

namespace UE::MVVM::BindingEntry
{

void SEventRow::Construct(const FArguments& Args, SBindingsList* InBindingsList, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FWidgetBlueprintEditor>& InBlueprintEditor, UWidgetBlueprint* InBlueprint, const TSharedPtr<FBindingEntry>& InEntry)
{
	SBaseRow::Construct(SBaseRow::FArguments(), InBindingsList, OwnerTableView, InBlueprintEditor, InBlueprint, InEntry);

	TSharedPtr<SWidget> ChildContent = ChildSlot.DetachWidget();
	ChildSlot
	[
		// Add a single pixel top and bottom border for this widget.
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.Padding(0.0f, 2.0f, 0.0f, 1.0f)
		[
			// Restore the border that we're meant to have that reacts to selection/hover/etc.
			SNew(SBorder)
			.BorderImage(this, &SEventRow::GetBorderImage)
			.Padding(0.0f)
			[
				ChildContent.ToSharedRef()
			]
		]
	];
}

TSharedRef<SWidget> SEventRow::BuildRowWidget()
{
	return SNew(SBorder)
	.BorderImage(FAppStyle::Get().GetBrush("PlainBorder"))
	.Padding(0.0f)
	.BorderBackgroundColor(this, &SEventRow::GetErrorBorderColor)
	[
		SNew(SBox)
		.HeightOverride(30.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SEventRow::IsEventCompiled)
				.OnCheckStateChanged(this, &SEventRow::OnIsEventCompileChanged)
			]

			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(150.0f)
				[
					SNew(SFieldSelector, GetBlueprint())
					.OnGetLinkedValue(this, &SEventRow::GetFieldSelectedValue, true)
					.OnSelectionChanged(this, &SEventRow::HandleFieldSelectionChanged, true)
					.OnGetSelectionContext(this, &SEventRow::GetSelectedSelectionContext, true)
					.OnDrop(this, &SEventRow::HandleFieldSelectorDrop, true)
					.OnDragOver(this, &SEventRow::HandleFieldSelectorDragOver, true)
					.ShowContext(false)
					.CanCreateEvent(true)
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.WidthOverride(16.0f)
				.HeightOverride(16.0f)
				[
					SNew(SImage)
					.Image(FMVVMEditorStyle::Get().GetBrush("BindingMode.OneWay"))
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(150.0f)
				[
					SNew(SFieldSelector, GetBlueprint())
					.OnGetLinkedValue(this, &SEventRow::GetFieldSelectedValue, false)
					.OnSelectionChanged(this, &SEventRow::HandleFieldSelectionChanged, false)
					.OnGetSelectionContext(this, &SEventRow::GetSelectedSelectionContext, false)
					.OnDrop(this, &SEventRow::HandleFieldSelectorDrop, false)
					.OnDragOver(this, &SEventRow::HandleFieldSelectorDragOver, false)
					.IsBindingToEvent(true)
				]
			]

			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]

			+ SHorizontalBox::Slot()
			.Padding(0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SSimpleButton)
				.Icon(FAppStyle::Get().GetBrush("Icons.Error"))
				.Visibility(this, &SEventRow::GetErrorButtonVisibility)
				.ToolTipText(this, &SEventRow::GetErrorButtonToolTip)
				.OnClicked(this, &SEventRow::OnErrorButtonClicked)
			]

			+ SHorizontalBox::Slot()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				BuildContextMenuButton()
			]
		]
	];
}

UMVVMBlueprintViewEvent* SEventRow::GetEvent() const
{
	return GetEntry()->GetEvent();
}

FSlateColor SEventRow::GetErrorBorderColor() const
{
	if (const UMVVMBlueprintViewEvent* Event = GetEvent())
	{
		if (Event->HasCompilationMessage(UMVVMBlueprintViewEvent::EMessageType::Error))
		{
			return FStyleColors::Error;
		}
		else if (Event->HasCompilationMessage(UMVVMBlueprintViewEvent::EMessageType::Warning))
		{
			return FStyleColors::Warning;
		}
	}
	return FStyleColors::Transparent;
}

EVisibility SEventRow::GetErrorButtonVisibility() const
{
	if (const UMVVMBlueprintViewEvent* Event = GetEvent())
	{
		bool bHasBindingError = Event->HasCompilationMessage(UMVVMBlueprintViewEvent::EMessageType::Error);
		bool bHasBindingWarning = Event->HasCompilationMessage(UMVVMBlueprintViewEvent::EMessageType::Warning);
		return bHasBindingError || bHasBindingWarning ? EVisibility::Visible : EVisibility::Hidden;
	}
	return EVisibility::Collapsed;
}

FText SEventRow::GetErrorButtonToolTip() const
{
	if (const UMVVMBlueprintViewEvent* Event = GetEvent())
	{
		TArray<FText> BindingErrorList = Event->GetCompilationMessages(UMVVMBlueprintViewEvent::EMessageType::Error);
		TArray<FText> BindingWarningList = Event->GetCompilationMessages(UMVVMBlueprintViewEvent::EMessageType::Warning);
		BindingErrorList.Append(BindingWarningList);

		static const FText NewLineText = FText::FromString(TEXT("\n"));
		FText HintText = LOCTEXT("ErrorButtonText", "Errors and Warnings: (Click to show in a separate window)");
		FText ErrorsText = FText::Join(NewLineText, BindingErrorList);
		return FText::Join(NewLineText, HintText, ErrorsText);
	}
	return FText();
}

FReply SEventRow::OnErrorButtonClicked()
{
	ErrorItems.Reset();

	if (const UMVVMBlueprintViewEvent* Event = GetEvent())
	{
		for (const FText& ErrorText : Event->GetCompilationMessages(UMVVMBlueprintViewEvent::EMessageType::Error))
		{
			ErrorItems.Add(MakeShared<FText>(ErrorText));
		}

		for (const FText& WarningText : Event->GetCompilationMessages(UMVVMBlueprintViewEvent::EMessageType::Warning))
		{
			ErrorItems.Add(MakeShared<FText>(WarningText));
		}

		const FText BindingDisplayName = Event->GetDisplayName(true);
		TSharedRef<SCustomDialog> ErrorDialog = SNew(SCustomDialog)
			.Title(FText::Format(LOCTEXT("Compilation Errors and Warnings", "Compilation Errors and Warnings for {0}"), BindingDisplayName))
			.Buttons({
				SCustomDialog::FButton(LOCTEXT("OK", "OK"))
				})
			.Content()
			[
				SNew(SListView<TSharedPtr<FText>>)
					.ListItemsSource(&ErrorItems)
					.OnGenerateRow(this, &SEventRow::OnGenerateErrorRow)
			];

		ErrorDialog->Show();
	}

	return FReply::Handled();
}

TSharedRef<ITableRow> SEventRow::OnGenerateErrorRow(TSharedPtr<FText> Text, const TSharedRef<STableViewBase>& TableView) const
{
	return SNew(STableRow<TSharedPtr<FText>>, TableView)
	.Content()
	[
		SNew(SBorder)
		.BorderBackgroundColor(FStyleColors::Background)
		[
			SNew(STextBlock)
			.Text(*Text.Get())
		]
	];
}

ECheckBoxState SEventRow::IsEventEnabled() const
{
	if (const UMVVMBlueprintViewEvent* Event = GetEvent())
	{
		return Event->bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void SEventRow::OnIsEventEnableChanged(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Undetermined)
	{
		return;
	}

	if (UMVVMBlueprintViewEvent* Event = GetEvent())
	{
		GetEditorSubsystem()->SetEnabledForEvent(Event, NewState == ECheckBoxState::Checked);
	}
}

ECheckBoxState SEventRow::IsEventCompiled() const
{
	if (const UMVVMBlueprintViewEvent* Event = GetEvent())
	{
		return Event->bCompile ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void SEventRow::OnIsEventCompileChanged(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Undetermined)
	{
		return;
	}

	if (UMVVMBlueprintViewEvent* Event = GetEvent())
	{
		GetEditorSubsystem()->SetCompileForEvent(Event, NewState == ECheckBoxState::Checked);
	}
}

FMVVMLinkedPinValue SEventRow::GetFieldSelectedValue(bool bEvent) const
{
	if (const UMVVMBlueprintViewEvent* Event = GetEvent())
	{
		return bEvent ? FMVVMLinkedPinValue(Event->GetEventPath()) : FMVVMLinkedPinValue(Event->GetDestinationPath());
	}
	return FMVVMLinkedPinValue();
}

void SEventRow::HandleFieldSelectionChanged(FMVVMLinkedPinValue Value, SFieldSelectorMenu::ESelectionType SelectionType, bool bEvent)
{
	UWidgetBlueprint* WidgetBlueprint = GetBlueprint();
	UMVVMBlueprintViewEvent* Event = GetEvent();
	if (WidgetBlueprint && Event)
	{
		UMVVMEditorSubsystem* Subsystem = GetEditorSubsystem();
		if (bEvent)
		{
			const bool bRequestBindingConversion = SelectionType == SFieldSelectorMenu::ESelectionType::Binding;
			Subsystem->SetEventPath(Event, Value.IsPropertyPath() ? Value.GetPropertyPath() : FMVVMBlueprintPropertyPath(), bRequestBindingConversion);
		}
		else
		{
			Subsystem->SetEventDestinationPath(Event, Value.IsPropertyPath() ? Value.GetPropertyPath() : FMVVMBlueprintPropertyPath());
		}
	}
}

FFieldSelectionContext SEventRow::GetSelectedSelectionContext(bool bEvent) const
{
	FFieldSelectionContext Result;
	const UWidgetBlueprint* WidgetBlueprintPtr = GetBlueprint();
	UMVVMBlueprintViewEvent* Event = GetEvent();
	if (WidgetBlueprintPtr == nullptr || Event == nullptr)
	{
		return Result;
	}

	Result.BindingMode = EMVVMBindingMode::OneTimeToDestination;
	if (bEvent && !Event->GetEventPath().GetWidgetName().IsNone())
	{
		Result.FixedBindingSource = FBindingSource::CreateForWidget(WidgetBlueprintPtr, Event->GetEventPath().GetWidgetName());
	}

	Result.bAllowWidgets = true;
	Result.bAllowViewModels = !bEvent;
	Result.bAllowConversionFunctions = false;
	Result.bReadable = bEvent;
	Result.bWritable = !bEvent;

	return Result;
}

FReply SEventRow::HandleFieldSelectorDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bEvent)
{
	UMVVMBlueprintViewEvent* Event = GetEvent();
	if (Event == nullptr)
	{
		return FReply::Unhandled();
	}

	TOptional<FMVVMBlueprintPropertyPath> PropertyPath = BindingEntry::FRowHelper::DropFieldSelector(GetBlueprint(), DragDropEvent, bEvent);
	if (!PropertyPath.IsSet())
	{
		return FReply::Handled();
	}

	if (bEvent)
	{
		GetEditorSubsystem()->SetEventPath(Event, PropertyPath.GetValue(), true);
	}
	else
	{
		GetEditorSubsystem()->SetEventDestinationPath(Event, PropertyPath.GetValue());
	}
	return FReply::Handled();
}

FReply SEventRow::HandleFieldSelectorDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bEvent)
{
	return BindingEntry::FRowHelper::DragOverFieldSelector(GetBlueprint(), DragDropEvent, bEvent);
}

} // namespace

#undef LOCTEXT_NAMESPACE
