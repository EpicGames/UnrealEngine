// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Types/MVVMBindingSource.h"
#include "Types/MVVMExecutionMode.h"
#include "Widgets/BindingEntry/SMVVMBaseRow.h"
#include "Widgets/SMVVMFieldSelectorMenu.h"

namespace UE::MVVM
{
class SBindingsList;
}

namespace UE::MVVM::BindingEntry
{

/**
 *
 */
class SEventRow : public SBaseRow
{
public:
	SLATE_BEGIN_ARGS(SEventRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, SBindingsList* BindingsList, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FWidgetBlueprintEditor>& InBlueprintEditor, UWidgetBlueprint* InBlueprint, const TSharedPtr<FBindingEntry>& InEntry);

protected:
	virtual TSharedRef<SWidget> BuildRowWidget() override;
	virtual const ANSICHAR* GetTableRowStyle() const override
	{
		return "BindingView.BindingRow";
	}

private:
	UMVVMBlueprintViewEvent* GetEvent() const;

	FSlateColor GetErrorBorderColor() const;

	EVisibility GetErrorButtonVisibility() const;

	FText GetErrorButtonToolTip() const;

	FReply OnErrorButtonClicked();

	TSharedRef<ITableRow> OnGenerateErrorRow(TSharedPtr<FText> Text, const TSharedRef<STableViewBase>& TableView) const;

	ECheckBoxState IsEventEnabled() const;

	void OnIsEventEnableChanged(ECheckBoxState NewState);

	ECheckBoxState IsEventCompiled() const;

	void OnIsEventCompileChanged(ECheckBoxState NewState);

	FMVVMLinkedPinValue GetFieldSelectedValue(bool bEvent) const;

	void HandleFieldSelectionChanged(FMVVMLinkedPinValue Value, SFieldSelectorMenu::ESelectionType SelectionType, bool bEvent);

	FFieldSelectionContext GetSelectedSelectionContext(bool bEvent) const;

	FReply HandleFieldSelectorDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bEvent);

	FReply HandleFieldSelectorDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bEvent);

private:
	TArray<TSharedPtr<FText>> ErrorItems;
};

} // namespace UE::MVVM
