// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/BindingEntry/SMVVMFunctionParameterRow.h"

#include "Types/MVVMBindingMode.h"
#include "Types/MVVMBindingEntry.h"
#include "MVVMBlueprintViewBinding.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMBlueprintViewCondition.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Framework/MVVMConversionHelper.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMVVMEventParameter.h"
#include "Widgets/SMVVMFunctionParameter.h"
#include "Widgets/SMVVMConditionParameter.h"

#define LOCTEXT_NAMESPACE "BindingListView_FunctionParameterRow"

namespace UE::MVVM::BindingEntry
{

TSharedRef<SWidget> SFunctionParameterRow::BuildRowWidget()
{
	UMVVMEditorSubsystem* EditorSubsystem = GetEditorSubsystem();

	const FMVVMBlueprintPin* Pin = nullptr;
	const UEdGraphPin* GraphPin = nullptr;
	TSharedPtr<SWidget> ContentWidget;

	if (GetEntry()->GetRowType() == FBindingEntry::ERowType::BindingParameter)
	{
		bool bSimpleConversionFunction = false;
		UMVVMBlueprintView* View = EditorSubsystem->GetView(GetBlueprint());
		FMVVMBlueprintViewBinding* Binding = GetEntry()->GetBinding(View);
		check(Binding);

		const bool bSourceToDestination = UE::MVVM::IsForwardBinding(Binding->BindingType);
		UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding->Conversion.GetConversionFunction(bSourceToDestination);
		check(ConversionFunction);

		Pin = ConversionFunction->GetPins().FindByPredicate([ArgId = GetEntry()->GetBindingParameterId()](const FMVVMBlueprintPin& Other) { return Other.GetId() == ArgId; });
		GraphPin = ConversionFunction->GetOrCreateGraphPin(GetBlueprint(), GetEntry()->GetBindingParameterId());

		ContentWidget = SNew(SFunctionParameter, GetBlueprint())
			.BindingId(Binding->BindingId)
			.ParameterId(GetEntry()->GetBindingParameterId())
			.SourceToDestination(bSourceToDestination)
			.AllowDefault(!bSimpleConversionFunction);

		// Disable UI for autogenerated parameters
		FString MVVMBindToDestinationString = UE::MVVM::FConversionHelper::GetBindToDestinationStringFromConversionFunction(GetBlueprint(), ConversionFunction);
		if (!MVVMBindToDestinationString.IsEmpty())
		{
			if (Pin->GetId().GetNames().Contains(MVVMBindToDestinationString))
			{
				ContentWidget->SetEnabled(false);
				ContentWidget->SetToolTipText(LOCTEXT("MVVMBindToDestinationGeneratedParam", "Disabled since autogenerated param. As requested by 'MVVMBindToDestination' metadata in this UFunction."));
			}
		}

	}
	else if (GetEntry()->GetRowType() == FBindingEntry::ERowType::EventParameter)
	{
		UMVVMBlueprintViewEvent* ViewEvent = GetEntry()->GetEvent();
		check(ViewEvent);

		Pin = ViewEvent->GetPins().FindByPredicate([ArgId = GetEntry()->GetEventParameterId()](const FMVVMBlueprintPin& Other) { return Other.GetId() == ArgId; });
		GraphPin = ViewEvent->GetOrCreateGraphPin(GetEntry()->GetEventParameterId());

		ContentWidget = SNew(SEventParameter, GetBlueprint())
			.Event(GetEntry()->GetEvent())
			.ParameterId(GetEntry()->GetEventParameterId())
			.AllowDefault(true);
	}
	else if (GetEntry()->GetRowType() == FBindingEntry::ERowType::ConditionParameter)
	{
		UMVVMBlueprintViewCondition* ViewCondition = GetEntry()->GetCondition();
		check(ViewCondition);

		Pin = ViewCondition->GetPins().FindByPredicate([ArgId = GetEntry()->GetConditionParameterId()](const FMVVMBlueprintPin& Other) { return Other.GetId() == ArgId; });
		GraphPin = ViewCondition->GetOrCreateGraphPin(GetEntry()->GetConditionParameterId());

		ContentWidget = SNew(SConditionParameter, GetBlueprint())
			.Condition(GetEntry()->GetCondition())
			.ParameterId(GetEntry()->GetConditionParameterId())
			.AllowDefault(true);
	}

	FSlateColor PrimaryColor, SecondaryColor;
	const FSlateBrush* PrimaryBrush = nullptr;
	const FSlateBrush* SecondaryBrush = nullptr;
	FText DisplayName, ToolTip;
	bool bTextColorIsRed = false;
	if (Pin == nullptr || GraphPin == nullptr || Pin->GetStatus() == EMVVMBlueprintPinStatus::Orphaned)
	{
		PrimaryBrush = FBlueprintEditorUtils::GetIconFromPin(FEdGraphPinType());
		PrimaryColor = FLinearColor::Red;
		DisplayName = GraphPin ? GraphPin->GetDisplayName() : FText::GetEmpty();
		if (GraphPin == nullptr && Pin != nullptr)
		{
			if (!Pin->GetId().GetNames().IsEmpty())
			{
				DisplayName = FText::FromName(Pin->GetId().GetNames().Last());
			}
		}
		bTextColorIsRed = true;
	}
	else
	{
		PrimaryBrush = FBlueprintEditor::GetVarIconAndColorFromPinType(GraphPin->PinType, PrimaryColor, SecondaryBrush, SecondaryColor);
		DisplayName = GraphPin->GetDisplayName();
		ToolTip = FText::FromString(GraphPin->PinToolTip);
	}

	TSharedPtr<STextBlock> DisplayNameTextBlock;
	TSharedRef<SWidget> Result = SNew(SBox)
	.HeightOverride(30)
	.ToolTipText(ToolTip)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(0, 0, 8, 0)
		.AutoWidth()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 8, 0)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(PrimaryBrush)
				.ColorAndOpacity(PrimaryColor)
				.DesiredSizeOverride(FVector2D(16, 16))
			]

			+ SHorizontalBox::Slot()
			[
				SAssignNew(DisplayNameTextBlock, STextBlock)
				.Text(DisplayName)
			]
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ContentWidget.ToSharedRef()
		]
	];

	if (bTextColorIsRed)
	{
		DisplayNameTextBlock->SetColorAndOpacity(PrimaryColor);
	}

	return Result;
}

} // namespace

#undef LOCTEXT_NAMESPACE
