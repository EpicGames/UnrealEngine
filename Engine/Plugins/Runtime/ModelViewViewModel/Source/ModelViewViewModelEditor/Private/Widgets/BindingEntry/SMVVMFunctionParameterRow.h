// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Widgets/BindingEntry/SMVVMBaseRow.h"
#include "Widgets/SMVVMFieldSelectorMenu.h"


namespace UE::MVVM::BindingEntry
{

/**
 *
 */
class SFunctionParameterRow : public SBaseRow
{
protected:
	virtual TSharedRef<SWidget> BuildRowWidget() override;
	virtual const ANSICHAR* GetTableRowStyle() const override
	{
		return "BindingView.ParameterRow";
	}
};

} // namespace UE::MVVM
