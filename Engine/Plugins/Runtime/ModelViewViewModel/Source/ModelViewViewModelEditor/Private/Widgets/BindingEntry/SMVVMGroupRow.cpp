// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/BindingEntry/SMVVMGroupRow.h"

#include "MVVMBlueprintViewBinding.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMPropertyPath.h"

#include "Styling/AppStyle.h"

#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMVVMSourceSelector.h"
#include "SSimpleButton.h"

namespace UE::MVVM::BindingEntry
{

void SGroupRow::Construct(const FArguments& Args, SBindingsList* InBindingsList, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FWidgetBlueprintEditor>& InBlueprintEditor, UWidgetBlueprint* InBlueprint, const TSharedPtr<FBindingEntry>& InEntry)
{
	SBaseRow::Construct(SBaseRow::FArguments(), InBindingsList, OwnerTableView, InBlueprintEditor, InBlueprint, InEntry);

	TSharedPtr<SWidget> ChildContent = ChildSlot.DetachWidget();
	ChildSlot
	[
		// Add a single pixel top and bottom border for this widget.
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.Padding(0.0f, 1.0f)
		[
			// Restore the border that we're meant to have that reacts to selection/hover/etc.
			SNew(SBorder)
			.BorderImage(this, &SGroupRow::GetBorderImage)
			.Padding(0.0f)
			[
				ChildContent.ToSharedRef()
			]
		]
	];
}

TSharedRef<SWidget> SGroupRow::BuildRowWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(2.0f, 1.0f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Fill)
		.AutoWidth()
		[
			SNew(SBox)
			.MinDesiredWidth(150.0f)
			[
				SNew(SBindingContextSelector, GetBlueprint())
				.ShowClear(false)
				.AutoRefresh(true)
				.ViewModels(false)
				.SelectedBindingSource(this, &SGroupRow::GetSelectedWidget)
				.OnSelectionChanged(this, &SGroupRow::SetSelectedWidget)
			]
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SSimpleButton)
			.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
			.IsEnabled_Lambda([this]() { return !GetEntry()->GetGroupName().IsNone(); })
			.OnClicked(this, &SGroupRow::AddBinding)
		];
}

FBindingSource SGroupRow::GetSelectedWidget() const
{
	return GetEntry()->IsGroupWidget()
		? FBindingSource::CreateForWidget(GetBlueprint(), GetEntry()->GetGroupName())
		: FBindingSource::CreateForViewModel(GetBlueprint(), GetEntry()->GetGroupAsViewModel());
}

void SGroupRow::SetSelectedWidget(FBindingSource Source)
{
	UWidgetBlueprint* WidgetBlueprint = GetBlueprint();
	UMVVMBlueprintView* View = GetBlueprintView();
	if (WidgetBlueprint && View)
	{
		UMVVMEditorSubsystem* EditorSubsystem = GetEditorSubsystem();

		for (const TSharedPtr<FBindingEntry>& ChildEntry : GetEntry()->GetAllChildren())
		{
			if (ChildEntry->GetRowType() == FBindingEntry::ERowType::Binding)
			{
				if (FMVVMBlueprintViewBinding* Binding = ChildEntry->GetBinding(View))
				{
					FMVVMBlueprintPropertyPath CurrentPath = Binding->DestinationPath;
					Source.SetSourceTo(CurrentPath);

					EditorSubsystem->SetDestinationPathForBinding(WidgetBlueprint, *Binding, CurrentPath, false);
				}
			}

			if (ChildEntry->GetRowType() == FBindingEntry::ERowType::Event)
			{
				if (UMVVMBlueprintViewEvent* Event = ChildEntry->GetEvent())
				{
					FMVVMBlueprintPropertyPath CurrentPath = Event->GetEventPath();
					Source.SetSourceTo(CurrentPath);

					EditorSubsystem->SetEventPath(Event, CurrentPath, true);
				}
			}
		}
	}
}

FReply SGroupRow::AddBinding() const
{
	if (UWidgetBlueprint* WidgetBlueprint = GetBlueprint())
	{
		UMVVMEditorSubsystem* EditorSubsystem = GetEditorSubsystem();
		FMVVMBlueprintViewBinding& Binding = EditorSubsystem->AddBinding(WidgetBlueprint);
		FMVVMBlueprintPropertyPath Path;
		TSharedPtr<FBindingEntry> Entry = GetEntry();
		if (Entry->IsGroupWidget())
		{
			if (Entry->GetGroupName() == WidgetBlueprint->GetFName())
			{
				Path.SetSelfContext();
			}
			else
			{
				Path.SetWidgetName(Entry->GetGroupName());
			}
		}
		else
		{
			Path.SetViewModelId(Entry->GetGroupAsViewModel());
		}

		EditorSubsystem->SetDestinationPathForBinding(WidgetBlueprint, Binding, Path, false);
	}

	return FReply::Handled();
}

} // namespace
