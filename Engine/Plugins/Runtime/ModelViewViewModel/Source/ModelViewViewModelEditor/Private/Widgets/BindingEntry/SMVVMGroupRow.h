// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Types/MVVMBindingSource.h"
#include "Widgets/BindingEntry/SMVVMBaseRow.h"


namespace UE::MVVM
{
class SBindingsList;
}

namespace UE::MVVM::BindingEntry
{

/**
 *
 */
class SGroupRow : public SBaseRow
{
public:
	SLATE_BEGIN_ARGS(SGroupRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, SBindingsList* BindingsList, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FWidgetBlueprintEditor>& InBlueprintEditor, UWidgetBlueprint* InBlueprint, const TSharedPtr<FBindingEntry>& InEntry);

protected:
	virtual TSharedRef<SWidget> BuildRowWidget() override;
	virtual const ANSICHAR* GetTableRowStyle() const override
	{
		return "BindingView.WidgetRow";
	}


private:
	FBindingSource GetSelectedWidget() const;

	void SetSelectedWidget(FBindingSource Source);

	FReply AddBinding() const;
};

} // namespace UE::MVVM
