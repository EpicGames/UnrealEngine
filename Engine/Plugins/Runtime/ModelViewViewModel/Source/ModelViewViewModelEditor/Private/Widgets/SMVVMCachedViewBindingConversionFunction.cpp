// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMCachedViewBindingConversionFunction.h"

#include "Editor.h"
#include "MVVMBlueprintView.h"
#include "MVVMEditorSubsystem.h"
#include "Styling/MVVMEditorStyle.h"
#include "WidgetBlueprint.h"
#include "Widgets/SMVVMPropertyPath.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCachedViewBindingConversionFunction"

namespace UE::MVVM
{

void SCachedViewBindingConversionFunction::Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint)
{
	WidgetBlueprint = InWidgetBlueprint;
	OnGetConversionFunction = InArgs._OnGetConversionFunction;

	ChildSlot
	[
		SNew(SHorizontalBox)
		.ToolTipText_Lambda([this]() 
		{
			FConversionFunctionVariant ConversionFunction = OnGetConversionFunction.Execute();
			if (ConversionFunction.IsType<const UFunction*>())
			{
				const UFunction* FoundFunction = ConversionFunction.Get<const UFunction*>();
				return FoundFunction != nullptr ? FoundFunction->GetToolTipText() : FText::GetEmpty();
			}
			else if (ConversionFunction.IsType<TSubclassOf<UK2Node>>())
			{
				TSubclassOf<UK2Node> FoundFunction = ConversionFunction.Get<TSubclassOf<UK2Node>>();
				return FoundFunction.Get() != nullptr ? FoundFunction->GetToolTipText() : FText::GetEmpty();
			}
			return FText::GetEmpty();
		})
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		.AutoWidth()
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
			.Image(FAppStyle::Get().GetBrush("GraphEditor.Function_16x"))
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text_Lambda([this]() 
			{ 
				FConversionFunctionVariant ConversionFunction = OnGetConversionFunction.Execute();
				if (ConversionFunction.IsType<const UFunction*>())
				{
					const UFunction* FoundFunction = ConversionFunction.Get<const UFunction*>();
					return FoundFunction != nullptr ? FoundFunction->GetDisplayNameText() : FText::GetEmpty();
				}
				else if (ConversionFunction.IsType<TSubclassOf<UK2Node>>())
				{
					TSubclassOf<UK2Node> FoundFunction = ConversionFunction.Get<TSubclassOf<UK2Node>>();
					if (FoundFunction && FoundFunction->GetDefaultObject<UK2Node>())
					{
						return FoundFunction->GetDefaultObject<UK2Node>()->GetNodeTitle(ENodeTitleType::ListView);
					}
					return FoundFunction.Get() != nullptr ? FoundFunction->GetDisplayNameText() : FText::GetEmpty();
				}
				return FText::GetEmpty();
			})
		]
	];
}


} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
