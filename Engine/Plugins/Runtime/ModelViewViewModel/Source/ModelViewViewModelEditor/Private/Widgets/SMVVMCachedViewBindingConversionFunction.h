// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"
#include "Misc/TVariant.h"
#include "Templates/SubclassOf.h"

#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SCompoundWidget.h"

class UFunction;
class UWidgetBlueprint;

namespace UE::MVVM
{

class SCachedViewBindingConversionFunction : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;

public:
	using FConversionFunctionVariant = TVariant<const UFunction*, TSubclassOf<UK2Node>, FEmptyVariantState>;
	DECLARE_DELEGATE_RetVal(FConversionFunctionVariant, FOnGetConversionFunction);

	SLATE_BEGIN_ARGS(SCachedViewBindingConversionFunction)
		: _TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
	{}
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_EVENT(FOnGetConversionFunction, OnGetConversionFunction)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const UWidgetBlueprint* WidgetBlueprint);

private:
	TWeakObjectPtr<const UWidgetBlueprint> WidgetBlueprint;
	FOnGetConversionFunction OnGetConversionFunction;
};

} // namespace UE::MVVM
