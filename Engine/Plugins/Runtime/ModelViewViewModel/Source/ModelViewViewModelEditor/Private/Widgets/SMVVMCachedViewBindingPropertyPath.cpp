// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMCachedViewBindingPropertyPath.h"

#include "Editor.h"
#include "MVVMBlueprintView.h"
#include "MVVMEditorSubsystem.h"
#include "Styling/MVVMEditorStyle.h"
#include "WidgetBlueprint.h"
#include "Widgets/SMVVMPropertyPath.h"

#define LOCTEXT_NAMESPACE "SCachedViewBindingPropertyPath"

namespace UE::MVVM
{

void SCachedViewBindingPropertyPath::Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint)
{
	SetCanTick(true);

	WidgetBlueprint = InWidgetBlueprint;
	OnGetPropertyPath = InArgs._OnGetPropertyPath;
	CachedPropertyPath = OnGetPropertyPath.Execute();

	ChildSlot
	[
		SAssignNew(PropertyPathWidget, SPropertyPath, InWidgetBlueprint)
		.TextStyle(InArgs._TextStyle)
		.PropertyPath(CachedPropertyPath)
		.ShowContext(InArgs._ShowContext)
		.ShowOnlyLastPath(InArgs._ShowOnlyLastPath)
		.ShowFieldNotify(InArgs._ShowFieldNotify)
	];
}

void SCachedViewBindingPropertyPath::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	FMVVMBlueprintPropertyPath NewPropertyPath = OnGetPropertyPath.Execute();
	if (NewPropertyPath != CachedPropertyPath)
	{
		CachedPropertyPath = NewPropertyPath;
		PropertyPathWidget->SetPropertyPath(NewPropertyPath);
	}
}


} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
