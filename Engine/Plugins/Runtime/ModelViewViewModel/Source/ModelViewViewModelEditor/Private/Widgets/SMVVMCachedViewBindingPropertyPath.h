// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMPropertyPath.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SCompoundWidget.h"

class SHorizontalBox;
class UWidgetBlueprint;

namespace UE::MVVM
{
class SPropertyPath;



class SCachedViewBindingPropertyPath : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;

public:
	DECLARE_DELEGATE_RetVal(FMVVMBlueprintPropertyPath, FOnGetPropertyPath);

	SLATE_BEGIN_ARGS(SCachedViewBindingPropertyPath)
		: _TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
	{}
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_EVENT(FOnGetPropertyPath, OnGetPropertyPath)
		SLATE_ARGUMENT_DEFAULT(bool, ShowContext) = true;
		SLATE_ARGUMENT_DEFAULT(bool, ShowOnlyLastPath) = false;
		SLATE_ARGUMENT_DEFAULT(bool, ShowFieldNotify) = true;
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const UWidgetBlueprint* WidgetBlueprint);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	TWeakObjectPtr<const UWidgetBlueprint> WidgetBlueprint;
	TSharedPtr<SPropertyPath> PropertyPathWidget;
	FOnGetPropertyPath OnGetPropertyPath;
	FMVVMBlueprintPropertyPath CachedPropertyPath;
};

} // namespace UE::MVVM
