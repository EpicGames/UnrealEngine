// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMVVMEventParameter.h" 
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMEditorSubsystem.h"
#include "NodeFactory.h"
#include "SGraphPin.h"
#include "Styling/MVVMEditorStyle.h"
#include "WidgetBlueprint.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMVVMFieldSelector.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MVVMEventParameter"

namespace UE::MVVM
{

void SEventParameter::Construct(const FArguments& InArgs, UWidgetBlueprint* InWidgetBlueprint)
{
	WidgetBlueprint = InWidgetBlueprint;
	check(InWidgetBlueprint != nullptr);

	ViewEvent = InArgs._Event;
	check(InArgs._Event);

	ParameterId = InArgs._ParameterId;
	check(ParameterId.IsValid());

	bAllowDefault = InArgs._AllowDefault;

	bool bIsBooleanPin = false;
	TSharedRef<SWidget> ValueWidget = SNullWidget::NullWidget;
	if (UEdGraphPin* Pin = InArgs._Event->GetOrCreateGraphPin(ParameterId))
	{
		bIsBooleanPin = Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean;
		// create a new pin widget so that we can get the default value widget out of it
		if (TSharedPtr<SGraphPin> PinWidget = FNodeFactory::CreatePinWidget(Pin))
		{
			GraphPin = PinWidget;
			ValueWidget = PinWidget->GetDefaultValueWidget();
		}

		if (ValueWidget == SNullWidget::NullWidget)
		{
			ValueWidget = SNew(STextBlock)
				.Text(LOCTEXT("DefaultValue", "Default Value"))
				.TextStyle(FAppStyle::Get(), "HintText");
		}
		// booleans are represented by a checkbox which doesn't expand to the minsize we have, so don't put a border around them
		else if (!bIsBooleanPin)
		{
			ValueWidget = SNew(SBorder)
				.Padding(0.0f)
				.BorderImage(FMVVMEditorStyle::Get().GetBrush("FunctionParameter.Border"))
				[
					ValueWidget
				];
		}
	}

	bDefaultValueVisible = !OnGetSelectedField().IsValid();

	TSharedPtr<SHorizontalBox> HBox;

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(bIsBooleanPin ? FOptionalSize() : 100.0f)
		[
			SAssignNew(HBox, SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex(this, &SEventParameter::GetCurrentWidgetIndex)
				+ SWidgetSwitcher::Slot()
				[
					ValueWidget
				]
				+ SWidgetSwitcher::Slot()
				[
					SNew(SFieldSelector, WidgetBlueprint.Get())
					.OnGetLinkedValue(this, &SEventParameter::OnGetSelectedField)
					.OnSelectionChanged(this, &SEventParameter::HandleFieldSelectionChanged)
					.OnGetSelectionContext(this, &SEventParameter::GetSelectedSelectionContext)
				]
			]
		]
	];

	if (InArgs._AllowDefault)
	{
		HBox->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("BindArgument", "Bind this argument to a property."))
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.IsChecked(this, &SEventParameter::OnGetIsBindArgumentChecked)
				.OnCheckStateChanged(this, &SEventParameter::OnBindArgumentChecked)
				.Padding(FMargin(4))
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
					.Image_Lambda([this]()
						{
							ECheckBoxState CheckState = OnGetIsBindArgumentChecked();
							return (CheckState == ECheckBoxState::Checked) ? FAppStyle::GetBrush("Icons.Link") : FAppStyle::GetBrush("Icons.Unlink");
						})
				]
			];		
	}
}

int32 SEventParameter::GetCurrentWidgetIndex() const
{
	return bDefaultValueVisible && bAllowDefault ? 0 : 1;
}

ECheckBoxState SEventParameter::OnGetIsBindArgumentChecked() const 
{
	return bDefaultValueVisible ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}

void SEventParameter::OnBindArgumentChecked(ECheckBoxState Checked)
{
	bDefaultValueVisible = (Checked != ECheckBoxState::Checked);

	if (bDefaultValueVisible)
	{
		PreviousSelectedField = OnGetSelectedField();
		SetSelectedField(FMVVMLinkedPinValue());
	}
	else
	{
		SetSelectedField(PreviousSelectedField);
	}
}

FMVVMLinkedPinValue SEventParameter::OnGetSelectedField() const
{
	if (const UMVVMBlueprintViewEvent* EventPtr = ViewEvent.Get())
	{
		return FMVVMLinkedPinValue(EventPtr->GetPinPath(ParameterId));
	}
	return FMVVMLinkedPinValue();
}

void SEventParameter::SetSelectedField(const FMVVMLinkedPinValue& Path)
{
	if (UMVVMBlueprintViewEvent* EventPtr = ViewEvent.Get())
	{
		const UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		Subsystem->SetEventArgumentPath(EventPtr, ParameterId, Path.IsPropertyPath() ? Path.GetPropertyPath(): FMVVMBlueprintPropertyPath());
	}
}

void SEventParameter::HandleFieldSelectionChanged(FMVVMLinkedPinValue Value, SFieldSelectorMenu::ESelectionType SelectionType)
{
	SetSelectedField(Value);
}

FFieldSelectionContext SEventParameter::GetSelectedSelectionContext() const
{
	FFieldSelectionContext Result;

	const UMVVMBlueprintViewEvent* EventPtr = ViewEvent.Get();
	if (EventPtr == nullptr)
	{
		return Result;
	}
	
	Result.BindingMode = EMVVMBindingMode::OneTimeToDestination;
	//Result.AssignableTo = ConversionFunction->FindPropertyByName(ParameterId);
	Result.bAllowWidgets = true;
	Result.bAllowViewModels = true;
	Result.bAllowConversionFunctions = false;
	Result.bReadable = true;
	Result.bWritable = false;

	return Result;
}

}//namespace
#undef LOCTEXT_NAMESPACE
