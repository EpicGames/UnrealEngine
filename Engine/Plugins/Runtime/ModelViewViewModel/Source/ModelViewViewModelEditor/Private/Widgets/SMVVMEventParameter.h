// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMBlueprintPin.h"
#include "MVVMPropertyPath.h"
#include "Types/MVVMLinkedPinValue.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SMVVMFieldSelectorMenu.h"

class SGraphPin;
class UK2Node_CallFunction;
class UWidgetBlueprint;

class UMVVMBlueprintViewEvent;

enum class ECheckBoxState : uint8;

namespace UE::MVVM
{

class SFieldSelector;

class SEventParameter : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEventParameter) {}
		SLATE_ARGUMENT_DEFAULT(UMVVMBlueprintViewEvent*, Event) =  nullptr;
		SLATE_ARGUMENT(FMVVMBlueprintPinId, ParameterId)
		SLATE_ARGUMENT_DEFAULT(bool, AllowDefault) = true;
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UWidgetBlueprint* WidgetBlueprint);

private:
	ECheckBoxState OnGetIsBindArgumentChecked() const;
	void OnBindArgumentChecked(ECheckBoxState Checked);

	FMVVMLinkedPinValue OnGetSelectedField() const;
	void SetSelectedField(const FMVVMLinkedPinValue& Path);

	void HandleFieldSelectionChanged(FMVVMLinkedPinValue Value, SFieldSelectorMenu::ESelectionType SelectionType);
	FFieldSelectionContext GetSelectedSelectionContext() const;

	int32 GetCurrentWidgetIndex() const;

private:
	TWeakObjectPtr<UWidgetBlueprint> WidgetBlueprint;
	TWeakObjectPtr<UMVVMBlueprintViewEvent> ViewEvent;
	FMVVMBlueprintPinId ParameterId;
	/** This reference is just to keep the default value widget alive. */
	TSharedPtr<SGraphPin> GraphPin;

	FMVVMLinkedPinValue PreviousSelectedField;

	bool bAllowDefault = true;
	bool bDefaultValueVisible = true;
};

}
