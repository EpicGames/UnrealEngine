// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMFieldDisplay.h"

#include "MVVMPropertyPath.h"
#include "Styling/MVVMEditorStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "WidgetBlueprint.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SMVVMCachedViewBindingPropertyPath.h"
#include "Widgets/SMVVMCachedViewBindingConversionFunction.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MVVMFieldDisplay"

namespace UE::MVVM
{

void SFieldDisplay::Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint)
{
	check(InWidgetBlueprint);
	
	TextStyle = InArgs._TextStyle;
	OnGetLinkedValue = InArgs._OnGetLinkedValue;

	ChildSlot
	[
		SNew(SWidgetSwitcher)
		.WidgetIndex(this, &SFieldDisplay::GetCurrentDisplayIndex)
		//0-Property/Function (from Widget or Viewmodel).
		//0-Property/Function argument of conversion function
		+ SWidgetSwitcher::Slot()
		.Padding(0.0f, 0.0f, 8.0f, 0.0f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SCachedViewBindingPropertyPath, InWidgetBlueprint)
			.TextStyle(TextStyle)
			.ShowContext(InArgs._ShowContext)
			.OnGetPropertyPath(this, &SFieldDisplay::HandleGetPropertyPath)
			.ShowFieldNotify(InArgs._ShowFieldNotify)
		]

		//1-Conversion Function
		+ SWidgetSwitcher::Slot()
		.Padding(0.0f, 0.0f, 8.0f, 0.0f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SCachedViewBindingConversionFunction, InWidgetBlueprint)
			.TextStyle(TextStyle)
			.OnGetConversionFunction(this, &SFieldDisplay::HandleGetConversionFunction)
		]

		//2-Nothing selected.
		+ SWidgetSwitcher::Slot()
		[
			SNew(SBox)
			.Padding(FMargin(8.0f, 0.0f, 8.0f, 0.0f))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "HintText")
				.Text(LOCTEXT("None", "No field selected"))
			]
		]
	];
}

int32 SFieldDisplay::GetCurrentDisplayIndex() const
{
	if (OnGetLinkedValue.IsBound())
	{
		FMVVMLinkedPinValue LinkedValue = OnGetLinkedValue.Execute();
		if (LinkedValue.IsConversionFunction() || LinkedValue.IsConversionNode())
		{
			return 1;
		}
		else if (LinkedValue.IsPropertyPath())
		{
			return 0;
		}
	}
	return 2;
}

FMVVMBlueprintPropertyPath SFieldDisplay::HandleGetPropertyPath() const
{
	if (OnGetLinkedValue.IsBound())
	{
		FMVVMLinkedPinValue LinkedValue = OnGetLinkedValue.Execute();
		if (LinkedValue.IsPropertyPath())
		{
			return LinkedValue.GetPropertyPath();
		}
	}
	return FMVVMBlueprintPropertyPath();
}

TVariant<const UFunction*, TSubclassOf<UK2Node>, FEmptyVariantState> SFieldDisplay::HandleGetConversionFunction() const
{
	using FReturnType = TVariant<const UFunction*, TSubclassOf<UK2Node>, FEmptyVariantState>;
	if (OnGetLinkedValue.IsBound())
	{
		FMVVMLinkedPinValue LinkedValue = OnGetLinkedValue.Execute();
		if (ensure(LinkedValue.IsConversionFunction() || LinkedValue.IsConversionNode()))
		{
			if (LinkedValue.IsConversionFunction())
			{
				return FReturnType(TInPlaceType<const UFunction*>(), LinkedValue.GetConversionFunction());
			}
			else
			{
				check(LinkedValue.IsConversionNode());
				return FReturnType(TInPlaceType<TSubclassOf<UK2Node>>(), LinkedValue.GetConversionNode());
			}
		}
	}
	return FReturnType(TInPlaceType<FEmptyVariantState>());
}

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
