// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Types/MVVMLinkedPinValue.h"
#include "Widgets/SCompoundWidget.h"

struct FMVVMBlueprintPropertyPath;

namespace UE::MVVM
{

class SFieldDisplay : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal(FMVVMLinkedPinValue, FOnGetLinkedPinValue);

	SLATE_BEGIN_ARGS(SFieldDisplay)
		: _TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		{
		}
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_ARGUMENT_DEFAULT(bool, ShowContext) = true;
		SLATE_ARGUMENT_DEFAULT(bool, ShowFieldNotify) = true;
		SLATE_EVENT(FOnGetLinkedPinValue, OnGetLinkedValue)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint);

private:
	int32 GetCurrentDisplayIndex() const;
	FMVVMBlueprintPropertyPath HandleGetPropertyPath() const;
	TVariant<const UFunction*, TSubclassOf<UK2Node>, FEmptyVariantState> HandleGetConversionFunction() const;

public:
	FOnGetLinkedPinValue OnGetLinkedValue;

private:
	const FTextBlockStyle* TextStyle = nullptr;
}; 

} // namespace UE::MVVM
