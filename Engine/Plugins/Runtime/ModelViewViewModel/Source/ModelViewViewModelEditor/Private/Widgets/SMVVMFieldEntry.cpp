// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMFieldEntry.h"

#include "Bindings/MVVMBindingHelper.h"
#include "SMVVMFieldIcon.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MVVMFieldEntry"

namespace UE::MVVM
{

namespace Private
{

FText GetFieldDisplayName(const FMVVMConstFieldVariant& Field)
{
	if (!Field.IsEmpty())
	{
		if (Field.IsProperty())
		{
			return Field.GetProperty()->GetDisplayNameText();
		}
		if (Field.IsFunction())
		{
			return Field.GetFunction()->GetDisplayNameText();
		}
	}
	return LOCTEXT("None", "<None>");
}

FString GetFunctionSignature(const UFunction* Function)
{
	if (Function == nullptr)
	{
		return FString();
	}

	TStringBuilder<256> Signature;

	const FProperty* ReturnProperty = BindingHelper::GetReturnProperty(Function);
	if (ReturnProperty != nullptr)
	{
		Signature << ReturnProperty->GetCPPType();
	}
	else
	{
		Signature << TEXT("void");
	}

	Signature << TEXT(" ");
	Signature << Function->GetName();
	Signature << TEXT("(");

	TArray<const FProperty*> Arguments = BindingHelper::GetAllArgumentProperties(Function);
	Signature << FString::JoinBy(Arguments, TEXT(", "), 
		[](const FProperty* Property)
		{
			return Property->GetCPPType();
		}
	);

	Signature << TEXT(")");
	return Signature.ToString();
}

FText GetPathToolTip(TConstArrayView<FMVVMConstFieldVariant> Path)
{
	if (Path.IsEmpty())
	{
		return FText::GetEmpty();
	}

	FMVVMConstFieldVariant LastField = Path.Last();
	if (!LastField.IsEmpty())
	{
		FText LastToolTip, LastType;
		if (LastField.IsFunction() && LastField.GetFunction() != nullptr)
		{
			LastToolTip = LastField.GetFunction()->GetToolTipText();
			LastType = FText::FromString(GetFunctionSignature(LastField.GetFunction()));
		}
		else if (LastField.IsProperty() && LastField.GetProperty() != nullptr)
		{
			LastToolTip = LastField.GetProperty()->GetToolTipText();
			LastType = FText::Format(LOCTEXT("PropertyFormat", "Property: {0} {1}"),
				FText::FromString(LastField.GetProperty()->GetCPPType()),
				FText::FromString(LastField.GetProperty()->GetName())
			);
		}

		// only show the joined name if we've got more than 1 part to the path
		if (Path.Num() > 1)
		{
			TArray<FText> DisplayNames;
			for (const FMVVMConstFieldVariant& Field : Path)
			{
				DisplayNames.Add(GetFieldDisplayName(Field));
			}

			const FText JoinedDisplayName = FText::Join(FText::FromString(TEXT(".")), DisplayNames);

			return FText::Join(FText::FromString(TEXT("\n")),
				JoinedDisplayName,
				LastToolTip,
				LastType
			);
		}
		else
		{
			return FText::Join(FText::FromString(TEXT("\n")),
				LastToolTip,
				LastType
			);
		}
	}

	return FText::GetEmpty();
}

} // namespace Private

void SFieldPaths::Construct(const FArguments& InArgs)
{
	TextStyle = InArgs._TextStyle;

	HighlightField = InArgs._HighlightField;

	ChildSlot
	[
		SAssignNew(FieldBox, SHorizontalBox)
	];

	SetFieldPaths(InArgs._FieldPaths);
}

void SFieldPaths::SetFieldPaths(TArrayView<UE::MVVM::FMVVMConstFieldVariant> InPropertyPath)
{
	FieldBox->ClearChildren();

	int32 Index = 0;
	if (bShowOnlyLast && InPropertyPath.Num() > 0)
	{
		Index = InPropertyPath.Num() - 1;
	}

	for (; Index < InPropertyPath.Num(); ++Index)
	{
		bool bIsValid = InPropertyPath[Index].IsValid();
		bool bIsFieldValid = (bIsValid && InPropertyPath[Index].IsFunction()) ? !InPropertyPath[Index].GetFunction()->HasAnyFlags(RF_NewerVersionExists) : bIsValid;
		UStruct* Struct = InPropertyPath[Index].GetOwner();
		bool bIsOwnerValid = bIsValid && Struct && !Struct->HasAnyFlags(RF_NewerVersionExists);

		if (bIsFieldValid && bIsOwnerValid)
		{
			bool bShowFieldNotify = HighlightField.IsSet() && HighlightField.GetValue().Contains(Index);
			if (bShowFieldNotify)
			{
				FieldBox->AddSlot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
						.Image(FAppStyle::GetBrush("Kismet.VariableList.FieldNotify"))
					];
			}

			FieldBox->AddSlot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SFieldIcon)
					.Field(InPropertyPath[Index])
				];

			FieldBox->AddSlot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(4.0f, 0.f, 0.f, 0.f)
				[
					SNew(STextBlock)
					.TextStyle(TextStyle)
					.Clipping(EWidgetClipping::OnDemand)
					.SimpleTextMode(true)
					.Text(Private::GetFieldDisplayName(InPropertyPath[Index]))
				];
		}
		else
		{
			FieldBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(TextStyle)
					.Clipping(EWidgetClipping::OnDemand)
					.SimpleTextMode(true)
					.Text(LOCTEXT("Invalid", "Invalid Field"))
				];
		}

		// if not the last, then we need to add a chevron separator
		if (Index != InPropertyPath.Num() - 1)
		{
			FieldBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
				];
		}
	}

	SetToolTipText(Private::GetPathToolTip(InPropertyPath));
} 

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
