// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMPropertyPath.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateWidgetStyleAsset.h"

template <typename ValueType, typename ErrorType> class TValueOrError;

class SHorizontalBox;

namespace UE::MVVM
{

class SFieldIcon;

/** Show the TArray<FMVVMBlueprintFieldPath>. */
class SFieldPaths : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFieldPaths)
		: _TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
	{}
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_ARGUMENT(TArrayView<UE::MVVM::FMVVMConstFieldVariant>, FieldPaths)
		SLATE_ARGUMENT(bool, ShowOnlyLast)
		SLATE_ARGUMENT(TOptional<FInt32Range>, HighlightField)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void SetFieldPaths(TArrayView<UE::MVVM::FMVVMConstFieldVariant> InField);

private:
	const FTextBlockStyle* TextStyle = nullptr;
	TSharedPtr<SHorizontalBox> FieldBox;
	TOptional<FInt32Range> HighlightField;
	bool bShowOnlyLast = false;
};

} // namespace UE::MVVM
