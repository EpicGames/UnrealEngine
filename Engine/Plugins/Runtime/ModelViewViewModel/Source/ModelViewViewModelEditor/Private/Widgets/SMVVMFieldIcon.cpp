// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMFieldIcon.h"

#include "Bindings/MVVMBindingHelper.h"
#include "BlueprintEditor.h"
#include "Widgets/Images/SLayeredImage.h"

#define LOCTEXT_NAMESPACE "SFieldIcon"

namespace UE::MVVM
{

void SFieldIcon::Construct(const FArguments& Args)
{
	ChildSlot
	[
		SAssignNew(LayeredImage, SLayeredImage)
	];

	RefreshBinding(Args._Field);
}

void SFieldIcon::RefreshBinding(const FMVVMConstFieldVariant& Field)
{
	LayeredImage->SetImage(nullptr);
	LayeredImage->RemoveAllLayers();

	if (Field.IsEmpty())
	{
		return;
	}

	const FProperty* IconProperty = nullptr;

	if (Field.IsProperty())
	{
		IconProperty = Field.GetProperty();
	}
	else if (Field.IsFunction())
	{
		const UFunction* Function = Field.GetFunction();
		const FProperty* ReturnProperty = BindingHelper::GetReturnProperty(Function);
		if (ReturnProperty != nullptr)
		{
			IconProperty = ReturnProperty;
		}
		else
		{
			IconProperty = BindingHelper::GetFirstArgumentProperty(Function);
		}
	}

	if (IconProperty != nullptr)
	{
		FSlateColor PrimaryColor, SecondaryColor;
		const FSlateBrush* SecondaryBrush = nullptr;
		const FSlateBrush* PrimaryBrush = FBlueprintEditor::GetVarIconAndColorFromProperty(IconProperty, PrimaryColor, SecondaryBrush, SecondaryColor);

		LayeredImage->SetImage(PrimaryBrush);
		LayeredImage->SetColorAndOpacity(PrimaryColor);
		LayeredImage->AddLayer(SecondaryBrush, SecondaryColor);
	}
}

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
