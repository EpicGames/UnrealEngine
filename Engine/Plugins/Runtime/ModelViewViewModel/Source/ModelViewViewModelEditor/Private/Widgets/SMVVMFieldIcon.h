// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/MVVMFieldVariant.h"
#include "Widgets/SCompoundWidget.h"

class SLayeredImage;

namespace UE::MVVM
{

class SFieldIcon : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFieldIcon) {}
		SLATE_ARGUMENT(FMVVMConstFieldVariant, Field)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);
	void RefreshBinding(const FMVVMConstFieldVariant& Field);

private:
	TSharedPtr<SLayeredImage> LayeredImage;
};

} // namespace UE::MVVM
