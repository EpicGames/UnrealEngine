// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMFieldSelector.h"

#include "WidgetBlueprint.h"

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Styling/MVVMEditorStyle.h"

#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "MVVMFieldSelector"

namespace UE::MVVM
{

namespace Private
{

FBindingSource GetSourceFromPath(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& Path)
{
	return FBindingSource::CreateFromPropertyPath(WidgetBlueprint, Path);
}

} // namespace Private

void SFieldSelector::Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint)
{
	WidgetBlueprint = InWidgetBlueprint;
	check(InWidgetBlueprint);

	OnSelectionChanged = InArgs._OnSelectionChanged;
	OnGetSelectionContext = InArgs._OnGetSelectionContext;
	OnDropEvent = InArgs._OnDrop;
	OnDragOverEvent = InArgs._OnDragOver;
	bIsBindingToEvent = InArgs._IsBindingToEvent;
	bCanCreateEvent = InArgs._CanCreateEvent;

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(200.0f)
		[
			SAssignNew(ComboButton, SComboButton)
			.ComboButtonStyle(FMVVMEditorStyle::Get(), "FieldSelector.ComboButton")
			.OnGetMenuContent(this, &SFieldSelector::HandleGetMenuContent)
			.ContentPadding(FMargin(4.0f, 2.0f))
			.ButtonContent()
			[
				SAssignNew(FieldDisplay, SFieldDisplay, InWidgetBlueprint)
				.TextStyle(InArgs._TextStyle)
				.OnGetLinkedValue(InArgs._OnGetLinkedValue)
				.ShowFieldNotify(InArgs._ShowFieldNotify)
			]
		]
	];
}

TSharedRef<SWidget> SFieldSelector::HandleGetMenuContent()
{
	const UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get();
	if (!WidgetBlueprintPtr || !FieldDisplay)
	{
		return SNullWidget::NullWidget;
	}

	TOptional<FMVVMLinkedPinValue> CurrentSelected;
	if (FieldDisplay->OnGetLinkedValue.IsBound())
	{
		CurrentSelected = FieldDisplay->OnGetLinkedValue.Execute();
	}

	FFieldSelectionContext SelectionContext;
	if (OnGetSelectionContext.IsBound())
	{
		SelectionContext = OnGetSelectionContext.Execute();
	}

	TSharedRef<SFieldSelectorMenu> Menu = SNew(SFieldSelectorMenu, WidgetBlueprintPtr)
		.CurrentSelected(CurrentSelected)
		.OnSelected(this, &SFieldSelector::HandleFieldSelectionChanged)
		.OnMenuCloseRequested(this, &SFieldSelector::HandleMenuClosed)
		.SelectionContext(SelectionContext)
		.IsBindingToEvent(bIsBindingToEvent)
		.CanCreateEvent(bCanCreateEvent)
		;

	ComboButton->SetMenuContentWidgetToFocus(Menu->GetWidgetToFocus());

	return Menu;
}

void SFieldSelector::HandleFieldSelectionChanged(FMVVMLinkedPinValue LinkedValue, SFieldSelectorMenu::ESelectionType SelectionType)
{
	if (ComboButton.IsValid())
	{
		ComboButton->SetIsOpen(false);
	}

	if (OnSelectionChanged.IsBound())
	{
		OnSelectionChanged.Execute(LinkedValue, SelectionType);
	}
}

void SFieldSelector::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>())
	{
		DecoratedDragDropOp->ResetToDefaultToolTip();
	}
}

FReply SFieldSelector::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (OnDragOverEvent.IsBound())
	{
		return OnDragOverEvent.Execute(MyGeometry, DragDropEvent);
	}

	return FReply::Unhandled();
}

FReply SFieldSelector::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (OnDropEvent.IsBound())
	{
		return OnDropEvent.Execute(MyGeometry, DragDropEvent);
	}
	return FReply::Unhandled();
}

void SFieldSelector::HandleMenuClosed()
{
	if (ComboButton.IsValid())
	{
		ComboButton->SetIsOpen(false);
	}
}

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
