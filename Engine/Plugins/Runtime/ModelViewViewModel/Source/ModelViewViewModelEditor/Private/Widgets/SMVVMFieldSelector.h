// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SMVVMFieldDisplay.h"
#include "Widgets/SMVVMFieldSelectorMenu.h"

namespace UE::MVVM { class SCachedViewBindingPropertyPath; }

class SComboButton;

namespace UE::MVVM
{

class SFieldSelector : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal(FFieldSelectionContext, FOnGetSelectionContext);
	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnDrop, const FGeometry&, const FDragDropEvent&);
	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnDragOver, const FGeometry&, const FDragDropEvent&);

	SLATE_BEGIN_ARGS(SFieldSelector)
		: _TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		{
		}
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_ARGUMENT_DEFAULT(bool, ShowContext) = true;
		SLATE_ARGUMENT_DEFAULT(bool, IsBindingToEvent) = false;
		SLATE_ARGUMENT_DEFAULT(bool, ShowFieldNotify) = true;
		SLATE_EVENT(SFieldDisplay::FOnGetLinkedPinValue, OnGetLinkedValue)
		SLATE_EVENT(SFieldSelectorMenu::FOnLinkedValueSelected, OnSelectionChanged)
		SLATE_EVENT(FOnGetSelectionContext, OnGetSelectionContext)
		SLATE_EVENT(FOnDrop, OnDrop)
		SLATE_EVENT(FOnDragOver, OnDragOver)
		SLATE_ARGUMENT_DEFAULT(bool, CanCreateEvent) = false;
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint);
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

private:
	TSharedRef<SWidget> CreateSourcePanel();
	TSharedRef<SWidget> HandleGetMenuContent();

	void HandleFieldSelectionChanged(FMVVMLinkedPinValue NewValue, SFieldSelectorMenu::ESelectionType SelectionType);
	void HandleMenuClosed();

private:
	TSharedPtr<SCachedViewBindingPropertyPath> PropertyPathWidget;
	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<SFieldDisplay> FieldDisplay;
	TWeakObjectPtr<const UWidgetBlueprint> WidgetBlueprint;
	SFieldSelectorMenu::FOnLinkedValueSelected OnSelectionChanged;
	FOnGetSelectionContext OnGetSelectionContext;
	FOnDrop OnDropEvent;
	FOnDragOver OnDragOverEvent;
	bool bIsBindingToEvent = false;
	bool bCanCreateEvent = false;
}; 

} // namespace UE::MVVM
