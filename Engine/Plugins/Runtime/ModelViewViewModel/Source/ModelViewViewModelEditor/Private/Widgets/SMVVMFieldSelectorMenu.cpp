// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMFieldSelectorMenu.h"

#include "Bindings/MVVMBindingHelper.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Hierarchy/SReadOnlyHierarchyView.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMEditorSubsystem.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "SPrimaryButton.h"
#include "Styling/MVVMEditorStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Types/MVVMBindingMode.h"
#include "Widgets/SMVVMFieldEntry.h"
#include "Widgets/SMVVMFieldIcon.h"
#include "Widgets/SMVVMPropertyPath.h"
#include "Widgets/SMVVMSourceEntry.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SMVVMCachedViewBindingPropertyPath.h"
#include "Widgets/SMVVMCachedViewBindingConversionFunction.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "MVVMFieldSelectorMenu"

namespace UE::MVVM
{

namespace Private
{
EFieldVisibility GetFieldVisibilityFlags(EMVVMBindingMode Mode, bool bReadable, bool bWritable)
{
	EFieldVisibility Flags = EFieldVisibility::None;

	if (bReadable)
	{
		Flags |= EFieldVisibility::Readable;
	}
	if (bWritable)
	{
		Flags |= EFieldVisibility::Writable;
	}
	if (!IsOneTimeBinding(Mode))
	{
		Flags |= EFieldVisibility::Notify;
	}

	return Flags;
}
} // namespace

void SFieldSelectorMenu::Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint)
{
	SetCanTick(true);

	WidgetBlueprint = InWidgetBlueprint;
	check(InWidgetBlueprint);

	OnSelected = InArgs._OnSelected;
	OnSelectedConversion = InArgs._OnSelectedConversion;
	OnMenuCloseRequested = InArgs._OnMenuCloseRequested;
	SelectionContext = InArgs._SelectionContext;
	bCanCreateEvent = InArgs._CanCreateEvent;
	bCanConvertField = InArgs._CanConvertField;

	bIsMenuInitialized = false;
	const bool bIsClearEnabled = InArgs._CurrentSelected.IsSet() && InArgs._CurrentSelected.GetValue().IsValid() && InArgs._IsClearEnableable;

	const FProperty* AssignableToProperty = nullptr;
	const bool bShowConversionFunctions = SelectionContext.bAllowConversionFunctions;

	AssignableToProperty = SelectionContext.AssignableTo;

	if (bShowConversionFunctions || bCanConvertField)
	{
		GenerateConversionFunctionItems();
	}

	TSharedRef<SVerticalBox> VBox = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.Padding(0, 4, 0, 4)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(3.f, 0.f, 3.f, 0.f)
			[
				SAssignNew(SearchBox, SSearchBox)
				.OnTextChanged(this, &SFieldSelectorMenu::HandleSearchBoxTextChanged)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SFieldSelectorMenu::HandleEnabledContextToggleChanged)
				.IsChecked(this, &SFieldSelectorMenu::ToggleEnabledContext)
				.Visibility(InArgs._IsEnableContextToggleAvailable ? EVisibility::Visible : EVisibility::Collapsed)
				.ToolTipText(LOCTEXT("ContextFlagToolTip", "Should the list be filtered to only properties and functions that make sense in the current context?"))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MVVMContextSensitiveToggle", "Context Sensitive"))
				]
			]
		];

	if (InArgs._ShowTooltipText)
	{
		VBox->AddSlot()
			.AutoHeight()
			.Padding(3.0, 5.0, 3.0, 5.0)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(this, &SFieldSelectorMenu::GetTooltipText)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SCachedViewBindingPropertyPath, WidgetBlueprint.Get())
					.ShowContext(false)
					.ShowFieldNotify(false)
					.ShowOnlyLastPath(true)
					.OnGetPropertyPath(this, &SFieldSelectorMenu::GetTooltipPropertyPath)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(this, &SFieldSelectorMenu::GetAdditionalTooltipText)
				]
			];
	}

	VBox->AddSlot()
		.FillHeight(1.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			[
				SNew(SSplitter)
				.PhysicalSplitterHandleSize(4.0f)
				+ SSplitter::Slot()
				.Value(0.5f)
				.MinSize(100.0f)
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						CreateBindingContextPanel(InArgs)
					]
				]
				+ SSplitter::Slot()
				.Value(0.5f)
				.MinSize(100.0f)
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						CreateBindingListPanel(InArgs, AssignableToProperty)
					]
				]
			]
		];

	VBox->AddSlot()
		.Padding(4.0f, 4.0f, 4.0f, 0.0f)
		.HAlign(HAlign_Right)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(EventSelectButton, SPrimaryButton)
						.OnClicked(this, &SFieldSelectorMenu::HandleEventSelectClicked)
						.Text(LOCTEXT("EventSelect", "Event"))
						.ToolTipText(LOCTEXT("SelectorEventTooltip", "Create an event."))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SPrimaryButton)
						.OnClicked(this, &SFieldSelectorMenu::HandleSelectClicked)
						.IsEnabled(this, &SFieldSelectorMenu::IsSelectEnabled)
						.Text(LOCTEXT("Select", "Select"))
						.ToolTipText(LOCTEXT("SelectorBindingTooltip", "Create a binding with the field."))
				]
			+ SHorizontalBox::Slot()
				.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
				.AutoWidth()
				[
					SNew(SButton)
						.OnClicked(this, &SFieldSelectorMenu::HandleClearClicked)
						.IsEnabled(bIsClearEnabled)
						.HAlign(HAlign_Center)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("Clear", "Clear"))
						]
				]
			+ SHorizontalBox::Slot()
				.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
				.AutoWidth()
				[
					SNew(SButton)
						.OnClicked(this, &SFieldSelectorMenu::HandleCancelClicked)
						.HAlign(HAlign_Center)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("Cancel", "Cancel"))
						]
				]
		];

	TSharedRef<SWidget> MenuWidget = SNew(SBox)
		.MinDesiredWidth(400.0f)
		.MinDesiredHeight(200.0f)
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(8.0f, 2.0f, 8.0f, 3.0f))
				[
					VBox
				]
		];

	ChildSlot
		[
			MenuWidget
		];

	bIsMenuInitialized = true;
	UpdateSelection();
}

void SFieldSelectorMenu::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	TOptional<FMVVMLinkedPinValue> NewSelection = GetCurrentSelection();
	if (NewSelection != CurrentSelectedValue)
	{
		SetSelectedValue(NewSelection);
		UpdateSelection();
	}
}

void SFieldSelectorMenu::UpdateSelection()
{
	check(EventSelectButton);
	EventSelectButton->SetVisibility(IsEventSelectEnabled() ? EVisibility::Visible : EVisibility::Collapsed);

	if(!bIsSelectingConversion && ConversionFunctionCategoryTree.IsValid())
	{
		// If users enabled auto-conversion, do not show the conversion function selection tree. Users will select conversion function when auto-converting
		ConversionFunctionCategoryTree->SetVisibility(BindingList->IsConversionEnabled() ? EVisibility::Collapsed : EVisibility::Visible);
	}

	if (bIsSelectingConversion)
	{
		if (BindingList.IsValid())
		{
			BindingList->SetEnabled(false);
			BindingList->ClearSources();
		}

		if (ViewModelList.IsValid())
		{
			ViewModelList->SetEnabled(false);
			ViewModelList->ClearSelection();
		}

		if (WidgetList.IsValid())
		{
			WidgetList->SetEnabled(false);
			WidgetList->ClearSelection();
		}
	}

	Invalidate(EInvalidateWidgetReason::Layout);
}

void SFieldSelectorMenu::SetSelectedValue(const TOptional<FMVVMLinkedPinValue>& NewSelection)
{
	if (bIsSelectingConversion)
	{
		if (NewSelection.IsSet())
		{
			// Find the available conversion
			for (const FMVVMFieldConversion& AvailableConversion : AvailableFieldConversions)
			{
				if (NewSelection->IsConversionFunction() && AvailableConversion.Function.IsFunction()
					&& AvailableConversion.Function.GetFunction()->GetFName() == NewSelection->GetConversionFunction()->GetFName())
				{
					CurrentSelectedConversionValue = AvailableConversion;
					break;
				}
				if (NewSelection->IsConversionNode() && AvailableConversion.Function.IsNode()
					&& AvailableConversion.Function.GetNode()->GetFName() == NewSelection->GetConversionNode()->GetFName())
				{
					CurrentSelectedConversionValue = AvailableConversion;
					break;
				}
			}
		}
	}
	else
	{
		CurrentSelectedValue = NewSelection;
	}
}


FText SFieldSelectorMenu::GetTooltipText() const
{
	if (CurrentSelectedValue.IsSet())
	{
		return LOCTEXT("FieldSelector_TooltipBindTo", "Bind to ");
	}
	return LOCTEXT("FieldSelector_Tooltip", "Select a property to bind to:");
}

FText SFieldSelectorMenu::GetAdditionalTooltipText() const
{
	if (bIsSelectingConversion)
	{
		return LOCTEXT("FieldSelector_TooltipConversion", " with conversion function:");
	}
	return FText::GetEmpty();
}

FMVVMBlueprintPropertyPath SFieldSelectorMenu::GetTooltipPropertyPath() const
{
	return CurrentSelectedValue.Get(FMVVMLinkedPinValue()).GetPropertyPath();
}

TSharedRef<SWidget> SFieldSelectorMenu::GetWidgetToFocus() const
{
	return SearchBox.ToSharedRef();
}

TOptional<FMVVMLinkedPinValue> SFieldSelectorMenu::GetCurrentSelection() const
{
	TOptional<FMVVMLinkedPinValue> Value;
	if (ConversionFunctionList.IsValid())
	{
		TArray<FConversionFunctionValue> Selection;
		if (ConversionFunctionList.IsValid())
		{
			Selection = ConversionFunctionList->GetSelectedItems();
		}
		if (Selection.Num() > 0)
		{
			Value = FMVVMLinkedPinValue(Selection[0]);
		}
	}

	if (BindingList.IsValid())
	{
		FMVVMBlueprintPropertyPath SelectedProperty = BindingList->GetSelectedProperty();
		if (SelectedProperty.IsValid())
		{
			ensure(Value.IsSet() == false);
			Value = FMVVMLinkedPinValue(MoveTemp(SelectedProperty));
		}
	}
	return Value;
}

bool SFieldSelectorMenu::IsSelectEnabled() const
{
	if (bIsSelectingConversion)
	{
		return CurrentSelectedConversionValue.IsSet();
	}

	return CurrentSelectedValue.IsSet();
}

bool SFieldSelectorMenu::IsEventSelectEnabled() const
{
	if (bCanCreateEvent && CurrentSelectedValue.IsSet() && CurrentSelectedValue.GetValue().IsPropertyPath())
	{
		return UMVVMBlueprintViewEvent::Supports(WidgetBlueprint.Get(), CurrentSelectedValue.GetValue().GetPropertyPath());
	}
	return false;
}

FReply SFieldSelectorMenu::HandleSelectClicked()
{
	if (bCanConvertField)
	{
		return PromptConversionSelection();
	}

	OnSelected.ExecuteIfBound(CurrentSelectedValue.Get(FMVVMLinkedPinValue()), ESelectionType::Binding);

	return FReply::Handled();
}

FReply SFieldSelectorMenu::HandleEventSelectClicked()
{
	OnSelected.ExecuteIfBound(CurrentSelectedValue.Get(FMVVMLinkedPinValue()), ESelectionType::Event);

	return FReply::Handled();
}

FReply SFieldSelectorMenu::HandleClearClicked()
{
	FMVVMBlueprintPropertyPath NewProperty;
	
	if (SelectionContext.FixedBindingSource.IsSet())
	{
		FBindingSource Source = SelectionContext.FixedBindingSource.GetValue();
		Source.SetSourceTo(NewProperty);
	}

	OnSelected.ExecuteIfBound(FMVVMLinkedPinValue(MoveTemp(NewProperty)), ESelectionType::None);

	return FReply::Handled();
}

FReply SFieldSelectorMenu::HandleCancelClicked()
{
	OnMenuCloseRequested.ExecuteIfBound();

	return FReply::Handled();
}


FReply SFieldSelectorMenu::PromptConversionSelection()
{
	if (!bIsSelectingConversion)
	{
		check(BindingList);
		AvailableFieldConversions = BindingList->GetSelectedConversions();

		if (AvailableFieldConversions.Num() == 0)
		{
			OnSelectedConversion.ExecuteIfBound(CurrentSelectedValue.Get(FMVVMLinkedPinValue()), FMVVMFieldConversion(), ESelectionType::Binding);
		}
		else if (AvailableFieldConversions.Num() == 1)
		{
			CurrentSelectedConversionValue = AvailableFieldConversions[0];
			OnSelectedConversion.ExecuteIfBound(CurrentSelectedValue.Get(FMVVMLinkedPinValue()), CurrentSelectedConversionValue.Get(FMVVMFieldConversion()), ESelectionType::Binding);
		}
		else if (AvailableFieldConversions.Num() > 1)
		{
			// If multiple are available, display conversion function list and wait for users to select.
			bIsSelectingConversion = true;

			UpdateSelection();
			Algo::SortBy(ConversionFunctions, &FConversionFunctionValue::GetDisplayName, FText::FSortPredicate());
			FilterConversionFunctions();

			if (ConversionFunctionList.IsValid())
			{
				ConversionFunctionList->SetVisibility(EVisibility::Visible);
				ConversionFunctionList->RequestListRefresh();
			}
		}

		return FReply::Handled();
	}

	// If we are selecting conversion, the select button is disabled until a valid conversion is selected
	bIsSelectingConversion = false;
	AvailableFieldConversions.Empty();
	OnSelectedConversion.ExecuteIfBound(CurrentSelectedValue.Get(FMVVMLinkedPinValue()), CurrentSelectedConversionValue.Get(FMVVMFieldConversion()), ESelectionType::Binding);

	return FReply::Handled();
}

void SFieldSelectorMenu::SetPropertyPathSelection(const FMVVMBlueprintPropertyPath& SelectedPath)
{
	SetSelectedValue(FMVVMLinkedPinValue(SelectedPath));
	if (IsEventSelectEnabled())
	{
		HandleEventSelectClicked();
	}
	else
	{
		HandleSelectClicked();
	}
}

void SFieldSelectorMenu::SetConversionFunctionSelection(const FConversionFunctionValue SelectedFunction)
{
	SetSelectedValue(FMVVMLinkedPinValue(SelectedFunction));
	if (IsEventSelectEnabled())
	{
		HandleEventSelectClicked();
	}
	else
	{
		HandleSelectClicked();
	}
}

TSharedPtr<SFieldSelectorMenu::FConversionFunctionItem> SFieldSelectorMenu::FindConversionFunctionCategory(const TArray<TSharedPtr<FConversionFunctionItem>>& Items, TArrayView<FString> CategoryNameParts) const
{
	if (CategoryNameParts.Num() > 0)
	{
		for (const TSharedPtr<FConversionFunctionItem>& Item : Items)
		{
			if (Item->GetCategoryName() == CategoryNameParts[0])
			{
				TArrayView<FString> RemainingParts = CategoryNameParts.RightChop(1);

				// last category part, this is what we're looking for
				if (RemainingParts.Num() == 0)
				{
					return Item;
				}

				// recurse into children
				return FindConversionFunctionCategory(Item->Children, RemainingParts);
			}
		}
	}

	return TSharedPtr<FConversionFunctionItem>();
}

void SFieldSelectorMenu::HandleSearchBoxTextChanged(const FText& NewText)
{
	if (BindingList.IsValid())
	{
		BindingList->SetRawFilterText(NewText);
	}

	if (ConversionFunctionCategoryTree.IsValid())
	{
		TArray<TSharedPtr<FConversionFunctionItem>> OldSelectedCategories;
		ConversionFunctionCategoryTree->GetSelectedItems(OldSelectedCategories);

		FilterConversionFunctionCategories();
		ConversionFunctionCategoryTree->RequestTreeRefresh();

		TArray<TSharedPtr<FConversionFunctionItem>> NewSelectedCategories;

		// reselect old selection
		for (const TSharedPtr<FConversionFunctionItem>& OldItem : OldSelectedCategories)
		{
			if (TSharedPtr<FConversionFunctionItem> NewItem = FindConversionFunctionCategory(FilteredConversionFunctionRoot, OldItem->CategoryPath))
			{
				NewSelectedCategories.Add(NewItem);
			}
		}
		
		if (NewSelectedCategories.Num() > 0)
		{
			ConversionFunctionCategoryTree->SetItemSelection(NewSelectedCategories, true);
		}
	}

	if (ConversionFunctionList.IsValid())
	{
		TArray<FConversionFunctionValue> OldSelectedFunctions;
		ConversionFunctionList->GetSelectedItems(OldSelectedFunctions);

		FilterConversionFunctions();
		ConversionFunctionList->RequestListRefresh();

		ConversionFunctionList->SetItemSelection(OldSelectedFunctions, true);
	}
}

void SFieldSelectorMenu::HandleViewModelSelected(FBindingSource Source, ESelectInfo::Type)
{
	if (!bIsMenuInitialized)
	{
		return;
	}

	if (BindingList.IsValid())
	{
		BindingList->ClearSources();
	}

	TArray<FBindingSource> Selection = ViewModelList->GetSelectedItems();
	if (Selection.Num() == 0)
	{
		return;
	}

	if (WidgetList.IsValid())
	{
		WidgetList->ClearSelection();
	}

	if (ConversionFunctionCategoryTree.IsValid())
	{
		ConversionFunctionCategoryTree->ClearSelection();
	}

	if (BindingList.IsValid())
	{
		BindingList->AddSources(Selection);
	}
}

void SFieldSelectorMenu::HandleWidgetSelected(FName WidgetName, ESelectInfo::Type)
{
	if (!bIsMenuInitialized)
	{
		return;
	}

	if (BindingList.IsValid())
	{
		BindingList->ClearSources();
	}

	TArray<FName> Selection = WidgetList->GetSelectedWidgets();
	if (Selection.Num() == 0)
	{
		return;
	}

	if (ViewModelList.IsValid())
	{
		ViewModelList->ClearSelection();
	}

	if (ConversionFunctionCategoryTree.IsValid())
	{
		ConversionFunctionCategoryTree->ClearSelection();
	}

	if (BindingList.IsValid())
	{
		const UWidgetBlueprint* WidgetBP = WidgetBlueprint.Get();

		TArray<FBindingSource> Sources;
		Algo::Transform(Selection, Sources, [WidgetBP](const FName& WidgetName)
			{
				return FBindingSource::CreateForWidget(WidgetBP, WidgetName);
			});

		BindingList->AddSources(Sources);
	}
}

TSharedRef<ITableRow> SFieldSelectorMenu::HandleGenerateViewModelRow(FBindingSource ViewModel, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(STableRow<FBindingSource>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(4)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FSlateIconFinder::FindIconBrushForClass(ViewModel.GetClass()))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(ViewModel.GetDisplayName())
				.ColorAndOpacity(FSlateColor::UseForeground())
				.HighlightText_Lambda([this]() { return SearchBox.IsValid() ? SearchBox->GetText() : FText::GetEmpty(); })
			]
		];
}

int32 SFieldSelectorMenu::FilterConversionFunctionCategoryChildren(const TArray<FString>& FilterStrings, const TArray<TSharedPtr<FConversionFunctionItem>>& SourceArray, TArray<TSharedPtr<FConversionFunctionItem>>& OutDestArray)
{
	int32 NumFunctions = 0;

	for (const TSharedPtr<FConversionFunctionItem>& SourceItem : SourceArray)
	{
		// check if our name matches the filters
		bool bMatchesFilters = false;
		if (SourceItem->Function != nullptr)
		{
			bMatchesFilters = true;
			for (const FString& Filter : FilterStrings)
			{
				bool bFoundMatches = false;
				for (const FString& Keyword : SourceItem->SearchKeywords)
				{
					if (Keyword.Contains(Filter))
					{
						bFoundMatches = true;
						break;
					}
				}

				if (!bFoundMatches)
				{
					bMatchesFilters = false;
					break;
				}
			}
		}

		int32 NumChildren = 0;
		TArray<TSharedPtr<FConversionFunctionItem>> FilteredChildren;
		if (bMatchesFilters)
		{
			ensureAlways(SourceItem->Function != nullptr);
			NumChildren = 1;
		}
		else
		{
			// if we don't match, then we still want to check all our children
			NumChildren = FilterConversionFunctionCategoryChildren(FilterStrings, SourceItem->Children, FilteredChildren);
		}

		// then add this item to the destination array
		if (NumChildren > 0)
		{
			TSharedPtr<FConversionFunctionItem>& NewItem = OutDestArray.Add_GetRef(MakeShared<FConversionFunctionItem>());
			NewItem->CategoryPath = SourceItem->CategoryPath;
			NewItem->Function = SourceItem->Function; 
			NewItem->Children = FilteredChildren;
			NewItem->NumFunctions = NumChildren;

			NumFunctions += NewItem->NumFunctions;
		}
	}

	return NumFunctions;
}

void SFieldSelectorMenu::FilterConversionFunctionCategories()
{
	FilteredConversionFunctionRoot.Reset();

	TArray<FString> FilterStrings;
	if (SearchBox.IsValid())
	{
		SearchBox->GetText().ToString().ParseIntoArrayWS(FilterStrings);
	}

	if (FilterStrings.IsEmpty())
	{
		// don't bother filtering if we don't have any search terms
		FilteredConversionFunctionRoot = ConversionFunctionRoot;
		return;
	}

	TSharedPtr<FConversionFunctionItem> RootItem = FilteredConversionFunctionRoot.Add_GetRef(MakeShared<FConversionFunctionItem>());
	RootItem->CategoryPath = { TEXT("Conversion Functions") };
	
	RootItem->NumFunctions = FilterConversionFunctionCategoryChildren(FilterStrings, ConversionFunctionRoot[0]->Children, FilteredConversionFunctionRoot[0]->Children);
	
	ExpandFunctionCategoryTree(FilteredConversionFunctionRoot, true);
}

int32 SFieldSelectorMenu::SortConversionFunctionItemsRecursive(TArray<TSharedPtr<FConversionFunctionItem>>& Items)
{
	int32 NumFound = 0;

	Items.Sort([](const TSharedPtr<FConversionFunctionItem>& A, const TSharedPtr<FConversionFunctionItem>& B)
		{
			if (!A->GetCategoryName().IsEmpty() && !B->GetCategoryName().IsEmpty())
			{
				return A->GetCategoryName() < B->GetCategoryName();
			}
			if (!A->GetCategoryName().IsEmpty() && B->GetCategoryName().IsEmpty())
			{
				return true;
			}
			if (A->GetCategoryName().IsEmpty() && !B->GetCategoryName().IsEmpty())
			{
				return false;
			}
			if (A->Function.IsValid() && B->Function.IsValid())
			{
				return A->Function.GetDisplayName().CompareTo(B->Function.GetDisplayName()) <= 0;
			}
			return true;
		});

	for (const TSharedPtr<FConversionFunctionItem>& Item : Items)
	{
		NumFound += SortConversionFunctionItemsRecursive(Item->Children);

		if (Item->Function != nullptr)
		{
			NumFound += 1;
		}
	}

	return NumFound;
}

void SFieldSelectorMenu::GenerateConversionFunctionItems()
{
	const FProperty* AssignToProperty = SelectionContext.AssignableTo;
	UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	TArray<FConversionFunctionValue> AllConversionFunctions = Subsystem->GetConversionFunctions(WidgetBlueprint.Get(), nullptr, AssignToProperty);

	auto AddFunctionToItem = [](FConversionFunctionValue Function, const TSharedPtr<FConversionFunctionItem>& Parent)
	{
		TSharedPtr<FConversionFunctionItem>& Item = Parent->Children.Add_GetRef(MakeShared<FConversionFunctionItem>());
		Item->Function = Function;
		Item->SearchKeywords = Function.GetSearchKeywords();
		Item->NumFunctions = 1;
		Parent->NumFunctions += 1;
	};

	TSharedPtr<FConversionFunctionItem> CurrentSelectedItem;

	TArray<FString> CategoryPath;

	ConversionFunctionRoot.Reset();
	TSharedPtr<FConversionFunctionItem>& RootItem = ConversionFunctionRoot.Add_GetRef(MakeShared<FConversionFunctionItem>());
	RootItem->CategoryPath = { TEXT("Conversion Functions") };

	for (const FConversionFunctionValue& Function : AllConversionFunctions)
	{
		const FText& CategoryName = Function.GetCategory();
		if (CategoryName.IsEmpty())
		{
			AddFunctionToItem(Function, RootItem);
			continue;
		}

		// split into subcategories and trim
		CategoryPath.Reset();
		CategoryName.ToString().ParseIntoArray(CategoryPath, TEXT("|"));
		for (FString& SubCategory : CategoryPath)
		{
			SubCategory.TrimStartAndEndInline();
		}

		TSharedPtr<FConversionFunctionItem> ParentItem = RootItem;

		// create items for the entire category path
		// eg. "Math|Boolean|AND" 
		// Math 
		//   > Boolean
		//     > AND
		for (int32 PathIndex = 0; PathIndex < CategoryPath.Num(); ++PathIndex)
		{
			ParentItem->NumFunctions += 1;

			ParentItem = FindOrCreateItemForCategory(ParentItem->Children, MakeArrayView(CategoryPath.GetData(), PathIndex + 1));
		}

		AddFunctionToItem(Function, ParentItem);
	}

	int32 NumItems = SortConversionFunctionItemsRecursive(ConversionFunctionRoot);
	ensure(NumItems == RootItem->NumFunctions);
}

TSharedPtr<SFieldSelectorMenu::FConversionFunctionItem> SFieldSelectorMenu::ExpandFunctionCategoryTreeToItem(const FConversionFunctionValue Function)
{
	TArray<TSharedPtr<FConversionFunctionItem>> Path;

	FText FullCategoryName = Function.GetCategory();
	if (FullCategoryName.IsEmpty())
	{
		Path.Add(FilteredConversionFunctionRoot[0]);
	}
	else
	{
		TArray<FString> CategoryPath;
		FullCategoryName.ToString().ParseIntoArray(CategoryPath, TEXT("|"));

		TSharedPtr<FConversionFunctionItem> CurrentParent = FilteredConversionFunctionRoot[0];

		for (const FString& SubCategory : CategoryPath)
		{
			const FString Trimmed = SubCategory.TrimStartAndEnd();

			TSharedPtr<FConversionFunctionItem>* FoundItem =
				CurrentParent->Children.FindByPredicate([Trimmed, Function](const TSharedPtr<FConversionFunctionItem>& Item)
					{
						return Item->GetCategoryName() == Trimmed || Item->Function == Function;
					});

			if (FoundItem != nullptr)
			{
				Path.Add(*FoundItem);
				CurrentParent = *FoundItem;
			}
		}
	}

	if (Path.Num() > 0)
	{
		ConversionFunctionCategoryTree->SetItemExpansion(FilteredConversionFunctionRoot[0], true);
		ExpandFunctionCategoryTree(Path, false);
		return Path.Last();
	}
	return TSharedPtr<FConversionFunctionItem>();
}

void SFieldSelectorMenu::ExpandFunctionCategoryTree(const TArray<TSharedPtr<FConversionFunctionItem>>& Items, bool bRecursive)
{
	for (const TSharedPtr<FConversionFunctionItem>& Item : Items)
	{
		ConversionFunctionCategoryTree->SetItemExpansion(Item, true);

		if (bRecursive)
		{
			ExpandFunctionCategoryTree(Item->Children, bRecursive);
		}
	}
}

void SFieldSelectorMenu::FilterViewModels(const FText& NewText)
{
	FilteredViewModelSources.Reset();

	TArray<FString> FilterStrings;
	NewText.ToString().ParseIntoArrayWS(FilterStrings);

	if (FilterStrings.IsEmpty())
	{
		FilteredViewModelSources = ViewModelSources;
		return;
	}

	for (const FBindingSource& BindingSource : ViewModelSources)
	{
		const FString DisplayName = BindingSource.GetDisplayName().ToString();
		const FString ClassName = BindingSource.GetClass() != nullptr ? BindingSource.GetClass()->GetName() : FString();

		bool bMatchesFilters = true;
		
		for (const FString& Filter : FilterStrings)
		{
			if (!DisplayName.Contains(Filter) && 
				!ClassName.Contains(Filter))
			{
				bMatchesFilters = false;
				break;
			}
		}

		if (bMatchesFilters)
		{
			FilteredViewModelSources.Add(BindingSource);
		}
	}
}

TSharedRef<SWidget> SFieldSelectorMenu::CreateBindingContextPanel(const FArguments& InArgs)
{
	// show source picker
	TSharedRef<SVerticalBox> StackedSourcePicker = SNew(SVerticalBox);

	const bool bUseFixedSource = SelectionContext.FixedBindingSource.IsSet() && SelectionContext.FixedBindingSource.GetValue().IsValid();
	const EMVVMBlueprintFieldPathSource FixedFieldPathSource = bUseFixedSource ? SelectionContext.FixedBindingSource.GetValue().GetSource() : EMVVMBlueprintFieldPathSource::None;
	EMVVMBlueprintFieldPathSource CurrentPropertyFieldPathSource = EMVVMBlueprintFieldPathSource::None;

	FMVVMLinkedPinValue CurrentFieldSelectedLinkedValue = InArgs._CurrentSelected.IsSet() ? InArgs._CurrentSelected.GetValue() : FMVVMLinkedPinValue();
	if (InArgs._CurrentSelected.IsSet() && CurrentFieldSelectedLinkedValue.IsPropertyPath())
	{
		CurrentPropertyFieldPathSource = CurrentFieldSelectedLinkedValue.GetPropertyPath().GetSource(WidgetBlueprint.Get());
	}

	if (SelectionContext.bAllowViewModels)
	{
		if (FixedFieldPathSource == EMVVMBlueprintFieldPathSource::ViewModel)
		{
			ViewModelSources.Add(SelectionContext.FixedBindingSource.GetValue());
		}
		else
		{
			ViewModelSources = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->GetAllViewModels(WidgetBlueprint.Get());
		}

		FilteredViewModelSources = ViewModelSources;

		ViewModelList = SNew(SListView<FBindingSource>)
			.ListItemsSource(&FilteredViewModelSources)
			.OnGenerateRow(this, &SFieldSelectorMenu::HandleGenerateViewModelRow)
			.SelectionMode(ESelectionMode::Multi)
			.OnSelectionChanged(this, &SFieldSelectorMenu::HandleViewModelSelected);

		FBindingSource SelectedSource;
		if (CurrentPropertyFieldPathSource == EMVVMBlueprintFieldPathSource::ViewModel)
		{
			for (const FBindingSource& Source : FilteredViewModelSources)
			{
				check(InArgs._CurrentSelected.IsSet());
				check(CurrentFieldSelectedLinkedValue.IsPropertyPath());
				if (Source.GetViewModelId() == CurrentFieldSelectedLinkedValue.GetPropertyPath().GetViewModelId())
				{
					SelectedSource = Source;
				}
			}
		}

		if (SelectedSource.IsValid())
		{
			ViewModelList->SetItemSelection(SelectedSource, true);
		}

		StackedSourcePicker->AddSlot()
			.AutoHeight()
			[
				ViewModelList.ToSharedRef()
			];
	}
	
	if (SelectionContext.bAllowWidgets)
	{
		TArray<FName> ShowOnly;
		if (FixedFieldPathSource == EMVVMBlueprintFieldPathSource::Widget)
		{
			ShowOnly.Add(SelectionContext.FixedBindingSource.GetValue().GetWidgetName());
		}
		else if (FixedFieldPathSource == EMVVMBlueprintFieldPathSource::SelfContext)
		{
			ShowOnly.Add(WidgetBlueprint.Get()->GetFName());
		}

		WidgetList = SNew(SReadOnlyHierarchyView, WidgetBlueprint.Get())
			.OnSelectionChanged(this, &SFieldSelectorMenu::HandleWidgetSelected)
			.SelectionMode(ESelectionMode::Multi)
			.ShowSearch(false)
			.ShowOnly(ShowOnly)
			.ExpandAll(false);

		if (CurrentPropertyFieldPathSource == EMVVMBlueprintFieldPathSource::Widget)
		{
			check(InArgs._CurrentSelected.IsSet());
			check(CurrentFieldSelectedLinkedValue.IsPropertyPath());
			WidgetList->SetSelectedWidget(CurrentFieldSelectedLinkedValue.GetPropertyPath().GetWidgetName());
		}
		else if (CurrentPropertyFieldPathSource == EMVVMBlueprintFieldPathSource::SelfContext)
		{
			WidgetList->SetSelectedWidget(WidgetBlueprint.Get()->GetFName());
		}

		StackedSourcePicker->AddSlot()
			.AutoHeight()
			[
				WidgetList.ToSharedRef()
			];
	}

	if (SelectionContext.bAllowConversionFunctions)
	{
		FilterConversionFunctionCategories();
		
		SAssignNew(ConversionFunctionCategoryTree, STreeView<TSharedPtr<FConversionFunctionItem>>)
			.SelectionMode(ESelectionMode::Multi)
			.TreeItemsSource(&FilteredConversionFunctionRoot)
			.OnGenerateRow(this, &SFieldSelectorMenu::HandleGenerateConversionFunctionCategoryRow)
			.OnSelectionChanged(this, &SFieldSelectorMenu::HandleConversionFunctionCategorySelected)
			.OnGetChildren(this, &SFieldSelectorMenu::HandleGetConversionFunctionCategoryChildren);

		if (CurrentFieldSelectedLinkedValue.IsConversionFunction() || CurrentFieldSelectedLinkedValue.IsConversionNode())
		{
			const FConversionFunctionValue ConversionFunction = CurrentFieldSelectedLinkedValue.IsConversionFunction()
				? FConversionFunctionValue(CurrentFieldSelectedLinkedValue.GetConversionFunction())
				: FConversionFunctionValue(CurrentFieldSelectedLinkedValue.GetConversionNode());
			if (ConversionFunction.IsValid())
			{
				TSharedPtr<FConversionFunctionItem> FunctionItem = ExpandFunctionCategoryTreeToItem(ConversionFunction);
				if (FunctionItem)
				{
					TGuardValue<bool> TmpGuard(bIsMenuInitialized, true);
					ConversionFunctionCategoryTree->SetItemSelection(FunctionItem, true);
				}
			}
		}

		// bCanConvertField handles visibility of this
		if (!SelectionContext.bAllowConversionFunctions)
		{
			ConversionFunctionCategoryTree->SetVisibility(EVisibility::Collapsed);
		}

		StackedSourcePicker->AddSlot()
			.AutoHeight()
			[
				ConversionFunctionCategoryTree.ToSharedRef()
			];
	}

	return StackedSourcePicker;
}

TSharedRef<SWidget> SFieldSelectorMenu::CreateBindingListPanel(const FArguments& InArgs, const FProperty* AssignableToProperty)
{
	BindingList = SNew(SSourceBindingList, WidgetBlueprint.Get())
		.ShowSearchBox(false)
		.OnDoubleClicked(this, &SFieldSelectorMenu::SetPropertyPathSelection)
		.FieldVisibilityFlags(Private::GetFieldVisibilityFlags(SelectionContext.BindingMode, SelectionContext.bReadable, SelectionContext.bWritable))
		.AssignableTo(AssignableToProperty)
		.IsBindingToEvent(InArgs._IsBindingToEvent)
		.bEnableConversion(bCanConvertField);

	const bool bValidSelectedProperty = InArgs._CurrentSelected.IsSet() && InArgs._CurrentSelected.GetValue().IsPropertyPath();

	if (SelectionContext.FixedBindingSource.IsSet())
	{
		// Single fixed source, don't show the separate source panel.
		BindingList->AddSource(SelectionContext.FixedBindingSource.GetValue());
	}
	else if (bValidSelectedProperty)
	{
		FBindingSource Source = FBindingSource::CreateFromPropertyPath(WidgetBlueprint.Get(), InArgs._CurrentSelected.GetValue().GetPropertyPath());
		BindingList->AddSource(Source);
	}

	if (bValidSelectedProperty)
	{
		BindingList->SetSelectedProperty(InArgs._CurrentSelected.GetValue().GetPropertyPath());
	}


	TSharedRef<SVerticalBox> BindingListVBox = SNew(SVerticalBox);
	BindingListVBox->AddSlot()
		[
			BindingList.ToSharedRef()
		];

	if (SelectionContext.bAllowConversionFunctions || bCanConvertField)
	{
		const bool bValidSelectedFunction = InArgs._CurrentSelected.IsSet() && (InArgs._CurrentSelected.GetValue().IsConversionFunction() || InArgs._CurrentSelected.GetValue().IsConversionNode());

		BindingListVBox->AddSlot()
			.AutoHeight()
			[
				SAssignNew(ConversionFunctionList, SListView<FConversionFunctionValue>)
				.SelectionMode(ESelectionMode::Single)
				.ListItemsSource(&FilteredConversionFunctions)
				.OnMouseButtonDoubleClick(this, &SFieldSelectorMenu::SetConversionFunctionSelection)
				.OnGenerateRow(this, &SFieldSelectorMenu::HandleGenerateConversionFunctionRow)
			];

		if (bValidSelectedFunction)
		{
			FConversionFunctionValue ConversionFunctionValue = InArgs._CurrentSelected.GetValue().IsConversionFunction()
				? FConversionFunctionValue(InArgs._CurrentSelected.GetValue().GetConversionFunction())
				: FConversionFunctionValue(InArgs._CurrentSelected.GetValue().GetConversionNode());
			ConversionFunctionList->SetItemSelection(ConversionFunctionValue, true);
		}

		if (bCanConvertField)
		{
			ConversionFunctionList->SetVisibility(EVisibility::Collapsed);
		}
	}

	return BindingListVBox;
}

void SFieldSelectorMenu::HandleEnabledContextToggleChanged(ECheckBoxState CheckState)
{
	UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint.Get());
	check(ExtensionView);

	FMVVMViewBindingFilterSettings FilterSettings = ExtensionView->GetFilterSettings();
	FilterSettings.FilterFlags = CheckState == ECheckBoxState::Checked ? EFilterFlag::All : EFilterFlag::None;
	ExtensionView->SetFilterSettings(FilterSettings);

	BindingList->EnableConversion(bCanConvertField && FilterSettings.FilterFlags == EFilterFlag::All);

	if (WidgetList.IsValid() && WidgetList->GetSelectedWidgets().Num() > 0)
	{
		HandleWidgetSelected(WidgetBlueprint->GetFName(), ESelectInfo::Direct);
	}
	else if (ViewModelList.IsValid() && ViewModelList->GetNumItemsSelected() > 0)
	{
		HandleViewModelSelected(FBindingSource(), ESelectInfo::Direct);
	}
}

ECheckBoxState SFieldSelectorMenu::ToggleEnabledContext() const
{
	UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint.Get());
	check(ExtensionView);

	return ExtensionView->GetFilterSettings().FilterFlags == EFilterFlag::All ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

TSharedPtr<SFieldSelectorMenu::FConversionFunctionItem> SFieldSelectorMenu::FindOrCreateItemForCategory(TArray<TSharedPtr<FConversionFunctionItem>>& Items, TArrayView<FString> CategoryPath)
{
	check(CategoryPath.Num() > 0);

	const FString& CategoryName = CategoryPath.Last();

	int32 Idx = 0;
	for (; Idx < Items.Num(); ++Idx)
	{
		// found item
		if (Items[Idx]->GetCategoryName() == CategoryName)
		{
			return Items[Idx];
		}

		// passed the place where it should have been, break out
		if (Items[Idx]->GetCategoryName() > CategoryName)
		{
			break;
		}
	}

	TSharedPtr<FConversionFunctionItem> NewItem = Items.Insert_GetRef(MakeShared<FConversionFunctionItem>(), Idx);
	NewItem->CategoryPath = CategoryPath;
	return NewItem;
}

TSharedRef<ITableRow> SFieldSelectorMenu::HandleGenerateConversionFunctionCategoryRow(TSharedPtr<FConversionFunctionItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	FText DisplayName = FText::FormatOrdered(FText::FromString("{0} ({1})"), FText::FromString(Item->GetCategoryName()), FText::FromString(LexToString(Item->NumFunctions)));

	return SNew(STableRow<TSharedPtr<FConversionFunctionItem>>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0, 2.0f, 4.0f, 2.0f)
			.AutoWidth()
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
				.Image(FAppStyle::Get().GetBrush("GraphEditor.Function_16x"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(Item == FilteredConversionFunctionRoot[0] ? FAppStyle::Get().GetFontStyle("NormalText") : FAppStyle::Get().GetFontStyle("BoldFont"))
				.Text(DisplayName)
				.ToolTipText(FText::FromString(Item->GetCategoryName()))
				.HighlightText_Lambda([this]() { return SearchBox.IsValid() ? SearchBox->GetText() : FText::GetEmpty(); })
			]
		];
}

void SFieldSelectorMenu::HandleGetConversionFunctionCategoryChildren(TSharedPtr<FConversionFunctionItem> Item, TArray<TSharedPtr<FConversionFunctionItem>>& OutItems) const
{
	Algo::TransformIf(Item->Children, OutItems,
		[](const TSharedPtr<FConversionFunctionItem>& Item)
		{
			return !Item->GetCategoryName().IsEmpty();
		},
		[](const TSharedPtr<FConversionFunctionItem>& Item)
		{
			return Item;
		});
}

TSharedRef<ITableRow> SFieldSelectorMenu::HandleGenerateConversionFunctionRow(const FConversionFunctionValue Function, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<FConversionFunctionValue>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0, 0, 4.0f, 0)
			.AutoWidth()
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
				.Image(FAppStyle::Get().GetBrush("GraphEditor.Function_16x"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Function.GetDisplayName())
				.ToolTipText(Function.GetTooltip())
				.HighlightText_Lambda([this]() { return SearchBox.IsValid() ? SearchBox->GetText() : FText::GetEmpty(); })
			]
		];
}

void SFieldSelectorMenu::AddConversionFunctionChildrenRecursive(const TSharedPtr<FConversionFunctionItem>& Parent, TArray<FConversionFunctionValue>& OutFunctions)
{
	for (const TSharedPtr<FConversionFunctionItem>& Item : Parent->Children)
	{
		if (Item->Function.IsValid())
		{
			int32 Index = 0;
			for (; Index < OutFunctions.Num(); ++Index)
			{
				if (OutFunctions[Index].GetFName().Compare(Item->Function.GetFName()) > 0)
				{
					break;
				}
			}

			OutFunctions.Insert(Item->Function, Index);
		}
		else
		{
			AddConversionFunctionChildrenRecursive(Item, OutFunctions);
		}
	}
}

void SFieldSelectorMenu::FilterConversionFunctions()
{
	if (bIsSelectingConversion)
	{
		FilteredConversionFunctions.Reset();
		for (const FMVVMFieldConversion& AvailableConversion : AvailableFieldConversions)
		{
			if (AvailableConversion.IsValid())
			{
				FilteredConversionFunctions.Add(AvailableConversion.Function);
			}
		}
		return;
	}

	TArray<FString> FilterStrings;
	if (SearchBox.IsValid())
	{
		SearchBox->GetText().ToString().ParseIntoArrayWS(FilterStrings);
	}

	if (FilterStrings.IsEmpty())
	{
		FilteredConversionFunctions = ConversionFunctions;
		return;
	}

	FilteredConversionFunctions.Reset();
	for (const FConversionFunctionValue& Function : ConversionFunctions)
	{
		FString FunctionName = Function.GetName();
		FString DisplayName = Function.GetDisplayName().ToString();
		FString MetadataKeywords;
		if (Function.IsFunction())
		{
			MetadataKeywords = Function.GetFunction()->GetMetaDataText(FBlueprintMetadata::MD_FunctionKeywords, TEXT("UObjectKeywords"), Function.GetFullGroupName(false)).ToString();
		}

		bool bMatches = true;
		for (const FString& Filter : FilterStrings)
		{
			if (!FunctionName.Contains(Filter) && !DisplayName.Contains(Filter) && !MetadataKeywords.Contains(Filter))
			{
				bMatches = false;
				break;
			}
		}

		if (bMatches)
		{
			FilteredConversionFunctions.Add(Function);
		}
	}
}

void SFieldSelectorMenu::HandleConversionFunctionCategorySelected(TSharedPtr<FConversionFunctionItem> SelectedItem, ESelectInfo::Type)
{
	if (!bIsMenuInitialized)
	{
		return;
	}

	ConversionFunctions.Reset();
	FilteredConversionFunctions.Reset();

	TArray<TSharedPtr<FConversionFunctionItem>> SelectedItems = ConversionFunctionCategoryTree->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		if (BindingList.IsValid())
		{
			BindingList->ClearSources();
		}

		if (ViewModelList.IsValid())
		{
			ViewModelList->ClearSelection();
		}

		if (WidgetList.IsValid())
		{
			WidgetList->ClearSelection();
		}

		for (const TSharedPtr<FConversionFunctionItem>& Item : SelectedItems)
		{
			AddConversionFunctionChildrenRecursive(Item, ConversionFunctions);
		}

		Algo::SortBy(ConversionFunctions, &FConversionFunctionValue::GetDisplayName, FText::FSortPredicate());

		FilterConversionFunctions();
	}

	if (ConversionFunctionList.IsValid())
	{
		ConversionFunctionList->RequestListRefresh();
	}
}

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
