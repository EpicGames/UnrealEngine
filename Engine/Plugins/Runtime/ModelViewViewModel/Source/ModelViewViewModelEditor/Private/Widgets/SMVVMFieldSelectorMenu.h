// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/MVVMBindingMode.h"
#include "Types/MVVMBindingSource.h"
#include "Types/MVVMLinkedPinValue.h"

#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SMVVMViewModelBindingListWidget.h"

class SPrimaryButton;

namespace ESelectInfo { enum Type : int; }
class ITableRow;
class SSearchBox;
class SReadOnlyHierarchyView;
class STableViewBase;

class SPropertyPath;
template <typename ItemType> class SListView;
template <typename ItemType> class STreeView;

class UWidgetBlueprint;

template<>
struct TListTypeTraits<UE::MVVM::FConversionFunctionValue>
{
public:
	using NullableType = UE::MVVM::FConversionFunctionValue;
	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<UE::MVVM::FConversionFunctionValue, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<UE::MVVM::FConversionFunctionValue, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<UE::MVVM::FConversionFunctionValue>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector&, TArray<UE::MVVM::FConversionFunctionValue>&, TSet<UE::MVVM::FConversionFunctionValue>&, TMap< const U*, UE::MVVM::FConversionFunctionValue>&)
	{
	}

	static bool IsPtrValid(UE::MVVM::FConversionFunctionValue InPtr)
	{
		return InPtr.IsValid();
	}

	static void ResetPtr(UE::MVVM::FConversionFunctionValue& InPtr)
	{
		InPtr = UE::MVVM::FConversionFunctionValue();
	}

	static  UE::MVVM::FConversionFunctionValue MakeNullPtr()
	{
		return UE::MVVM::FConversionFunctionValue();
	}

	static  UE::MVVM::FConversionFunctionValue NullableItemTypeConvertToItemType(UE::MVVM::FConversionFunctionValue InPtr)
	{
		return InPtr;
	}

	static FString DebugDump( UE::MVVM::FConversionFunctionValue InPtr)
	{
		return InPtr.GetName();
	}

	class SerializerType {};
};

template <>
struct TIsValidListItem<UE::MVVM::FConversionFunctionValue>
{
	enum
	{
		Value = true
	};
};

namespace UE::MVVM
{

struct FFieldSelectionContext
{
	EMVVMBindingMode BindingMode = EMVVMBindingMode::OneWayToDestination;
	const FProperty* AssignableTo = nullptr;
	TOptional<FBindingSource> FixedBindingSource;
	bool bAllowWidgets = true;
	bool bAllowViewModels = true;
	bool bAllowConversionFunctions = true;
	bool bReadable = true;
	bool bWritable = true;
};

class SFieldSelectorMenu : public SCompoundWidget
{
public:
	enum class ESelectionType
	{
		None,
		Binding,
		Event,
	};

	DECLARE_DELEGATE_RetVal(FFieldSelectionContext, FOnGetFieldSelectionContext);
	DECLARE_DELEGATE_TwoParams(FOnLinkedValueSelected, FMVVMLinkedPinValue, ESelectionType);
	DECLARE_DELEGATE_ThreeParams(FOnLinkedValueConversionSelected, FMVVMLinkedPinValue, const FMVVMFieldConversion&, ESelectionType);

	SLATE_BEGIN_ARGS(SFieldSelectorMenu){}
		SLATE_ARGUMENT(TOptional<FMVVMLinkedPinValue>, CurrentSelected)
		SLATE_EVENT(FOnLinkedValueSelected, OnSelected)
		SLATE_EVENT(FOnLinkedValueConversionSelected, OnSelectedConversion)
		SLATE_EVENT(FSimpleDelegate, OnMenuCloseRequested)
		SLATE_ARGUMENT(FFieldSelectionContext, SelectionContext)
		SLATE_ARGUMENT_DEFAULT(bool, IsBindingToEvent) { false };
		SLATE_ARGUMENT_DEFAULT(bool, CanCreateEvent) { false };
		SLATE_ARGUMENT_DEFAULT(bool, IsEnableContextToggleAvailable) { true };
		SLATE_ARGUMENT_DEFAULT(bool, IsClearEnableable) { true };
		SLATE_ARGUMENT_DEFAULT(bool, ShowTooltipText) { false };

		/**
		* If CanConvertField is set to true
		* - Menu will display Fields that can be converted to the Selected Property.
		* - Selections will be return via OnSelectedConversion
		*/
		SLATE_ARGUMENT_DEFAULT(bool, CanConvertField) { false };
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	TSharedRef<SWidget> GetWidgetToFocus() const;

private:
	struct FConversionFunctionItem
	{
		FString GetCategoryName() { return CategoryPath.Num() > 0 ? CategoryPath.Last() : FString(); }

		TArray<FString> CategoryPath;
		TArray<FString> SearchKeywords;
		UE::MVVM::FConversionFunctionValue Function;
		TArray<TSharedPtr<FConversionFunctionItem>> Children;
		int32 NumFunctions = 0;
	};

private:
	void SetPropertyPathSelection(const FMVVMBlueprintPropertyPath& SelectedPath);
	void SetConversionFunctionSelection(const UE::MVVM::FConversionFunctionValue SelectedFunction);

	TSharedRef<SWidget> CreateBindingContextPanel(const FArguments& InArgs);
	TSharedRef<SWidget> CreateBindingListPanel(const FArguments& InArgs, const FProperty* AssignableToProperty);

	void HandleWidgetSelected(FName WidgetName, ESelectInfo::Type);

	void HandleViewModelSelected(FBindingSource ViewModel, ESelectInfo::Type);
	TSharedRef<ITableRow> HandleGenerateViewModelRow(MVVM::FBindingSource ViewModel, const TSharedRef<STableViewBase>& OwnerTable) const;

	int32 SortConversionFunctionItemsRecursive(TArray<TSharedPtr<FConversionFunctionItem>>& Items);
	void GenerateConversionFunctionItems();
	void FilterConversionFunctionCategories();
	void FilterConversionFunctions();
	/** Recursively filter the items in SourceArray and place them into DestArray. Returns true if any items were added. */
	int32 FilterConversionFunctionCategoryChildren(const TArray<FString>& FilterStrings, const TArray<TSharedPtr<FConversionFunctionItem>>& SourceArray, TArray<TSharedPtr<FConversionFunctionItem>>& OutDestArray);
	void AddConversionFunctionChildrenRecursive(const TSharedPtr<FConversionFunctionItem>& Parent, TArray<UE::MVVM::FConversionFunctionValue>& OutFunctions);
	TSharedPtr<FConversionFunctionItem> FindOrCreateItemForCategory(TArray<TSharedPtr<FConversionFunctionItem>>& Items, TArrayView<FString> CategoryPath);
	TSharedPtr<FConversionFunctionItem> FindConversionFunctionCategory(const TArray<TSharedPtr<FConversionFunctionItem>>& Items, TArrayView<FString> CategoryNameParts) const;
	void HandleGetConversionFunctionCategoryChildren(TSharedPtr<FConversionFunctionItem> Item, TArray<TSharedPtr<FConversionFunctionItem>>& OutItems) const;
	void HandleConversionFunctionCategorySelected(TSharedPtr<FConversionFunctionItem> Item, ESelectInfo::Type);
	TSharedRef<ITableRow> HandleGenerateConversionFunctionCategoryRow(TSharedPtr<FConversionFunctionItem> Item, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedRef<ITableRow> HandleGenerateConversionFunctionRow(const UE::MVVM::FConversionFunctionValue Function, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedPtr<FConversionFunctionItem> ExpandFunctionCategoryTreeToItem(const UE::MVVM::FConversionFunctionValue Function);
	void ExpandFunctionCategoryTree(const TArray<TSharedPtr<FConversionFunctionItem>>& Items, bool bRecursive);

	void FilterViewModels(const FText& NewText);
	void HandleSearchBoxTextChanged(const FText& NewText);
	void HandleEnabledContextToggleChanged(ECheckBoxState CheckState);
	ECheckBoxState ToggleEnabledContext() const;

	TOptional<FMVVMLinkedPinValue> GetCurrentSelection() const;
	void UpdateSelection();
	void SetSelectedValue(const TOptional<FMVVMLinkedPinValue>& NewSelection);

	FText GetTooltipText() const;
	FText GetAdditionalTooltipText() const;
	FMVVMBlueprintPropertyPath GetTooltipPropertyPath() const;

	bool IsSelectEnabled() const;
	bool IsEventSelectEnabled() const;

	FReply HandleSelectClicked();
	FReply HandleEventSelectClicked();
	FReply HandleClearClicked();
	FReply HandleCancelClicked();

	FReply PromptConversionSelection();

private:
	TWeakObjectPtr<const UWidgetBlueprint> WidgetBlueprint;
	FOnLinkedValueSelected OnSelected;
	FOnLinkedValueConversionSelected OnSelectedConversion;

	FSimpleDelegate OnMenuCloseRequested;
	FFieldSelectionContext SelectionContext;

	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SPrimaryButton> EventSelectButton;

	//~ viewmodels (binding context panel)
	TSharedPtr<SListView<FBindingSource>> ViewModelList;
	TArray<FBindingSource> ViewModelSources;
	TArray<FBindingSource> FilteredViewModelSources;

	//~ widgets (binding context panel)
	TSharedPtr<SReadOnlyHierarchyView> WidgetList;

	//~ viewmodel and widgets (selection panel)
	TSharedPtr<SSourceBindingList> BindingList;

	//~ functions (binding context panel)
	TSharedPtr<STreeView<TSharedPtr<FConversionFunctionItem>>> ConversionFunctionCategoryTree;
	TArray<TSharedPtr<FConversionFunctionItem>> FilteredConversionFunctionRoot;
	TArray<TSharedPtr<FConversionFunctionItem>> ConversionFunctionRoot;

	//~ functions (selection panel)
	TSharedPtr<SListView<UE::MVVM::FConversionFunctionValue>> ConversionFunctionList;
	TArray<UE::MVVM::FConversionFunctionValue> ConversionFunctions;
	TArray<UE::MVVM::FConversionFunctionValue> FilteredConversionFunctions;
	TArray<FMVVMFieldConversion> AvailableFieldConversions;

	TOptional<FMVVMLinkedPinValue> CurrentSelectedValue;
	TOptional<FMVVMFieldConversion> CurrentSelectedConversionValue;

	bool bIsMenuInitialized = false;
	bool bCanCreateEvent = false;	
	bool bCanConvertField = false;
	bool bIsSelectingConversion = false;
};

} // namespace UE::MVVM
