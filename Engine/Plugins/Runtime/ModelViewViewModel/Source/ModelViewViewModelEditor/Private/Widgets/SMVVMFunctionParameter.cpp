// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMVVMFunctionParameter.h" 
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Framework/MVVMRowHelper.h"
#include "MVVMBlueprintFunctionReference.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMEditorSubsystem.h"
#include "NodeFactory.h"
#include "SGraphPin.h"
#include "Styling/MVVMEditorStyle.h"
#include "Types/MVVMBindingMode.h"

#include "WidgetBlueprint.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMVVMFieldSelector.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FunctionParameter"

namespace UE::MVVM
{

void SFunctionParameter::Construct(const FArguments& InArgs, UWidgetBlueprint* InWidgetBlueprint)
{
	WidgetBlueprint = InWidgetBlueprint;
	check(InWidgetBlueprint != nullptr);

	BindingId = InArgs._BindingId;
	check(BindingId.IsValid());

	ParameterId = InArgs._ParameterId;
	check(ParameterId.IsValid());
	
	bSourceToDestination = InArgs._SourceToDestination;
	bAllowDefault = InArgs._AllowDefault;

	UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	UMVVMBlueprintView* View = EditorSubsystem->GetView(InWidgetBlueprint);
	check(View);

	const FMVVMBlueprintViewBinding* Binding = View->GetBinding(BindingId);
	check(Binding);

	TSharedRef<SWidget> ValueWidget = SNullWidget::NullWidget;
	bool bIsBooleanPin = false;

	if (UEdGraphPin* Pin = EditorSubsystem->GetConversionFunctionArgumentPin(InWidgetBlueprint, *Binding, ParameterId, bSourceToDestination))
	{
		bIsBooleanPin = Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean;
		// create a new pin widget so that we can get the default value widget out of it
		if (TSharedPtr<SGraphPin> PinWidget = FNodeFactory::CreatePinWidget(Pin))
		{
			GraphPin = PinWidget;
			ValueWidget = PinWidget->GetDefaultValueWidget();
		}
	}

	if (ValueWidget == SNullWidget::NullWidget)
	{
		ValueWidget = SNew(STextBlock)
			.Text(LOCTEXT("DefaultValue", "Default Value"))
			.TextStyle(FAppStyle::Get(), "HintText");
	}
	// booleans are represented by a checkbox which doesn't expand to the minsize we have, so don't put a border around them
	else if (!bIsBooleanPin)
	{
		ValueWidget = SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FMVVMEditorStyle::Get().GetBrush("FunctionParameter.Border"))
			[
				ValueWidget
			];
	}

	bDefaultValueVisible = !OnGetSelectedField().IsValid();

	bool bFromViewModel = UE::MVVM::IsForwardBinding(Binding->BindingType);
	TSharedPtr<SHorizontalBox> HBox;

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(bIsBooleanPin ? FOptionalSize() : 100)
		[
			SAssignNew(HBox, SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex(this, &SFunctionParameter::GetCurrentWidgetIndex)
				+ SWidgetSwitcher::Slot()
				[
					ValueWidget
				]
				+ SWidgetSwitcher::Slot()
				[
					SNew(SFieldSelector, WidgetBlueprint.Get())
					.OnDrop(this, &SFunctionParameter::HandleFieldSelectorDrop)
					.OnDragOver(this, &SFunctionParameter::HandleFieldSelectorDragOver)
					.OnGetLinkedValue(this, &SFunctionParameter::OnGetSelectedField)
					.OnSelectionChanged(this, &SFunctionParameter::HandleFieldSelectionChanged)
					.OnGetSelectionContext(this, &SFunctionParameter::GetSelectedSelectionContext)
				]
			]
		]
	];

	if (InArgs._AllowDefault)
	{
		HBox->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(8, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("BindArgument", "Bind this argument to a property."))
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.IsChecked(this, &SFunctionParameter::OnGetIsBindArgumentChecked)
				.OnCheckStateChanged(this, &SFunctionParameter::OnBindArgumentChecked)
				.Padding(FMargin(4))
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16, 16))
					.Image_Lambda([this]()
						{
							ECheckBoxState CheckState = OnGetIsBindArgumentChecked();
							return (CheckState == ECheckBoxState::Checked) ? FAppStyle::GetBrush("Icons.Link") : FAppStyle::GetBrush("Icons.Unlink");
						})
				]
			];		
	}
}

int32 SFunctionParameter::GetCurrentWidgetIndex() const
{
	return bDefaultValueVisible && bAllowDefault ? 0 : 1;
}

ECheckBoxState SFunctionParameter::OnGetIsBindArgumentChecked() const 
{
	return bDefaultValueVisible ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}

void SFunctionParameter::OnBindArgumentChecked(ECheckBoxState Checked)
{
	UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();

	bDefaultValueVisible = (Checked != ECheckBoxState::Checked);

	if (bDefaultValueVisible)
	{
		PreviousSelectedField = OnGetSelectedField();
		SetSelectedField(FMVVMLinkedPinValue());
	}
	else
	{
		SetSelectedField(PreviousSelectedField);
	}
}

FMVVMLinkedPinValue SFunctionParameter::OnGetSelectedField() const
{
	if (const UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get())
	{
		const UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		if (const UMVVMBlueprintView* View = Subsystem->GetView(WidgetBlueprintPtr))
		{
			if (const FMVVMBlueprintViewBinding* Binding = View->GetBinding(BindingId))
			{
				return FMVVMLinkedPinValue(Subsystem->GetPathForConversionFunctionArgument(WidgetBlueprint.Get(), *Binding, ParameterId, bSourceToDestination));
			}
		}
	}
	return FMVVMLinkedPinValue();
}

void SFunctionParameter::SetSelectedField(const FMVVMLinkedPinValue& Value)
{
	if (const UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get())
	{
		const UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		if (UMVVMBlueprintView* View = Subsystem->GetView(WidgetBlueprintPtr))
		{
			if (FMVVMBlueprintViewBinding* Binding = View->GetBinding(BindingId))
			{
				Subsystem->SetPathForConversionFunctionArgument(WidgetBlueprint.Get(), *Binding, ParameterId, Value.IsPropertyPath() ? Value.GetPropertyPath() : FMVVMBlueprintPropertyPath(), bSourceToDestination);
			}
		}
	}
}

void SFunctionParameter::HandleFieldSelectionChanged(FMVVMLinkedPinValue Value, SFieldSelectorMenu::ESelectionType SelectionType)
{
	SetSelectedField(Value);
}

FFieldSelectionContext SFunctionParameter::GetSelectedSelectionContext() const
{
	FFieldSelectionContext Result;

	UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get();
	if (WidgetBlueprintPtr == nullptr)
	{
		return Result;
	}

	UMVVMBlueprintView* View = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->GetView(WidgetBlueprintPtr);
	check(View);

	const FMVVMBlueprintViewBinding* Binding = View->GetBinding(BindingId);
	if (Binding == nullptr)
	{
		return Result;
	}

	UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding->Conversion.GetConversionFunction(bSourceToDestination);
	if (ConversionFunction == nullptr)
	{
		return Result;
	}

	FProperty* PinProperty = nullptr;
	if (ParameterId.GetNames().Num() > 0)
	{
		FMVVMBlueprintFunctionReference FunctionReference = ConversionFunction->GetConversionFunction();
		if (FunctionReference.GetType() == EMVVMBlueprintFunctionReferenceType::Function)
		{
			const UFunction* Function = FunctionReference.GetFunction(WidgetBlueprintPtr);
			PinProperty = Function ? Function->FindPropertyByName(ParameterId.GetNames().Last()) : nullptr;
		}
	}

	Result.BindingMode = EMVVMBindingMode::OneWayToDestination;
	Result.AssignableTo = PinProperty;
	Result.bAllowWidgets = true;
	Result.bAllowViewModels = true;
	Result.bAllowConversionFunctions = false;
	Result.bReadable = true;
	Result.bWritable = false;

	return Result;
}

FReply SFunctionParameter::HandleFieldSelectorDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (const UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get())
	{
		const UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		if (UMVVMBlueprintView* View = Subsystem->GetView(WidgetBlueprintPtr))
		{
			if (FMVVMBlueprintViewBinding* Binding = View->GetBinding(BindingId))
			{
				TOptional<FMVVMBlueprintPropertyPath> PropertyPath = BindingEntry::FRowHelper::DropFieldSelector(WidgetBlueprintPtr, DragDropEvent, bSourceToDestination);
				if (PropertyPath.IsSet())
				{
					Subsystem->SetPathForConversionFunctionArgument(WidgetBlueprint.Get(), *Binding, ParameterId, PropertyPath.GetValue(), bSourceToDestination);
				}
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

FReply SFunctionParameter::HandleFieldSelectorDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get();
	if (WidgetBlueprintPtr == nullptr)
	{
		return FReply::Unhandled();
	}

	return BindingEntry::FRowHelper::DragOverFieldSelector(WidgetBlueprintPtr, DragDropEvent, bSourceToDestination);
}

}//namespace
#undef LOCTEXT_NAMESPACE
