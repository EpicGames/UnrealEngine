// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMPopoutEditor.h"

#include "ClassIconFinder.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/PropertyViewer/IFieldExpander.h"
#include "Internationalization/Text.h"
#include "MVVMSubsystem.h"
#include "Styling/MVVMEditorStyle.h"
#include "Types/MVVMAvailableBinding.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/PropertyViewer/SFieldIcon.h"
#include "Widgets/SMVVMViewModelBindingListWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMVVMPopoutEditor"

namespace UE::MVVM
{

void SMVVMPopoutEditor::OpenPopoutEditor(TNotNull<UObject*> InObjectToInspect, bool bInReadOnly)
{
	const FName TabId = *(FString(TEXT("SMVVMPopoutEditor:")) + InObjectToInspect->GetPathName());

	const TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();

	if (GlobalTabManager->FindExistingLiveTab(TabId))
	{
		GlobalTabManager->TryInvokeTab(TabId);
		return;
	}

	GlobalTabManager->UnregisterNomadTabSpawner(TabId);

	GlobalTabManager->RegisterNomadTabSpawner(
		TabId,
		FOnSpawnTab::CreateLambda(
			[ObjectWeak = TWeakObjectPtr<UObject>(InObjectToInspect), bInReadOnly](const FSpawnTabArgs& InSpawnArgs)
			{
				TSharedRef<SDockTab> Tab = SNew(SDockTab)
					.TabRole(ETabRole::NomadTab)
					.OnTabClosed_Lambda(
						[TabId = InSpawnArgs.GetTabId()](TSharedRef<SDockTab> InTab)
						{
							FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TabId.TabType);
						}
					);

				if (UObject* Object = ObjectWeak.Get())
				{
					const FText Label = FText::Format(
						LOCTEXT("LabelFormat", "{0} [{1}]"),
						FText::FromName(Object->GetFName()),
						FText::FromString(Object->GetPathName())
					);

					Tab->SetLabel(Label);
					Tab->SetTabIcon(FClassIconFinder::FindThumbnailForClass(Object->GetClass()));

					Tab->SetContent(
						SNew(SMVVMPopoutEditor, Object)
						.bReadOnly(bInReadOnly)
					);
				}

				return Tab;
			}
		)
	);

	if (!GlobalTabManager->TryInvokeTab(TabId).IsValid())
	{
		GlobalTabManager->UnregisterNomadTabSpawner(TabId);
	}
}

void SMVVMPopoutEditor::Construct(const FArguments& InArgs, UObject* InObjectToInspect)
{
	ObjectWeak = InObjectToInspect;
	FieldIterator = MakeUnique<UE::PropertyViewer::FFieldIterator_BlueprintVisible>();
	FieldExpander = MakeUnique<FFieldExpander_Bindable>();

	PropertyViewer = SNew(UE::PropertyViewer::SPropertyViewer)
		.PropertyVisibility(
			InArgs._bReadOnly
				? UE::PropertyViewer::SPropertyViewer::EPropertyVisibility::Visible
				: UE::PropertyViewer::SPropertyViewer::EPropertyVisibility::Editable
		)
		.bShowSearchBox(true)
		.bShowFieldIcon(true)
		.bSanitizeName(true)
		.FieldIterator(FieldIterator.Get())
		.FieldExpander(FieldExpander.Get())
		.OnGetPreSlot(this, &SMVVMPopoutEditor::HandleGetPreSlot)
		.OnGenerateContainer(this, &SMVVMPopoutEditor::HandleGenerateContainer);

	PropertyViewer->AddInstance(InObjectToInspect);

	ChildSlot
	[
		PropertyViewer.ToSharedRef()
	];
}

TSharedPtr<SWidget> SMVVMPopoutEditor::HandleGetPreSlot(UE::PropertyViewer::SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> FieldPath)
{
	if (FieldPath.Num() > 0)
	{
		return ConstructFieldPreSlot(ObjectWeak.Get(), Handle, FieldPath.Last());
	}

	return TSharedPtr<SWidget>();
}

TSharedRef<SWidget> SMVVMPopoutEditor::HandleGenerateContainer(UE::PropertyViewer::SPropertyViewer::FHandle ContainerHandle, TOptional<FText> DisplayName)
{
	if (UObject* Object = ObjectWeak.Get())
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(UE::PropertyViewer::SFieldIcon, Object->GetClass())
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromName(Object->GetFName()))
			];
	}

	return SNew(STextBlock)
		.Text(LOCTEXT("ObjectError", "Error. Invalid object."));
}

TSharedRef<SWidget> SMVVMPopoutEditor::ConstructFieldPreSlot(const UObject* InObject, UE::PropertyViewer::SPropertyViewer::FHandle Handle, const FFieldVariant FieldPath, const bool bIsForEvent)
{
	TSharedRef<SWidget> ImageWidget = SNullWidget::NullWidget;
	const UClass* ObjectClass = InObject->GetClass();
	FMVVMAvailableBinding Binding = bIsForEvent ? UMVVMSubsystem::GetAvailableBindingForEvent(FMVVMConstFieldVariant(FieldPath), ObjectClass) : UMVVMSubsystem::GetAvailableBindingForField(FMVVMConstFieldVariant(FieldPath), ObjectClass);
	if (Binding.IsValid())
	{
		const FSlateBrush* Brush = nullptr;
		if (Binding.HasNotify())
		{
			if (Binding.IsReadable() && Binding.IsWritable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.TwoWay");
			}
			else if (Binding.IsReadable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.OneWayToSource");
			}
			else if (Binding.IsWritable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.OneWay");
			}
		}
		else
		{
			if (Binding.IsReadable() && Binding.IsWritable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.OneTimeTwoWay");
			}
			else if (Binding.IsReadable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.OneTimeOneWay");
			}
			else if (Binding.IsWritable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.OneTimeOneWayToSource");
			}
		}

		if (Brush)
		{
			ImageWidget = SNew(SImage)
				.Image(Brush);
		}
	}

	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.WidthOverride(16.0f)
		.HeightOverride(16.0f)
		[
			ImageWidget
		];
}

}

#undef LOCTEXT_NAMESPACE
