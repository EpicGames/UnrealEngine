// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Misc/NotNull.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/PropertyViewer/SPropertyViewer.h"

class UObject;

namespace UE::PropertyViewer
{
	class FFieldExpander_Default;
	class FFieldIterator_BlueprintVisible;
}

namespace UE::MVVM
{

class FFieldIterator_Bindable;

class SMVVMPopoutEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMVVMPopoutEditor)
		: _bReadOnly(true)
		{}
		SLATE_ARGUMENT(bool, bReadOnly)
	SLATE_END_ARGS()

	static void OpenPopoutEditor(TNotNull<UObject*> InObjectToInspect, bool bInReadOnly = true);

	void Construct(const FArguments& InArgs, UObject* InObjectToInspect);

private:
	TWeakObjectPtr<UObject> ObjectWeak;
	TUniquePtr<UE::PropertyViewer::FFieldIterator_BlueprintVisible> FieldIterator;
	TUniquePtr<UE::PropertyViewer::FFieldExpander_Default> FieldExpander;
	TSharedPtr<UE::PropertyViewer::SPropertyViewer> PropertyViewer;

	TSharedPtr<SWidget> HandleGetPreSlot(UE::PropertyViewer::SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> FieldPath);
	TSharedRef<SWidget> HandleGenerateContainer(UE::PropertyViewer::SPropertyViewer::FHandle ContainerHandle, TOptional<FText> DisplayName);

	TSharedRef<SWidget> ConstructFieldPreSlot(const UObject* InObject, UE::PropertyViewer::SPropertyViewer::FHandle Handle, const FFieldVariant FieldPath, const bool bIsForEvent = false);
};

} // UE::MVVM