// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMPropertyPath.h"

#include "Bindings/MVVMFieldPathHelper.h"
#include "Blueprint/WidgetTree.h"
#include "Editor.h"
#include "MVVMBlueprintView.h"
#include "MVVMEditorSubsystem.h"
#include "Styling/MVVMEditorStyle.h"
#include "Types/MVVMBindingSource.h"
#include "WidgetBlueprint.h"

#include "Widgets/SMVVMFieldEntry.h"
#include "Widgets/SMVVMFieldIcon.h"
#include "Widgets/SMVVMSourceEntry.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SPropertyPath"

namespace UE::MVVM
{

void SPropertyPath::Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint)
{
	WidgetBlueprint = InWidgetBlueprint;
	TextStyle = InArgs._TextStyle;
	bShowContext = InArgs._ShowContext;
	bShowOnlyLastPath = InArgs._ShowOnlyLastPath;
	bShowFieldNotify = InArgs._ShowFieldNotify;

	ChildSlot
	[
		SAssignNew(FieldBox, SHorizontalBox)
	];
	SetPropertyPath(InArgs._PropertyPath);
}


void SPropertyPath::SetPropertyPath(const FMVVMBlueprintPropertyPath& InPropertyPath)
{
	FieldBox->ClearChildren();

	const UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get();
	const UClass* ClassContext = WidgetBlueprintPtr->SkeletonGeneratedClass ? WidgetBlueprintPtr->SkeletonGeneratedClass : WidgetBlueprintPtr->GeneratedClass;
	EMVVMBlueprintFieldPathSource Source = InPropertyPath.GetSource(WidgetBlueprintPtr);

	if (Source != EMVVMBlueprintFieldPathSource::None)
	{
		TArray<UE::MVVM::FMVVMConstFieldVariant> AllFields = InPropertyPath.GetCompleteFields(WidgetBlueprintPtr, true);

		// Find the NotifyFieldId to highlight it. Error are valid.
		TOptional<FInt32Range> HighlightRange;
		TValueOrError<UE::MVVM::FieldPathHelper::FParsedNotifyBindingInfo, FText> FieldPathResult = UE::MVVM::FieldPathHelper::GetNotifyBindingInfoFromFieldPath(ClassContext, AllFields);
		if (FieldPathResult.HasValue())
		{
			int32 RangeStart = FieldPathResult.GetValue().ViewModelIndex;
			HighlightRange = FInt32Range(RangeStart, RangeStart + 1);
		}

		// Remove the first item to show it as a SBindingContextEntry
		if ((Source == EMVVMBlueprintFieldPathSource::Widget || Source == EMVVMBlueprintFieldPathSource::ViewModel) && AllFields.Num() > 0)
		{
			AllFields.RemoveAt(0);
		}


		bool bHasSource = InPropertyPath.IsValid();
		if (bShowContext && bHasSource)
		{
			FieldBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SBindingContextEntry)
					.TextStyle(TextStyle)
					.BindingContext(FBindingSource::CreateFromPropertyPath(WidgetBlueprintPtr, InPropertyPath))
				];

			if (InPropertyPath.HasPaths())
			{
				FieldBox->AddSlot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
					];
			}
		}

		if (InPropertyPath.HasPaths())
		{
			FieldBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SFieldPaths)
					.TextStyle(TextStyle)
					.FieldPaths(AllFields)
					.HighlightField(bShowFieldNotify ? HighlightRange : TOptional<FInt32Range>())
					.ShowOnlyLast(bShowOnlyLastPath)
				];
		}
	}
	else
	{
		if (InPropertyPath.HasPaths())
		{
			FieldBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SFieldPaths)
					.TextStyle(TextStyle)
					.FieldPaths(InPropertyPath.GetFields(ClassContext))
					.ShowOnlyLast(bShowOnlyLastPath)
				];
		}
	}
}


} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
