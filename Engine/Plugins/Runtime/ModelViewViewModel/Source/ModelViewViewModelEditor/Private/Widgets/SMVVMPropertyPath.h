// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMPropertyPath.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SCompoundWidget.h"

class SHorizontalBox;
class UWidgetBlueprint;

namespace UE::MVVM
{

class SPropertyPath : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPropertyPath)
		: _TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
	{}
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_ARGUMENT(FMVVMBlueprintPropertyPath, PropertyPath)
		SLATE_ARGUMENT_DEFAULT(bool, ShowContext) = true;
		SLATE_ARGUMENT_DEFAULT(bool, ShowOnlyLastPath) = false;
		SLATE_ARGUMENT_DEFAULT(bool, ShowFieldNotify) = true;
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const UWidgetBlueprint* WidgetBlueprint);
	void SetPropertyPath(const FMVVMBlueprintPropertyPath& InPropertyPath);

private:
	TWeakObjectPtr<const UWidgetBlueprint> WidgetBlueprint;
	const FTextBlockStyle* TextStyle = nullptr;
	TSharedPtr<SHorizontalBox> FieldBox;
	bool bShowContext = false;
	bool bShowOnlyLastPath = false;
	bool bShowFieldNotify = true;
};

} // namespace UE::MVVM
