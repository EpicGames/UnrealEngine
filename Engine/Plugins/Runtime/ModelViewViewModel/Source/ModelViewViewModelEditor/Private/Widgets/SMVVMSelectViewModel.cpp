// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMSelectViewModel.h"

#include "Components/Widget.h"
#include "Extensions/UIComponent.h"
#include "Modules/ModuleManager.h"
#include "SPrimaryButton.h"
#include "Types/MVVMBindingSource.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMVVMViewModelBindingListWidget.h"

#include "ClassViewerModule.h"

#define LOCTEXT_NAMESPACE "SMVVMSelectViewModel"

namespace UE::MVVM
{

namespace Private
{
	bool IsValidViewModel(const UClass* InClass, EClassFlags DisallowedClassFlags)
	{
		if (InClass == nullptr || InClass->IsChildOf(UWidget::StaticClass()) || InClass->IsChildOf(UUIComponent::StaticClass()))
		{
			return false;
		}
		return InClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()) && !InClass->HasAnyClassFlags(DisallowedClassFlags);
	}
}

bool FViewModelClassFilter::IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs)
{
	return Private::IsValidViewModel(InClass, DisallowedClassFlags);
}

bool FViewModelClassFilter::IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs)
{
	if (InUnloadedClassData->IsChildOf(UWidget::StaticClass()))
	{
		return false;
	}
	return InUnloadedClassData->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()) && !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags);
}


void SMVVMSelectViewModel::Construct(const FArguments& InArgs, const UWidgetBlueprint* WidgetBlueprint)
{
	OnCancel = InArgs._OnCancel;
	OnViewModelCommitted = InArgs._OnViewModelCommitted;

	TSharedRef<FViewModelClassFilter> ClassFilter = MakeShared<FViewModelClassFilter>();
	DisallowedClassFlags = InArgs._DisallowedClassFlags.Get(ClassFilter->DisallowedClassFlags);
	ClassFilter->DisallowedClassFlags = InArgs._DisallowedClassFlags.GetValue();

	FClassViewerInitializationOptions ClassViewerOptions;
	ClassViewerOptions.DisplayMode = EClassViewerDisplayMode::TreeView;
	ClassViewerOptions.Mode = EClassViewerMode::ClassPicker;
	ClassViewerOptions.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	ClassViewerOptions.ClassFilters.Add(ClassFilter);

	ClassViewer = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer")
		.CreateClassViewer(ClassViewerOptions, FOnClassPicked::CreateSP(this, &SMVVMSelectViewModel::HandleClassPicked));

	ChildSlot
	[
		SNew(SBorder)
		.Padding(1)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SSplitter)
					.PhysicalSplitterHandleSize(2.0f)
					+ SSplitter::Slot()
					.Value(0.6f)
					[
						SNew(SBorder)
						.BorderImage(FStyleDefaults::GetNoBrush())
						[
							ClassViewer.ToSharedRef()
						]
					]
					+ SSplitter::Slot()
					.Value(0.4f)
					[
						SNew(SBorder)
						.BorderImage(FStyleDefaults::GetNoBrush())
						.Padding(4, 30, 4, 30)
						[
							SAssignNew(BindingListWidget, SSourceBindingList, WidgetBlueprint)
							.EnableSelection(false)
							.FieldVisibilityFlags(EFieldVisibility::None)
						]
					]
				]
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.AutoHeight()
				.Padding(8)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
					.AutoWidth()
					[
						SNew(SPrimaryButton)
						.Text(LOCTEXT("ViewModelSelectButtonText", "Select"))
						.OnClicked(this, &SMVVMSelectViewModel::HandleSelected)
						.IsEnabled(this, &SMVVMSelectViewModel::HandleIsSelectionEnabled)
					]
					+ SHorizontalBox::Slot()
					.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
					.AutoWidth()
					[
						SNew(SButton)
						.Text(LOCTEXT("ViewModelCancelButtonText", "Cancel"))
						.HAlign(HAlign_Center)
						.OnClicked(this, &SMVVMSelectViewModel::HandleCancel)
					]
				]
			]
		]
	];
}

void SMVVMSelectViewModel::HandleClassPicked(UClass* ClassPicked)
{
	BindingListWidget->ClearSources();
	SelectedClass.Reset();

	if (Private::IsValidViewModel(ClassPicked, DisallowedClassFlags))
	{
		SelectedClass = ClassPicked;
		BindingListWidget->AddSource(FBindingSource::CreateEmptySource(ClassPicked));
	}
}


FReply SMVVMSelectViewModel::HandleSelected()
{
	OnViewModelCommitted.ExecuteIfBound(SelectedClass.Get());
	return FReply::Handled();
}


FReply SMVVMSelectViewModel::HandleCancel()
{
	OnCancel.ExecuteIfBound();
	return FReply::Handled();
}


bool SMVVMSelectViewModel::HandleIsSelectionEnabled() const
{
	return SelectedClass.Get() != nullptr;
}

} //namespace

#undef LOCTEXT_NAMESPACE
