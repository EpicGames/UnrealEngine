// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClassViewerFilter.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/ObjectMacros.h"

class SClassViewer;
class UWidgetBlueprint;

namespace UE::MVVM
{
class SSourceBindingList;

class FViewModelClassFilter : public IClassViewerFilter
{
public:
	FViewModelClassFilter() = default;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override;
	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override;

	EClassFlags DisallowedClassFlags = CLASS_HideDropDown | CLASS_Hidden | CLASS_Deprecated | CLASS_Abstract;
};

class SMVVMSelectViewModel : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnValueChanged, const UClass*);

	SLATE_BEGIN_ARGS(SMVVMSelectViewModel) {}
		SLATE_EVENT(FSimpleDelegate, OnCancel)
		SLATE_EVENT(FOnValueChanged, OnViewModelCommitted)
		SLATE_ARGUMENT(TOptional<EClassFlags>, DisallowedClassFlags)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const UWidgetBlueprint* WidgetBlueprint);

private:
	void HandleClassPicked(UClass* ClassPicked);
	FReply HandleSelected();
	FReply HandleCancel();
	bool HandleIsSelectionEnabled() const;

private:
	TSharedPtr<SWidget> ClassViewer;
	TSharedPtr<SSourceBindingList> BindingListWidget;
	TWeakObjectPtr<const UClass> SelectedClass;

	FSimpleDelegate OnCancel;
	FOnValueChanged OnViewModelCommitted;
	EClassFlags DisallowedClassFlags;
};

} //namespace
