// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Widgets/SMVVMSourceEntry.h"

#include "Styling/SlateIconFinder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMVVMSource"

namespace UE::MVVM
{
	
namespace Private
{

const FSlateBrush* GetSourceIcon(const FBindingSource& Source)
{
	if (!Source.IsValid())
	{
		return nullptr;
	}

	return FSlateIconFinder::FindIconBrushForClass(Source.GetClass());
}

FLinearColor GetSourceColor(const FBindingSource& Source)
{
	const UClass* SourceClass = Source.GetClass();
	if (SourceClass == nullptr)
	{
		return FLinearColor::White;
	}

	uint32 Hash = GetTypeHash(SourceClass->GetName());
	FLinearColor Color = FLinearColor::White;
	Color.R = ((Hash * 1 % 96) + 32) / 256.f;
	Color.G = ((Hash * 2 % 96) + 32) / 256.f;
	Color.B = ((Hash * 3 % 96) + 32) / 256.f;
	return Color;
}

FText GetSourceDisplayName(const FBindingSource& Source)
{
	return Source.GetDisplayName();
}

} // namespace Private

void SBindingContextEntry::Construct(const FArguments& InArgs)
{
	ToolTipDescription = InArgs._ToolTipDescription;

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SAssignNew(Image, SImage)
		]
		+ SHorizontalBox::Slot()
		.Padding(4, 0, 0, 0)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SAssignNew(Label, STextBlock)
			.TextStyle(InArgs._TextStyle)
			.Clipping(EWidgetClipping::OnDemand)
		]
	];

	RefreshSource(InArgs._BindingContext);
}

void SBindingContextEntry::RefreshSource(const FBindingSource& Source)
{
	Image->SetImage(Private::GetSourceIcon(Source));
	Image->SetColorAndOpacity(Private::GetSourceColor(Source));
	Label->SetText(Private::GetSourceDisplayName(Source));

	if (Source.IsValid())
	{
		if (const UClass* SourceClass = Source.GetClass())
		{
			const FText ToolTipText = ToolTipDescription.IsEmpty() ? 
				FText::Join(FText::FromString(TEXT("\n")), Source.GetDisplayName(), SourceClass->GetDisplayNameText()) 
				: FText::Join(FText::FromString(TEXT("\n")), ToolTipDescription, Source.GetDisplayName(), SourceClass->GetDisplayNameText());
			SetToolTipText(ToolTipText);
		}
		else
		{
			SetToolTipText(ToolTipDescription.IsEmpty() ? Source.GetDisplayName() : FText::Join(FText::FromString(TEXT("\n")), ToolTipDescription, Source.GetDisplayName()));
		}
	}
	else if (!ToolTipDescription.IsEmpty())
	{
		SetToolTipText(ToolTipDescription);
	}
}

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE