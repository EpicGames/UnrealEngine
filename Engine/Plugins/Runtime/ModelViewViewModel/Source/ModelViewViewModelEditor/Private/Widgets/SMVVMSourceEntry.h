// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Types/MVVMBindingSource.h"
#include "Widgets/SCompoundWidget.h"

class SImage;
class STextBlock;

namespace UE::MVVM
{

class SBindingContextEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBindingContextEntry)
		: _TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		{
		}
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_ARGUMENT(UE::MVVM::FBindingSource, BindingContext)
		SLATE_ARGUMENT_DEFAULT(FText, ToolTipDescription) { FText::GetEmpty() };
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);
	void RefreshSource(const UE::MVVM::FBindingSource& Source);

private:
	TSharedPtr<STextBlock> Label;
	TSharedPtr<SImage> Image;
	FText ToolTipDescription;
};

} // namespace UE::MVVM
