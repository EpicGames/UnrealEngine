// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMVVMSourceSelector.h"

#include "Editor.h"
#include "Framework/Views/TableViewMetadata.h"
#include "MVVMEditorSubsystem.h"
#include "Hierarchy/SReadOnlyHierarchyView.h"
#include "SPrimaryButton.h"
#include "MVVMBlueprintView.h"
#include "SSimpleButton.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/SMVVMSourceEntry.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "MVVMSourceSelector"

namespace UE::MVVM
{

void SBindingContextSelector::Construct(const FArguments& Args, const UWidgetBlueprint* InWidgetBlueprint)
{
	TextStyle = Args._TextStyle;

	SelectedSourceAttribute = Args._SelectedBindingSource;
	check(SelectedSourceAttribute.IsSet());

	InitialSource = SelectedSourceAttribute.Get();

	WidgetBlueprint = InWidgetBlueprint;
	check(InWidgetBlueprint);

	bAutoRefresh = Args._AutoRefresh;
	bViewModels = Args._ViewModels;
	bShowClear = Args._ShowClear;
	
	Refresh();

	MenuAnchor = SNew(SMenuAnchor)
		.OnGetMenuContent(this, &SBindingContextSelector::OnGetMenuContent)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton").ButtonStyle)
			.OnClicked_Lambda([this]() 
			{
				MenuAnchor->SetIsOpen(true); 
				return FReply::Handled(); 
			})
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0, 2)
				.HAlign(HAlign_Fill)
				[
					SAssignNew(SelectedSourceWidget, SBindingContextEntry)
					.BindingContext(SelectedSource)
					.TextStyle(TextStyle)
					.ToolTipDescription(LOCTEXT("BindingOwner", "Binding Group Owner"))
				]
				+ SHorizontalBox::Slot()
				.Padding(2, 0)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.ChevronDown"))
				]
			]
		];
		

	if (bShowClear)
	{
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				MenuAnchor.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SSimpleButton)
				.Icon(FAppStyle::Get().GetBrush("Icons.X"))
				.ToolTipText(LOCTEXT("ClearField", "Clear source selection."))
				.Visibility(this, &SBindingContextSelector::GetClearVisibility)
				.OnClicked(this, &SBindingContextSelector::OnClearSource)
			]
		];
	}
	else
	{
		ChildSlot
		[
			MenuAnchor.ToSharedRef()
		];
	}

	OnSelectionChangedDelegate = Args._OnSelectionChanged;
	check(OnSelectionChangedDelegate.IsBound());
}

TSharedRef<SWidget> SBindingContextSelector::OnGetMenuContent()
{
	TSharedRef<SVerticalBox> VBox = SNew(SVerticalBox);

	if (bViewModels)
	{
		ViewModelList = SNew(SListView<FBindingSource>)
			.OnGenerateRow(this, &SBindingContextSelector::OnGenerateRow)
			.OnSelectionChanged(this, &SBindingContextSelector::OnViewModelSelectionChanged)
			.SelectionMode(ESelectionMode::Single)
			.ListItemsSource(&ViewModelSources);

		VBox->AddSlot()
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				.Padding(4)
				[
					ViewModelList.ToSharedRef()
				]
			];
	}
	else
	{
		WidgetHierarchy = SNew(SReadOnlyHierarchyView, WidgetBlueprint.Get())
			.OnSelectionChanged(this, &SBindingContextSelector::OnWidgetSelectionChanged)
			.ShowSearch(false);

		VBox->AddSlot()
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				.Padding(4)
				[
					WidgetHierarchy.ToSharedRef()
				]
			];
	}

	VBox->AddSlot()
		.Padding(4,0,4,4)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
			.AutoWidth()
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("Select", "Select"))
				.OnClicked(this, &SBindingContextSelector::HandleSelect)
				.IsEnabled(this, &SBindingContextSelector::IsSelectEnabled)
			]
			+ SHorizontalBox::Slot()
			.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("Cancel", "Cancel"))
				.HAlign(HAlign_Center)
				.OnClicked(this, &SBindingContextSelector::HandleCancel)
			]
		];

	return SNew(SBox)
		.MinDesiredWidth(300)
		.Padding(1)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(4)
			[
				VBox
			]
		];
}

bool SBindingContextSelector::IsSelectEnabled() const
{
	return SelectedSource.IsValid();
}

FReply SBindingContextSelector::HandleCancel()
{
	OnViewModelSelectionChanged(InitialSource, ESelectInfo::Direct);

	MenuAnchor->SetIsOpen(false);
	return FReply::Handled();
}

FReply SBindingContextSelector::HandleSelect()
{
	if (SelectedSourceWidget.IsValid())
	{
		SelectedSourceWidget->RefreshSource(SelectedSource);
	}

	OnSelectionChangedDelegate.ExecuteIfBound(SelectedSource);

	MenuAnchor->SetIsOpen(false);

	return FReply::Handled();
}

void SBindingContextSelector::OnViewModelSelectionChanged(FBindingSource Selected, ESelectInfo::Type SelectionType)
{
	SelectedSource = Selected;
}

void SBindingContextSelector::OnWidgetSelectionChanged(FName SelectedName, ESelectInfo::Type SelectionType)
{
	FBindingSource Selected = FBindingSource::CreateForWidget(WidgetBlueprint.Get(), SelectedName);

	SelectedSource = Selected;
}

void SBindingContextSelector::Refresh()
{
	SelectedSource = SelectedSourceAttribute.Get();

	if (SelectedSourceWidget.IsValid())
	{
		SelectedSourceWidget->RefreshSource(SelectedSource);
	}

	TSharedPtr<SWidget> SourcePicker;
	if (bViewModels)
	{
		UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		if (const UMVVMBlueprintView* View = EditorSubsystem->GetView(WidgetBlueprint.Get()))
		{
			const TArrayView<const FMVVMBlueprintViewModelContext> ViewModels = View->GetViewModels();
			ViewModelSources.Reserve(ViewModels.Num());

			for (const FMVVMBlueprintViewModelContext& ViewModel : ViewModels)
			{
				ViewModelSources.Add(FBindingSource::CreateForViewModel(WidgetBlueprint.Get(), ViewModel));
			}
		}
	}

	if (WidgetHierarchy.IsValid())
	{
		WidgetHierarchy->Refresh();
	}

	if (ViewModelList.IsValid())
	{
		if (SelectedSource.IsValid())
		{
			ViewModelList->SetItemSelection(SelectedSource, true);
		}
		else
		{
			ViewModelList->ClearSelection();
		}
	}
}

EVisibility SBindingContextSelector::GetClearVisibility() const
{
	return SelectedSource.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SBindingContextSelector::OnClearSource()
{
	if (ViewModelList.IsValid())
	{
		ViewModelList->ClearSelection();
	}

	return FReply::Handled();
}

TSharedRef<ITableRow> SBindingContextSelector::OnGenerateRow(FBindingSource Source, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<FBindingSource>, OwnerTable)
	[
		SNew(SBox)
		.Padding(4, 3)
		[
			SNew(SBindingContextEntry)
			.BindingContext(Source)
			.TextStyle(TextStyle)
		]
	];
}

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
