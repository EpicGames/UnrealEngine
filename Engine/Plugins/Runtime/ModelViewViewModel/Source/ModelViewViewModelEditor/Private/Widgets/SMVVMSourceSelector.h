// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/CoreStyle.h"
#include "Types/MVVMBindingSource.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateWidgetStyleAsset.h"

class SMenuAnchor;
class SReadOnlyHierarchyView;
class STableViewBase;
namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class SListView;

namespace UE::MVVM
{

class SBindingContextEntry;

class SBindingContextSelector : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FSelectionChanged, FBindingSource);

	SLATE_BEGIN_ARGS(SBindingContextSelector) :
		_TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "NormalText" ) )
		{
		}
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_ARGUMENT_DEFAULT(bool, ShowClear) = true;
		SLATE_ARGUMENT_DEFAULT(bool, AutoRefresh) = false;
		SLATE_ARGUMENT_DEFAULT(bool, ViewModels) = false;
		SLATE_ATTRIBUTE(FBindingSource, SelectedBindingSource)
		SLATE_EVENT(FSelectionChanged, OnSelectionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint);
	void Refresh();

private:
	void OnViewModelSelectionChanged(FBindingSource Selected, ESelectInfo::Type SelectionType);
	void OnWidgetSelectionChanged(FName SelectedName, ESelectInfo::Type SelectionType);
	TSharedRef<SWidget> OnGetMenuContent();

	FReply HandleSelect();
	FReply HandleCancel();
	bool IsSelectEnabled() const;

	EVisibility GetClearVisibility() const;
	FReply OnClearSource();

	TSharedRef<ITableRow> OnGenerateRow(FBindingSource Source, const TSharedRef<STableViewBase>& OwnerTable);

private:
	TWeakObjectPtr<const UWidgetBlueprint> WidgetBlueprint;
	FSelectionChanged OnSelectionChangedDelegate;
	const FTextBlockStyle* TextStyle = nullptr;

	TAttribute<TArray<FBindingSource>> AvailableSourcesAttribute;
	TAttribute<FBindingSource> SelectedSourceAttribute;

	TArray<FBindingSource> ViewModelSources;
	FBindingSource InitialSource;
	FBindingSource SelectedSource;

	TSharedPtr<SMenuAnchor> MenuAnchor;
	TSharedPtr<SListView<FBindingSource>> ViewModelList;
	TSharedPtr<SReadOnlyHierarchyView> WidgetHierarchy;
	TSharedPtr<SBindingContextEntry> SelectedSourceWidget;

	bool bAutoRefresh = false;
	bool bViewModels = false;
	bool bShowClear = true;
};

} // namespace UE::MVVM
