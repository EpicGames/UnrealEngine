// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMViewBindingListView.h"

#include "Blueprint/WidgetTree.h"
#include "BlueprintEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"

#include "Framework/MVVMRowHelper.h"
#include "MVVMBlueprintView.h"
#include "MVVMBlueprintViewBinding.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMBlueprintViewEvent.h"
#include "MVVMBlueprintViewCondition.h"
#include "MVVMEditorSubsystem.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "Types/MVVMBindingEntry.h"

#include "Widgets/BindingEntry/SMVVMBindingRow.h"
#include "Widgets/BindingEntry/SMVVMEventRow.h"
#include "Widgets/BindingEntry/SMVVMConditionRow.h"
#include "Widgets/BindingEntry/SMVVMFunctionParameterRow.h"
#include "Widgets/BindingEntry/SMVVMGroupRow.h"
#include "Widgets/SMVVMViewBindingPanel.h"
#include "Widgets/SMVVMViewModelPanel.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "BindingListView"

namespace UE::MVVM
{

namespace Private
{
	void ExpandAll(const TSharedPtr<STreeView<TSharedPtr<FBindingEntry>>>& TreeView, const TSharedPtr<FBindingEntry>& Entry)
	{
		TreeView->SetItemExpansion(Entry, true);

		for (const TSharedPtr<FBindingEntry>& Child : Entry->GetFilteredChildren())
		{
			ExpandAll(TreeView, Child);
		}
	}

	TSharedPtr<FBindingEntry> FindBinding(FGuid BindingId, TConstArrayView<TSharedPtr<FBindingEntry>> Entries)
	{
		for (const TSharedPtr<FBindingEntry>& Entry : Entries)
		{
			if (Entry->GetRowType() == FBindingEntry::ERowType::Binding && Entry->GetBindingId() == BindingId)
			{
				return Entry;
			}
			TSharedPtr<FBindingEntry> Result = FindBinding(BindingId, Entry->GetAllChildren());
			if (Result)
			{
				return Result;
			}
		}
		return TSharedPtr<FBindingEntry>();
	}
	
	TSharedPtr<FBindingEntry> FindEvent(const UMVVMBlueprintViewEvent* Event, TConstArrayView<TSharedPtr<FBindingEntry>> Entries)
	{
		for (const TSharedPtr<FBindingEntry>& Entry : Entries)
		{
			if (Entry->GetRowType() == FBindingEntry::ERowType::Event && Entry->GetEvent() == Event)
			{
				return Entry;
			}
			TSharedPtr<FBindingEntry> Result = FindEvent(Event, Entry->GetAllChildren());
			if (Result)
			{
				return Result;
			}
		}
		return TSharedPtr<FBindingEntry>();
	}

	TSharedPtr<FBindingEntry> FindCondition(const UMVVMBlueprintViewCondition* Condition, TConstArrayView<TSharedPtr<FBindingEntry>> Entries)
	{
		for (const TSharedPtr<FBindingEntry>& Entry : Entries)
		{
			if (Entry->GetRowType() == FBindingEntry::ERowType::Condition && Entry->GetCondition() == Condition)
			{
				return Entry;
			}
			TSharedPtr<FBindingEntry> Result = FindCondition(Condition, Entry->GetAllChildren());
			if (Result)
			{
				return Result;
			}
		}
		return TSharedPtr<FBindingEntry>();
	}

	TSharedPtr<FBindingEntry> FindParameter(FMVVMBlueprintPinId PinId, TConstArrayView<TSharedPtr<FBindingEntry>> Entries)
	{
		for (const TSharedPtr<FBindingEntry>& Entry : Entries)
		{
			if (Entry->GetRowType() == FBindingEntry::ERowType::ConditionParameter)
			{
				if (Entry->GetConditionParameterId() == PinId)
				{
					return Entry;
				}
			}
			else if (Entry->GetRowType() == FBindingEntry::ERowType::EventParameter)
			{
				if (Entry->GetEventParameterId() == PinId)
				{
					return Entry;
				}
			}

			TSharedPtr<FBindingEntry> Result = FindParameter(PinId, Entry->GetAllChildren());
			if (Result)
			{
				return Result;
			}
		}
		return TSharedPtr<FBindingEntry>();
	}

	void FilterEntryList(FString FilterString, TOptional<SBindingsList::EFilterMode> FilterMode, const TArray<TSharedPtr<FBindingEntry>>& RootGroups, TArray<TSharedPtr<FBindingEntry>>& FilteredRootGroups, UMVVMBlueprintView* BlueprintView, UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr)
	{
		TArray<TSharedPtr<FBindingEntry>> FirstFilterRootGroups;

		// First filtering pass: filter based on the filter mode.
		if (FilterMode.IsSet() && FilterMode.GetValue() != SBindingsList::EFilterMode::All)
		{
			for (const TSharedPtr<FBindingEntry>& GroupEntry : RootGroups)
			{
				bool bNeedGroupEntry = false;
				for (TSharedPtr<FBindingEntry> Entry : GroupEntry->GetAllChildren())
				{
					FBindingEntry* EntryPtr = Entry.Get();

					const bool bEntryHasLink = EntryPtr->GetHasPathToVerseVariable() || EntryPtr->GetHasPathToViewModel();
					const bool bEntryHasMatchingLink = (FilterMode.GetValue() == SBindingsList::EFilterMode::Viewmodel && EntryPtr->GetHasPathToViewModel()) ||
													   (FilterMode.GetValue() == SBindingsList::EFilterMode::Verse && EntryPtr->GetHasPathToVerseVariable());

					bool bChildHasLink = false;
					bool bChildHasMatchingLink = false;
					for (TSharedPtr<FBindingEntry> ChildEntry : EntryPtr->GetAllChildren())
					{
						bChildHasLink |= ChildEntry->GetHasPathToVerseVariable() || ChildEntry->GetHasPathToViewModel();

						bChildHasMatchingLink |= (FilterMode.GetValue() == SBindingsList::EFilterMode::Viewmodel && ChildEntry->GetHasPathToViewModel()) ||
												 (FilterMode.GetValue() == SBindingsList::EFilterMode::Verse && ChildEntry->GetHasPathToVerseVariable());
					}

					// Keep entries that are not linked to any field, or if they are linked, keep any that have at least one link we want.
					// An entry that is not tied to a viewmodel or a verse field will always be kept, and an entry that is linked to both will also always be kept, so they will be displayed whatever the filter mode is.
					const bool bHasLink = bEntryHasLink || bChildHasLink;
					if (!bHasLink || bEntryHasMatchingLink || bChildHasMatchingLink)
					{
						GroupEntry->AddFilteredChild(Entry);
						bNeedGroupEntry = true;
					}
					else
					{
						GroupEntry->SetUseFilteredChildList();
					}
				}
				if (bNeedGroupEntry)
				{
					FirstFilterRootGroups.Add(GroupEntry);
				}
			}
		}
		else
		{
			FirstFilterRootGroups = RootGroups;
		}

		// Second filtering pass: filter based on the filter string.
		if (!FilterString.TrimStartAndEnd().IsEmpty())
		{
			TArray<FString> SearchKeywords;
			FilterString.ParseIntoArray(SearchKeywords, TEXT(" "));

			struct FIsAllKeywordsInString
			{
				bool operator()(FString EntryString, TArray<FString>& SearchKeywords)
				{
					for (const FString& Keyword : SearchKeywords)
					{
						if (!EntryString.Contains(Keyword))
						{
							return false;
						}
					}
					return true;
				}
			} IsAllKeywordsInString;

			struct FAddFilteredEntry
			{
				FAddFilteredEntry(FIsAllKeywordsInString& InIsAllKeywordsInString, TArray<FString>& InSearchKeywords, UMVVMBlueprintView* InBlueprintView, UWidgetBlueprint* InWidgetBlueprint)
					: IsAllKeywordsInString(InIsAllKeywordsInString)
					, SearchKeywords(InSearchKeywords)
					, WidgetBlueprint(InWidgetBlueprint)
					, BlueprintView(InBlueprintView)
				{}
				bool operator()(TSharedPtr<FBindingEntry> ParentEntry)
				{
					bool bMatchFound = false;

					//Work on a copy of the array, as following code will modify it and we want a second pass of filtering, that will remove children, not add ones.
 					TConstArrayView<TSharedPtr<FBindingEntry>> FilteredChildren = ParentEntry->GetFilteredChildren();
 					ParentEntry->ResetFilteredChildren();

 					for (TSharedPtr<FBindingEntry> Entry : FilteredChildren)
					{
						const FString EntryString = Entry->GetSearchNameString(BlueprintView, WidgetBlueprint);
						const bool bEntryMatchFound = IsAllKeywordsInString(EntryString, SearchKeywords) || (*this)(Entry);
						if (bEntryMatchFound)
						{
							ParentEntry->AddFilteredChild(Entry);
							bMatchFound = true;
						}
					}
					ParentEntry->SetUseFilteredChildList();
					return bMatchFound;
				}
				FIsAllKeywordsInString& IsAllKeywordsInString;
				TArray<FString>& SearchKeywords;
				UWidgetBlueprint* WidgetBlueprint = nullptr;
				UMVVMBlueprintView* BlueprintView = nullptr;
			};
			FAddFilteredEntry AddFilteredEntry = FAddFilteredEntry(IsAllKeywordsInString, SearchKeywords, BlueprintView, MVVMExtensionPtr->GetWidgetBlueprint());

			for (const TSharedPtr<FBindingEntry>& GroupEntry : FirstFilterRootGroups)
			{
				FString EntryString = GroupEntry->GetSearchNameString(BlueprintView, MVVMExtensionPtr->GetWidgetBlueprint());

				// If the filter text is found in the group name, we keep the entire group.
				if (IsAllKeywordsInString(EntryString, SearchKeywords))
				{
					FilteredRootGroups.Add(GroupEntry);
				}
				else
				{
					if (AddFilteredEntry(GroupEntry))
					{
						FilteredRootGroups.Add(GroupEntry);
					}
				}
			}
		}
		else
		{
			FilteredRootGroups = FirstFilterRootGroups;
		}
	}

	void SetBindingEntrySelection(const TSharedPtr<STreeView<TSharedPtr<FBindingEntry>>>& TreeView, TConstArrayView<TSharedPtr<FBindingEntry>> Entries, TConstArrayView<const TSharedPtr<FBindingEntry>> EntriesToSelect)
	{
		if (!TreeView || Entries.IsEmpty() || EntriesToSelect.IsEmpty())
		{
			return;
		}

		for (const TSharedPtr<FBindingEntry>& Entry : Entries)
		{
			TFunction<bool(TSharedPtr<FBindingEntry>)> EquivalenceFunc = [Entry](TSharedPtr<FBindingEntry> Other) { return *Entry == *Other; };;
			if (EntriesToSelect.ContainsByPredicate(EquivalenceFunc))
			{
				TreeView->SetItemSelection(Entry, true);
				TreeView->RequestScrollIntoView(Entry);
			}

			SetBindingEntrySelection(TreeView, Entry->GetAllChildren(), EntriesToSelect);
		}
	}

} // namespace


void SBindingsList::Construct(const FArguments& InArgs, TSharedPtr<SBindingsPanel> Owner, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor, UMVVMWidgetBlueprintExtension_View* InMVVMExtension)
{
	BindingPanel = Owner;
	MVVMExtension = InMVVMExtension;
	WeakBlueprintEditor = InBlueprintEditor;
	check(InMVVMExtension);
	check(InMVVMExtension->GetBlueprintView());

	MVVMExtension->OnBlueprintViewChangedDelegate().AddSP(this, &SBindingsList::Refresh);
	MVVMExtension->GetBlueprintView()->OnBindingsUpdated.AddSP(this, &SBindingsList::Refresh);
	MVVMExtension->GetBlueprintView()->OnEventsUpdated.AddSP(this, &SBindingsList::Refresh);
	MVVMExtension->GetBlueprintView()->OnConditionsUpdated.AddSP(this, &SBindingsList::Refresh);
	MVVMExtension->GetBlueprintView()->OnEventParametersRegenerate.AddSP(this, &SBindingsList::EventParametersRegenerate);
	MVVMExtension->GetBlueprintView()->OnConditionParametersRegenerate.AddSP(this, &SBindingsList::ConditionParametersRegenerate);
	MVVMExtension->GetBlueprintView()->OnBindingsAdded.AddSP(this, &SBindingsList::ClearFilterText);
	MVVMExtension->GetBlueprintView()->OnViewModelsUpdated.AddSP(this, &SBindingsList::ForceRefresh);

	ChildSlot
	[
		SAssignNew(TreeView, STreeView<TSharedPtr<FBindingEntry>>)
		.TreeItemsSource(&FilteredRootGroups)
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SBindingsList::GenerateEntryRow)
		.OnGetChildren(this, &SBindingsList::GetChildrenOfEntry)
		.OnContextMenuOpening(this, &SBindingsList::OnSourceConstructContextMenu)
		.OnSelectionChanged(this, &SBindingsList::OnSourceListSelectionChanged)
	];

	Refresh();
}

SBindingsList::~SBindingsList()
{
	if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get())
	{
		MVVMExtensionPtr->OnBlueprintViewChangedDelegate().RemoveAll(this);
		MVVMExtensionPtr->GetBlueprintView()->OnBindingsUpdated.RemoveAll(this);
		MVVMExtensionPtr->GetBlueprintView()->OnEventsUpdated.RemoveAll(this);
		MVVMExtensionPtr->GetBlueprintView()->OnConditionsUpdated.RemoveAll(this);
		MVVMExtensionPtr->GetBlueprintView()->OnEventParametersRegenerate.RemoveAll(this);
		MVVMExtensionPtr->GetBlueprintView()->OnConditionParametersRegenerate.RemoveAll(this);
		MVVMExtensionPtr->GetBlueprintView()->OnBindingsAdded.RemoveAll(this);
		MVVMExtensionPtr->GetBlueprintView()->OnViewModelsUpdated.RemoveAll(this);
	}
}

void SBindingsList::GetChildrenOfEntry(TSharedPtr<FBindingEntry> Entry, TArray<TSharedPtr<FBindingEntry>>& OutChildren) const
{
	OutChildren.Append(Entry->GetFilteredChildren());
}

template<typename TEntryValueType>
void SBindingsList::RegisterWrapperGraphModified(TEntryValueType* EntryValue, TSharedPtr<FBindingEntry> BindingEntry)
{
	{
		const FObjectKey ObjectKey = FObjectKey(EntryValue);
		TPair<TWeakPtr<FBindingEntry>, FDelegateHandle>* FoundWrapperGraphModifiedPtr = WrapperGraphModifiedDelegates.Find(ObjectKey);
		if (FoundWrapperGraphModifiedPtr)
		{
			TSharedPtr<FBindingEntry> FoundWrapperGraphModified = FoundWrapperGraphModifiedPtr->Get<0>().Pin();
			if (FoundWrapperGraphModified != BindingEntry)
			{
				EntryValue->OnWrapperGraphModified.Remove(FoundWrapperGraphModifiedPtr->Get<1>());
				WrapperGraphModifiedDelegates.Remove(ObjectKey);
				FoundWrapperGraphModifiedPtr = nullptr;
			}
		}
		if (FoundWrapperGraphModifiedPtr == nullptr)
		{
			FDelegateHandle DelegateHandle = EntryValue->OnWrapperGraphModified.AddSP(this, &SBindingsList::HandleRefreshChildren, ObjectKey);
			WrapperGraphModifiedDelegates.Add(ObjectKey, { TWeakPtr<FBindingEntry>(BindingEntry), DelegateHandle });
		}
	}
}

void SBindingsList::EventParametersRegenerate(UMVVMBlueprintViewEvent* Event)
{
	if (TSharedPtr<FBindingEntry> EventEntry = Private::FindEvent(Event, AllRootGroups))
	{
		EventEntry->ResetChildren();
		Refresh();
	}
}

void SBindingsList::ConditionParametersRegenerate(UMVVMBlueprintViewCondition* Condition)
{
	if (TSharedPtr<FBindingEntry> ConditionEntry = Private::FindCondition(Condition, AllRootGroups))
	{
		ConditionEntry->ResetChildren();
		Refresh();
	}
}

void SBindingsList::Refresh()
{
	struct FPreviousGroup
	{
		TSharedPtr<FBindingEntry> Group;
		TArray<TSharedPtr<FBindingEntry>> Children;
	};

	TArray<FPreviousGroup> PreviousRootGroups;
	for (const TSharedPtr<FBindingEntry>& PreviousEntry : AllRootGroups)
	{
		ensure(PreviousEntry->GetRowType() == FBindingEntry::ERowType::Group);
		FPreviousGroup& NewItem = PreviousRootGroups.AddDefaulted_GetRef();
		NewItem.Group = PreviousEntry;

		struct FRecursiveAdd
		{
			void operator()(FPreviousGroup& NewItem, const TSharedPtr<FBindingEntry>& Entry)
			{
				for (TSharedPtr<FBindingEntry> PreviousChildEntry : Entry->GetAllChildren())
				{
					NewItem.Children.Add(PreviousChildEntry);
					(*this)(NewItem, PreviousChildEntry);
				}
				Entry->ResetChildren();
			}
		};
		FRecursiveAdd{}(NewItem, PreviousEntry);
	}

	AllRootGroups.Reset();
	FilteredRootGroups.Reset();

	TArray<TSharedPtr<FBindingEntry>> NewEntries;

	UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get();
	UMVVMBlueprintView* BlueprintView = MVVMExtensionPtr ? MVVMExtensionPtr->GetBlueprintView() : nullptr;
	UWidgetBlueprint* WidgetBlueprint = MVVMExtensionPtr ? MVVMExtensionPtr->GetWidgetBlueprint() : nullptr;

	// generate our entries
	// for each widget with bindings, create an entry at the root level
	// then add all bindings that reference that widget as its children
	if (BlueprintView)
	{
		auto FindPreviousGroupEntry = [&PreviousRootGroups, Self = this](FName GroupName)
		{
			return PreviousRootGroups.FindByPredicate([GroupName](const FPreviousGroup& Other) { return Other.Group->GetGroupName() == GroupName; });
		};
		auto FindGroupEntry = [&NewEntries, Self = this](FPreviousGroup* PreviousGroupEntry, FName GroupName, FGuid ViewModelId)
		{
			TSharedPtr<FBindingEntry> GroupEntry;
			if (PreviousGroupEntry)
			{
				GroupEntry = PreviousGroupEntry->Group;
			}
			else if (TSharedPtr<FBindingEntry>* FoundGroup = NewEntries.FindByPredicate([GroupName](const TSharedPtr<FBindingEntry>& Other)
				{ return Other->GetGroupName() == GroupName && Other->GetRowType() == FBindingEntry::ERowType::Group; }))
			{
				GroupEntry = *FoundGroup;
			}

			if (!GroupEntry.IsValid())
			{
				GroupEntry = MakeShared<FBindingEntry>();
				GroupEntry->SetGroup(GroupName, ViewModelId);

				NewEntries.Add(GroupEntry);
			}
			Self->AllRootGroups.AddUnique(GroupEntry);
			return GroupEntry;
		};

		auto IsVerseFieldVariable = [&WidgetBlueprint](const FMVVMBlueprintPropertyPath& Path) -> bool
		{
			if (Path.GetSource(WidgetBlueprint) != EMVVMBlueprintFieldPathSource::SelfContext)
			{
				return false;
			}

			TArray<UE::MVVM::FMVVMConstFieldVariant> Fields = Path.GetCompleteFields(WidgetBlueprint);
			if (Fields.Num() != 1 || !Fields[0].IsProperty())
			{
				return false;
			}

			const FBPVariableDescription* FoundVariable = WidgetBlueprint->NewVariables.FindByPredicate(
				[Field = Fields[0]](const FBPVariableDescription& NewVariable)
				{
					return NewVariable.VarName == Field.GetName();
				}
			);
			if (!FoundVariable)
			{
				return false;
			}

			return FoundVariable->HasMetaData("VerseVariable");
		};

		auto UpdateEntryPathInfo = [&WidgetBlueprint, IsVerseFieldVariable](TSharedPtr<FBindingEntry> BindingEntry, const FMVVMBlueprintPropertyPath& Path)
		{
			BindingEntry->SetHasPathToVerseVariable(IsVerseFieldVariable(Path));
			BindingEntry->SetHasPathToViewModel(Path.GetSource(WidgetBlueprint) == EMVVMBlueprintFieldPathSource::ViewModel);
		};

		auto UpdateEntryPathsInfo = [&WidgetBlueprint, IsVerseFieldVariable](TSharedPtr<FBindingEntry>  BindingEntry, const FMVVMBlueprintPropertyPath& Path1, const FMVVMBlueprintPropertyPath& Path2)
		{
			BindingEntry->SetHasPathToVerseVariable(IsVerseFieldVariable(Path1) || IsVerseFieldVariable(Path2));
			BindingEntry->SetHasPathToViewModel(Path1.GetSource(WidgetBlueprint) == EMVVMBlueprintFieldPathSource::ViewModel || Path2.GetSource(WidgetBlueprint) == EMVVMBlueprintFieldPathSource::ViewModel);
		};
			
		for (const FMVVMBlueprintViewBinding& Binding : BlueprintView->GetBindings())
		{
			// Make sure the graph for the bindings is generated
			if (Binding.Conversion.SourceToDestinationConversion)
			{
				Binding.Conversion.SourceToDestinationConversion->GetOrCreateWrapperGraph(WidgetBlueprint);
			}
			if (Binding.Conversion.DestinationToSourceConversion)
			{
				Binding.Conversion.DestinationToSourceConversion->GetOrCreateWrapperGraph(WidgetBlueprint);
			}
			
			FName GroupName;
			FGuid GroupViewModelId;
			switch (Binding.DestinationPath.GetSource(WidgetBlueprint))
			{
			case EMVVMBlueprintFieldPathSource::SelfContext:
				GroupName = WidgetBlueprint->GetFName();
				break;
			case EMVVMBlueprintFieldPathSource::Widget:
				GroupName = Binding.DestinationPath.GetWidgetName();
				break;
			case EMVVMBlueprintFieldPathSource::ViewModel:
				if (const FMVVMBlueprintViewModelContext* ViewModelContext = BlueprintView->FindViewModel(Binding.DestinationPath.GetViewModelId()))
				{
					GroupName = ViewModelContext->GetViewModelName();
					GroupViewModelId = ViewModelContext->GetViewModelId();
				}
				break;
			}

			// Find the group entry
			FPreviousGroup* PreviousGroupEntry = FindPreviousGroupEntry(GroupName);
			TSharedPtr<FBindingEntry> GroupEntry = FindGroupEntry(PreviousGroupEntry, GroupName, GroupViewModelId);

			// Create/Find the child entry
			TSharedPtr<FBindingEntry> BindingEntry;
			FGuid BindingId = Binding.BindingId;
			{
				if (PreviousGroupEntry)
				{
					if (TSharedPtr<FBindingEntry>* FoundBinding = PreviousGroupEntry->Children.FindByPredicate([BindingId](const TSharedPtr<FBindingEntry>& Other)
						{ return Other->GetBindingId() == BindingId && Other->GetRowType() == FBindingEntry::ERowType::Binding; }))
					{
						BindingEntry = *FoundBinding;
					}
				}

				if (!BindingEntry.IsValid())
				{
					BindingEntry = MakeShared<FBindingEntry>();
					BindingEntry->SetBindingId(BindingId);

					NewEntries.Add(BindingEntry);
				}

				UpdateEntryPathsInfo(BindingEntry, Binding.DestinationPath, Binding.SourcePath);
				GroupEntry->AddChild(BindingEntry);
			}

			// Create/Find entries for conversion function parameters
			if (UMVVMBlueprintViewConversionFunction* ConversionFunction = Binding.Conversion.GetConversionFunction(UE::MVVM::IsForwardBinding(Binding.BindingType)))
			{
				// Register to any modifications made in the graph
				RegisterWrapperGraphModified(ConversionFunction, BindingEntry);

				// Make sure the graph is up to date
				ConversionFunction->GetOrCreateWrapperGraph(MVVMExtensionPtr->GetWidgetBlueprint());

				for (const FMVVMBlueprintPin& Pin : ConversionFunction->GetPins())
				{
					UEdGraphPin* GraphPin = ConversionFunction->GetOrCreateGraphPin(MVVMExtensionPtr->GetWidgetBlueprint(), Pin.GetId());
					if (GraphPin && GraphPin->bHidden)
					{
						continue;
					}

					FEdGraphPinType PinType = GraphPin ? GraphPin->PinType : FEdGraphPinType();
					TSharedPtr<FBindingEntry> ArgumentEntry;
					if (PreviousGroupEntry)
					{
						TSharedPtr<FBindingEntry>* FoundParameter = PreviousGroupEntry->Children.FindByPredicate(
							[BindingId, &PinType, ArgumentId = Pin.GetId()](const TSharedPtr<FBindingEntry>& Other)
							{
								return Other->GetBindingId() == BindingId && Other->GetRowType() == FBindingEntry::ERowType::BindingParameter && Other->GetBindingParameterId() == ArgumentId && Other->GetBindingParameterType() == PinType;
							});
						if (FoundParameter)
						{
							ArgumentEntry = *FoundParameter;
						}
					}

					if (!ArgumentEntry.IsValid())
					{
						ArgumentEntry = MakeShared<FBindingEntry>();
						ArgumentEntry->SetBindingParameter(Binding.BindingId, Pin.GetId(), MoveTemp(PinType));

						NewEntries.Add(ArgumentEntry);
					}

					UpdateEntryPathInfo(ArgumentEntry, Pin.GetPath());
					BindingEntry->AddChild(ArgumentEntry);
				}
			}
		}

		for (UMVVMBlueprintViewEvent* Event : BlueprintView->GetEvents())
		{
			if (Event == nullptr)
			{
				continue;
			}

			// Make sure the graph is up to date
			Event->GetOrCreateWrapperGraph();

			FName GroupName;
			FGuid GroupViewModelId;
			switch (Event->GetEventPath().GetSource(WidgetBlueprint))
			{
			case EMVVMBlueprintFieldPathSource::SelfContext:
				GroupName = WidgetBlueprint->GetFName();
				break;
			case EMVVMBlueprintFieldPathSource::Widget:
				GroupName = Event->GetEventPath().GetWidgetName();
				break;
			case EMVVMBlueprintFieldPathSource::ViewModel:
				if (const FMVVMBlueprintViewModelContext* ViewModelContext = BlueprintView->FindViewModel(Event->GetEventPath().GetViewModelId()))
				{
					GroupName = ViewModelContext->GetViewModelName();
					GroupViewModelId = ViewModelContext->GetViewModelId();
				}
				break;
			}

			// Find the group entry
			FPreviousGroup* PreviousGroupEntry = FindPreviousGroupEntry(GroupName);
			TSharedPtr<FBindingEntry> GroupEntry = FindGroupEntry(PreviousGroupEntry, GroupName, GroupViewModelId);

			// Create/Find the child entry
			TSharedPtr<FBindingEntry> EventEntry;
			{
				if (PreviousGroupEntry)
				{
					if (TSharedPtr<FBindingEntry>* FoundBinding = PreviousGroupEntry->Children.FindByPredicate([Event](const TSharedPtr<FBindingEntry>& Other)
						{
							return Other->GetRowType() == FBindingEntry::ERowType::Event && Other->GetEvent() == Event;
						}))
					{
						EventEntry = *FoundBinding;
					}
				}

				if (!EventEntry.IsValid())
				{
					EventEntry = MakeShared<FBindingEntry>();
					EventEntry->SetEvent(Event);

					NewEntries.Add(EventEntry);
				}

				UpdateEntryPathsInfo(EventEntry, Event->GetEventPath(), Event->GetDestinationPath());
				GroupEntry->AddChild(EventEntry);
			}

			// Register to any modifications made by the graph
			RegisterWrapperGraphModified(Event, EventEntry);

			// Create/Find entries for function parameters
			for (const FMVVMBlueprintPin& Pin : Event->GetPins())
			{
				UEdGraphPin* GraphPin = Event->GetOrCreateGraphPin(Pin.GetId());
				if (GraphPin && GraphPin->bHidden)
				{
					continue;
				}

				FEdGraphPinType PinType = GraphPin ? GraphPin->PinType : FEdGraphPinType();
				TSharedPtr<FBindingEntry> ArgumentEntry;
				if (PreviousGroupEntry)
				{
					TSharedPtr<FBindingEntry>* FoundParameter = PreviousGroupEntry->Children.FindByPredicate(
						[Event, &PinType, ArgumentId = Pin.GetId()](const TSharedPtr<FBindingEntry>& Other)
						{
							return Other->GetRowType() == FBindingEntry::ERowType::EventParameter && Other->GetEvent() == Event && Other->GetEventParameterId() == ArgumentId && Other->GetEventParameterType() == PinType;
						});
					if (FoundParameter)
					{
						ArgumentEntry = *FoundParameter;
					}
				}

				if (!ArgumentEntry.IsValid())
				{
					ArgumentEntry = MakeShared<FBindingEntry>();
					ArgumentEntry->SetEventParameter(Event, Pin.GetId(), MoveTemp(PinType));

					NewEntries.Add(ArgumentEntry);
				}

				UpdateEntryPathInfo(ArgumentEntry, Pin.GetPath());
				EventEntry->AddChild(ArgumentEntry);
			}
		}

		FName GroupName = WidgetBlueprint->GetFName();
		FGuid GroupViewModelId;
		for (UMVVMBlueprintViewCondition* Condition : BlueprintView->GetConditions())
		{
			if (Condition == nullptr)
			{
				continue;
			}

			// Make sure the graph is up to date
			Condition->GetOrCreateWrapperGraph();

			// Find the group entry
			FPreviousGroup* PreviousGroupEntry = FindPreviousGroupEntry(GroupName);
			TSharedPtr<FBindingEntry> GroupEntry = FindGroupEntry(PreviousGroupEntry, GroupName, GroupViewModelId);

			// Create/Find the child entry
			TSharedPtr<FBindingEntry> ConditionEntry;
			{
				if (PreviousGroupEntry)
				{
					if (TSharedPtr<FBindingEntry>* FoundBinding = PreviousGroupEntry->Children.FindByPredicate([Condition](const TSharedPtr<FBindingEntry>& Other)
						{
							return Other->GetRowType() == FBindingEntry::ERowType::Condition && Other->GetCondition() == Condition;
						}))
					{
						ConditionEntry = *FoundBinding;
					}
				}

				if (!ConditionEntry.IsValid())
				{
					ConditionEntry = MakeShared<FBindingEntry>();
					ConditionEntry->SetCondition(Condition);

					NewEntries.Add(ConditionEntry);
				}

				UpdateEntryPathsInfo(ConditionEntry, Condition->GetConditionPath(), Condition->GetDestinationPath());
				GroupEntry->AddChild(ConditionEntry);
			}

			// Register to any modifications made by the graph
			RegisterWrapperGraphModified(Condition, ConditionEntry);

			// Create/Find entries for function parameters
			for (const FMVVMBlueprintPin& Pin : Condition->GetPins())
			{
				UEdGraphPin* GraphPin = Condition->GetOrCreateGraphPin(Pin.GetId());
				if (GraphPin && GraphPin->bHidden)
				{
					continue;
				}

				FEdGraphPinType PinType = GraphPin ? GraphPin->PinType : FEdGraphPinType();
				TSharedPtr<FBindingEntry> ArgumentEntry;
				if (PreviousGroupEntry)
				{
					TSharedPtr<FBindingEntry>* FoundParameter = PreviousGroupEntry->Children.FindByPredicate(
						[Condition, &PinType, ArgumentId = Pin.GetId()](const TSharedPtr<FBindingEntry>& Other)
						{
							return Other->GetRowType() == FBindingEntry::ERowType::ConditionParameter && Other->GetCondition() == Condition && Other->GetConditionParameterId() == ArgumentId && Other->GetConditionParameterType() == PinType;
						});
					if (FoundParameter)
					{
						ArgumentEntry = *FoundParameter;
					}
				}

				if (!ArgumentEntry.IsValid())
				{
					ArgumentEntry = MakeShared<FBindingEntry>();
					ArgumentEntry->SetConditionParameter(Condition, Pin.GetId(), MoveTemp(PinType));

					NewEntries.Add(ArgumentEntry);
				}

				UpdateEntryPathInfo(ArgumentEntry, Pin.GetPath());
				ConditionEntry->AddChild(ArgumentEntry);
			}
		}

		TOptional<SBindingsList::EFilterMode> FilterModeToUse;
		if (TSharedPtr<SBindingsPanel> BindingPanelPtr = BindingPanel.Pin())
		{
			if (BindingPanelPtr->IsFilterModeEnabled())
			{
				FilterModeToUse = FilterMode;
			}
		}
		Private::FilterEntryList(FilterText.ToString(), FilterModeToUse, AllRootGroups, FilteredRootGroups, BlueprintView, MVVMExtensionPtr);
	}

	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
		for (const TSharedPtr<FBindingEntry>& Entry : NewEntries)
		{
			Private::ExpandAll(TreeView, Entry);
		}
	}
}

void SBindingsList::ForceRefresh()
{
	AllRootGroups.Reset();
	FilteredRootGroups.Reset();
	Refresh();
}

void SBindingsList::HandleRefreshChildren(FObjectKey ObjectHolder)
{
	TPair<TWeakPtr<FBindingEntry>, FDelegateHandle>* Found = WrapperGraphModifiedDelegates.Find(ObjectHolder);
	if (Found)
	{
		TSharedPtr<FBindingEntry> FoundWrapperGraphModified = Found->Get<0>().Pin();
		if (FoundWrapperGraphModified)
		{
			FoundWrapperGraphModified->ResetChildren();
			Refresh();
		}
	}
}

TSharedRef<ITableRow> SBindingsList::GenerateEntryRow(TSharedPtr<FBindingEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<ITableRow> Row;

	if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get())
	{
		switch (Entry->GetRowType())
		{
			case FBindingEntry::ERowType::Group:
			{
				Row = SNew(UE::MVVM::BindingEntry::SGroupRow, this, OwnerTable, WeakBlueprintEditor.Pin(), MVVMExtensionPtr->GetWidgetBlueprint(), Entry);
				break;
			}
			case FBindingEntry::ERowType::Binding:
			{
				Row = SNew(UE::MVVM::BindingEntry::SBindingRow, this, OwnerTable, WeakBlueprintEditor.Pin(), MVVMExtensionPtr->GetWidgetBlueprint(), Entry);
				break;
			}
			case FBindingEntry::ERowType::BindingParameter:
			case FBindingEntry::ERowType::EventParameter:
			case FBindingEntry::ERowType::ConditionParameter:
			{
				if (!Entry->GetBindingParameterId().IsValid())
				{
					ensureMsgf(false, TEXT("Corrupted Binding Parameter."));
					return SNew(STableRow<TSharedPtr<FBindingEntry>>, OwnerTable);
				}
				Row = SNew(UE::MVVM::BindingEntry::SFunctionParameterRow, this, OwnerTable, WeakBlueprintEditor.Pin(), MVVMExtensionPtr->GetWidgetBlueprint(), Entry);
				break;
			}
			case FBindingEntry::ERowType::Event:
			{
				Row = SNew(UE::MVVM::BindingEntry::SEventRow, this, OwnerTable, WeakBlueprintEditor.Pin(), MVVMExtensionPtr->GetWidgetBlueprint(), Entry);
				break;
			}
			case FBindingEntry::ERowType::Condition:
			{
				Row = SNew(UE::MVVM::BindingEntry::SConditionRow, this, OwnerTable, WeakBlueprintEditor.Pin(), MVVMExtensionPtr->GetWidgetBlueprint(), Entry);
				break;
			}
		}
	
		//If this row was not spawned from the blueprint editor. Disable modifying, visual only
		Row->GetContent()->SetEnabled(WeakBlueprintEditor.IsValid());

		return Row.ToSharedRef();
	}

	ensureMsgf(false, TEXT("Failed to create binding or widget row."));
	return SNew(STableRow<TSharedPtr<FBindingEntry>>, OwnerTable);
}

SBindingsList::EFilterMode SBindingsList::GetActiveFilterMode() const
{
	return FilterMode;
}

void SBindingsList::OnActiveFilterModeChanged(EFilterMode InFilterMode)
{
	FilterMode = InFilterMode;
	Refresh();
}

void SBindingsList::OnFilterTextChanged(const FText& InFilterText)
{
	FilterText = InFilterText;
	Refresh();
}

void SBindingsList::ClearFilterText()
{
	FilterText = FText::GetEmpty();
}

TSharedPtr<SWidget> SBindingsList::OnSourceConstructContextMenu()
{
	UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get();
	UWidgetBlueprint* WidgetBlueprint = MVVMExtensionPtr ? MVVMExtensionPtr->GetWidgetBlueprint() : nullptr;
	UMVVMBlueprintView* View = MVVMExtensionPtr ? MVVMExtensionPtr->GetBlueprintView() : nullptr;
	BindingEntry::FOnContextMenuEntryCallback SelectionCallback = BindingEntry::FOnContextMenuEntryCallback::CreateSP(this, &SBindingsList::SetSelection);

	return BindingEntry::FRowHelper::CreateContextMenu(WidgetBlueprint, View, TreeView->GetSelectedItems(), SelectionCallback).MakeWidget();
}

void SBindingsList::RequestNavigateToBinding(FGuid BindingId)
{
	TSharedPtr<FBindingEntry> Entry = Private::FindBinding(BindingId, FilteredRootGroups);
	if (Entry && TreeView)
	{
		TreeView->RequestNavigateToItem(Entry);
	}
}

void SBindingsList::RequestNavigateToEvent(UMVVMBlueprintViewEvent* Event)
{
	TSharedPtr<FBindingEntry> Entry = Private::FindEvent(Event, FilteredRootGroups);
	if (Entry && TreeView)
	{
		TreeView->RequestNavigateToItem(Entry);
	}
}

void SBindingsList::RequestNavigateToCondition(UMVVMBlueprintViewCondition* Condition)
{
	TSharedPtr<FBindingEntry> Entry = Private::FindCondition(Condition, FilteredRootGroups);
	if (Entry && TreeView)
	{
		TreeView->RequestNavigateToItem(Entry);
	}
}

void SBindingsList::SelectBinding(FGuid BindingId, bool bShouldScrollIntoView)
{	
	if (TSharedPtr<FBindingEntry> Entry = Private::FindBinding(BindingId, FilteredRootGroups))
	{
		SetSelection(MakeConstArrayView({Entry}));
		if (bShouldScrollIntoView && TreeView)
		{
			TreeView->RequestNavigateToItem(Entry);
		}
	}
	else
	{
		SetSelection(MakeConstArrayView<TSharedPtr<FBindingEntry>>({}));
	}
}

void SBindingsList::SelectEvent(const UMVVMBlueprintViewEvent* Event, bool bShouldScrollIntoView)
{	
	if (TSharedPtr<FBindingEntry> Entry = Private::FindEvent(Event, FilteredRootGroups))
	{
		SetSelection(MakeConstArrayView({Entry}));
		if (bShouldScrollIntoView && TreeView)
		{
			TreeView->RequestNavigateToItem(Entry);
		}
	}
	else
	{
		SetSelection(MakeConstArrayView<TSharedPtr<FBindingEntry>>({}));
	}
}

void SBindingsList::SelectCondition(const UMVVMBlueprintViewCondition* Condition, bool bShouldScrollIntoView)
{	
	if (TSharedPtr<FBindingEntry> Entry = Private::FindCondition(Condition, FilteredRootGroups))
	{
		SetSelection(MakeConstArrayView({Entry}));
		if (bShouldScrollIntoView && TreeView)
		{
			TreeView->RequestNavigateToItem(Entry);
		}
	}
	else
	{
		SetSelection(MakeConstArrayView<TSharedPtr<FBindingEntry>>({}));
	}
}

void SBindingsList::SelectParameter(const FMVVMBlueprintPinId& PinId, bool bShouldScrollIntoView)
{
	if (TSharedPtr<FBindingEntry> Entry = Private::FindParameter(PinId, FilteredRootGroups))
	{
		SetSelection(MakeConstArrayView({Entry}));
		if (bShouldScrollIntoView && TreeView)
		{
			TreeView->RequestNavigateToItem(Entry);
		}
	}
	else
	{
		SetSelection(MakeConstArrayView<TSharedPtr<FBindingEntry>>({}));
	}
}

void SBindingsList::SetRootGroupsExpansion(bool InIsExpanded)
{
	if (TreeView.IsValid())
	{
		for (const TSharedPtr<FBindingEntry>& Entry : TreeView->GetRootItems())
		{
			TreeView->SetItemExpansion(Entry, InIsExpanded);
		}
	}
}

void SBindingsList::SetBindingsExpansion(bool InIsExpanded)
{
	if (TreeView.IsValid())
	{
		TArray< TSharedPtr<FBindingEntry>> EntriesToVisit;
		for (const TSharedPtr<FBindingEntry>& Entry : TreeView->GetRootItems())
		{
			// Ensure expanded binding is visible by expanding root group
			if (InIsExpanded)
			{
				TreeView->SetItemExpansion(Entry, true);
			}

			EntriesToVisit.Append(Entry->GetAllChildren());
		}

		while (!EntriesToVisit.IsEmpty())
		{
			const TSharedPtr<FBindingEntry>& Entry = EntriesToVisit.Pop();
			
			TreeView->SetItemExpansion(Entry, InIsExpanded);
			EntriesToVisit.Append(Entry->GetAllChildren());
		}
	}
}

void SBindingsList::SetSelection(TConstArrayView<const TSharedPtr<FBindingEntry>> InEntries)
{
	if (TreeView.IsValid())
	{
		TreeView->ClearSelection();
		Private::SetBindingEntrySelection(TreeView, TreeView->GetRootItems(), InEntries);
	}
}

FReply SBindingsList::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Delete)
	{
		if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get())
		{
			UMVVMBlueprintView* BlueprintView = MVVMExtensionPtr->GetBlueprintView();
			const UWidgetBlueprint* WidgetBlueprint = MVVMExtensionPtr->GetWidgetBlueprint();
			TArray<TSharedPtr<FBindingEntry>> Selection = TreeView->GetSelectedItems();

			BindingEntry::FRowHelper::DeleteEntries(WidgetBlueprint, BlueprintView, Selection);
		}
		return FReply::Handled();
	}

	else if (InKeyEvent.GetModifierKeys().IsControlDown())
	{
		if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get())
		{
			UMVVMBlueprintView* BlueprintView = MVVMExtensionPtr->GetBlueprintView();
			UWidgetBlueprint* WidgetBlueprint = MVVMExtensionPtr->GetWidgetBlueprint();
			TArray<TSharedPtr<FBindingEntry>> Selection = TreeView->GetSelectedItems();

			if (InKeyEvent.GetKey() == EKeys::C)
			{
				BindingEntry::FRowHelper::CopyEntries(WidgetBlueprint, BlueprintView, Selection);
				return FReply::Handled();		
			}
			else if (InKeyEvent.GetKey() == EKeys::V)
			{
				BindingEntry::FRowHelper::PasteEntries(WidgetBlueprint, BlueprintView, Selection);
				return FReply::Handled();		
			}
			else if (InKeyEvent.GetKey() == EKeys::D)
			{
				TArray<const TSharedPtr<FBindingEntry>> NewSelection;
				BindingEntry::FRowHelper::DuplicateEntries(WidgetBlueprint, BlueprintView, Selection, NewSelection);
				SetSelection(NewSelection);
				return FReply::Handled();		
			}
		}
	}

	return FReply::Unhandled();
}

void SBindingsList::OnSourceListSelectionChanged(TSharedPtr<FBindingEntry> Entry, ESelectInfo::Type SelectionType) const
{
	if (bSelectionChangedGuard)
	{
		return;
	}
	TGuardValue<bool> ReentrantGuard(bSelectionChangedGuard, true);

	if (TSharedPtr<SBindingsPanel> BindingPanelPtr = BindingPanel.Pin())
	{
		if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get())
		{
			if (UMVVMBlueprintView* View = MVVMExtensionPtr->GetBlueprintView())
			{
				TArray<TSharedPtr<FBindingEntry>> SelectedEntries = TreeView->GetSelectedItems();
				TArray<FBindingsSelectionVariantType> SelectionVariants;

				for (const TSharedPtr<FBindingEntry>& SelectedEntry : SelectedEntries)
				{
					switch (Entry->GetRowType())
					{
					case FBindingEntry::ERowType::Binding:
					{
						if (FMVVMBlueprintViewBinding* SelectedBinding = Entry->GetBinding(View))
						{
							FBindingsSelectionVariantType SelectionVariant;
							SelectionVariant.Set<FMVVMBlueprintViewBinding*>(SelectedBinding);
							SelectionVariants.Add(SelectionVariant);
						}
						break;
					}
					case FBindingEntry::ERowType::Condition:
					{
						if (UMVVMBlueprintViewCondition* SelectedCondition = Entry->GetCondition())
						{
							FBindingsSelectionVariantType SelectionVariant;
							SelectionVariant.Set<UMVVMBlueprintViewCondition*>(SelectedCondition);
							SelectionVariants.Add(SelectionVariant);
						}
						break;
					}
					case FBindingEntry::ERowType::Event:
					{
						if (UMVVMBlueprintViewEvent* SelectedEvent = Entry->GetEvent())
						{
							FBindingsSelectionVariantType SelectionVariant;
							SelectionVariant.Set<UMVVMBlueprintViewEvent*>(SelectedEvent);
							SelectionVariants.Add(SelectionVariant);
						}
						break;
					}
					default:
					{
						break;
					}
					}
				}

				BindingPanelPtr->OnBindingListSelectionChanged(SelectionVariants);
			}
		}
	}
}

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
