// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectKey.h"
#include "Widgets/SCompoundWidget.h"

namespace ESelectInfo { enum Type : int; }

class ITableRow;
template <typename ItemType> class STreeView;
class STableViewBase;
class UMVVMWidgetBlueprintExtension_View;
class UMVVMBlueprintViewEvent;
class UMVVMBlueprintViewCondition;
class FWidgetBlueprintEditor;

struct FMVVMBlueprintPinId;

namespace UE::MVVM
{
class SBindingsPanel;

struct FBindingEntry;

class SBindingsList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBindingsList) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SBindingsPanel> Owner, TSharedPtr<FWidgetBlueprintEditor> BlueprintEditor, UMVVMWidgetBlueprintExtension_View* MVVMExtension);
	~SBindingsList();

	void Refresh();

	void EventParametersRegenerate(UMVVMBlueprintViewEvent* Event);
	void ConditionParametersRegenerate(UMVVMBlueprintViewCondition* Condition);

	void OnFilterTextChanged(const FText& InFilterText);
	void ClearFilterText();

	/** Constructs context menu used for right click and dropdown button */
	TSharedPtr<SWidget> OnSourceConstructContextMenu();

	void RequestNavigateToBinding(FGuid BindingId);
	void RequestNavigateToEvent(UMVVMBlueprintViewEvent* Event);
	void RequestNavigateToCondition(UMVVMBlueprintViewCondition* Condition);
	
	void SetRootGroupsExpansion(bool InIsExpanded);
	void SetBindingsExpansion(bool InIsExpanded);

	void SetSelection(TConstArrayView<const TSharedPtr<FBindingEntry>> InEntries);

	void SelectBinding(FGuid BindingId, bool bShouldScrollIntoView);
	void SelectEvent(const UMVVMBlueprintViewEvent* Event, bool bShouldScrollIntoView);
	void SelectCondition(const UMVVMBlueprintViewCondition* Condition, bool bShouldScrollIntoView);
	void SelectParameter(const FMVVMBlueprintPinId& PinId, bool bShouldScrollIntoView);

	enum class EFilterMode : uint8
	{
		All = 0, // Display all bindings, no filtering
		Viewmodel = 1, // Display only bindings associated to a viewmodel
		Verse = 2	// Display only bindings associated to a verse field
	};

	EFilterMode GetActiveFilterMode() const;
	void OnActiveFilterModeChanged(EFilterMode InFilterMode);

private:
	TSharedRef<ITableRow> GenerateEntryRow(TSharedPtr<FBindingEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable);
	void OnSourceListSelectionChanged(TSharedPtr<FBindingEntry> Entry, ESelectInfo::Type SelectionType) const;
	void GetChildrenOfEntry(TSharedPtr<FBindingEntry> Entry, TArray<TSharedPtr<FBindingEntry>>& OutChildren) const;

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void ForceRefresh();
	template<typename TEntryValueType>
	void RegisterWrapperGraphModified(TEntryValueType* EntryValue, TSharedPtr<FBindingEntry> GroupEntry);
	void HandleRefreshChildren(FObjectKey EntryKey);

private:
	TWeakPtr<SBindingsPanel> BindingPanel;
	TSharedPtr<STreeView<TSharedPtr<FBindingEntry>>> TreeView;
	TMap<FObjectKey, TPair<TWeakPtr<FBindingEntry>, FDelegateHandle>> WrapperGraphModifiedDelegates;
	TArray<TSharedPtr<FBindingEntry>> AllRootGroups;
	TArray<TSharedPtr<FBindingEntry>> FilteredRootGroups;
	TWeakObjectPtr<UMVVMWidgetBlueprintExtension_View> MVVMExtension;
	TWeakPtr<FWidgetBlueprintEditor> WeakBlueprintEditor;
	mutable bool bSelectionChangedGuard = false;
	FText FilterText;
	EFilterMode FilterMode = EFilterMode::All;
};

} // namespace UE::MVVM
