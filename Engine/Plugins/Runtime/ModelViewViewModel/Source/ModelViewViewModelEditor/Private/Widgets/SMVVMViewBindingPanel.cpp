// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMViewBindingPanel.h"

#include "DetailsViewArgs.h"
#include "Engine/Engine.h"
#include "IDetailsView.h"
#include "MVVMBlueprintView.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMEditorSubsystem.h"
#include "MVVMViewBindingMenuContext.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "WidgetBlueprintEditor.h"
#include "MVVMBlueprintViewCondition.h"
#include "MVVMBlueprintViewEvent.h"
#include "WidgetBlueprintToolMenuContext.h"

#include "Customizations/MVVMConversionPathCustomization.h"
#include "Customizations/MVVMPropertyPathCustomization.h"
#include "Details/WidgetPropertyDragDropOp.h"
#include "Framework/MVVMBindingEditorHelper.h"
#include "Hierarchy/HierarchyWidgetDragDropOp.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"
#include "StatusBarSubsystem.h"
#include "ToolMenuContext.h"
#include "ToolMenus.h"
#include "Tabs/MVVMBindingSummoner.h"
#include "Tabs/MVVMViewModelSummoner.h"
#include "Styling/MVVMEditorStyle.h"
#include "Styling/ToolBarStyle.h"
#include "Toolkits/IToolkitHost.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SMVVMViewModelPanel.h"
#include "Widgets/SMVVMViewBindingListView.h"
#include "Widgets/ViewModelFieldDragDropOp.h"

#include "SPositiveActionButton.h"

#define LOCTEXT_NAMESPACE "BindingPanel"

bool CVar_MVVMViewBindingFilterMode_Enable = false;
FAutoConsoleVariableRef CVar_MVVMViewBindingFilterMode(TEXT("MVVM.ViewBinding.FilterMode.Enable"), CVar_MVVMViewBindingFilterMode_Enable, TEXT(""));

namespace UE::MVVM
{

namespace Private
{

struct FStructDetailNotifyHook : FNotifyHook
{
	virtual ~FStructDetailNotifyHook() = default;

	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override
	{
		if (Binding.BindingId.IsValid())
		{
			if (UMVVMWidgetBlueprintExtension_View* Extension = MVVMExtension.Get())
			{
				if (FMVVMBlueprintViewBinding* CurrentBinding = Extension->GetBlueprintView()->GetBinding(Binding.BindingId))
				{
					*CurrentBinding = Binding;
				}
			}
		}
	}

	FMVVMBlueprintViewBinding Binding;
	TWeakObjectPtr<UMVVMWidgetBlueprintExtension_View> MVVMExtension;
};

void SetSelectObjectsToViewSettings(TWeakPtr<FWidgetBlueprintEditor> WeakEditor)
{
	if (TSharedPtr<FWidgetBlueprintEditor> Editor = WeakEditor.Pin())
	{
		UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		if (!Subsystem)
		{
			return;
		}

		if (UMVVMBlueprintView* BlueprintView = Subsystem->GetView(Editor->GetWidgetBlueprintObj()))
		{
			Editor->CleanSelection();
			TSet<UObject*> Selections;
			Selections.Add(BlueprintView->GetSettings());
			Editor->SelectObjects(Selections);
		}
	}
}

} //namespace UE::MVVM::Private

/** */
void SBindingsPanel::Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor, bool bInIsDrawerTab)
{
	WeakBlueprintEditor = WidgetBlueprintEditor;
	bIsDrawerTab = bInIsDrawerTab;

	LoadSettings();

	UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj();
	check(WidgetBlueprint);

	const UMVVMDeveloperProjectSettings* MVVMSettings = GetDefault<UMVVMDeveloperProjectSettings>();
	const bool NeedToRequestView = MVVMSettings->bAllowBindingEditingWithoutViewModel;
	if (NeedToRequestView)
	{	//Make a view available from start without waiting for a view model to be added.
		GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->RequestView(WidgetBlueprint);
	}

	UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint);
	MVVMExtension = MVVMExtensionPtr;
	if (MVVMExtensionPtr)
	{
		BlueprintViewChangedDelegateHandle = MVVMExtensionPtr->OnBlueprintViewChangedDelegate().AddSP(this, &SBindingsPanel::HandleBlueprintViewChangedDelegate);
	}
	else
	{
		WidgetBlueprint->OnExtensionAdded.AddSP(this, &SBindingsPanel::HandleExtensionAdded);
	}

	{
		NotifyHook = MakePimpl<Private::FStructDetailNotifyHook>();
		NotifyHook->MVVMExtension = MVVMExtensionPtr;

		// Connection Settings
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.ViewIdentifier = NAME_None;
		DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsViewArgs.NotifyHook = NotifyHook.Get();

		FStructureDetailsViewArgs StructureDetailsViewArgs;
		StructDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureDetailsViewArgs, MakeShared<FStructOnScope>(FMVVMBlueprintViewBinding::StaticStruct(), reinterpret_cast<uint8*>(&NotifyHook->Binding)));
		StructDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(FMVVMBlueprintPropertyPath::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::MVVM::FPropertyPathCustomization::MakeInstance, WidgetBlueprint));
		StructDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(FMVVMBlueprintViewConversionPath::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::MVVM::FConversionPathCustomization::MakeInstance, WidgetBlueprint));
		StructDetailsView->GetDetailsView()->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SBindingsPanel::IsDetailsViewEditingEnabled));
	}

	HandleBlueprintViewChangedDelegate();
}

SBindingsPanel::~SBindingsPanel()
{
	if (UMVVMWidgetBlueprintExtension_View* Extension = MVVMExtension.Get())
	{
		Extension->OnBlueprintViewChangedDelegate().Remove(BlueprintViewChangedDelegateHandle);
		if (UMVVMBlueprintView* View = Extension->GetBlueprintView())
		{
			View->OnBindingsUpdated.RemoveAll(this);
		}
	}
	if (TSharedPtr<FWidgetBlueprintEditor> WidgetEditor = WeakBlueprintEditor.Pin())
	{
		UWidgetBlueprint* WidgetBlueprint = WidgetEditor->GetWidgetBlueprintObj();
		if (WidgetBlueprint)
		{
			WidgetBlueprint->OnExtensionAdded.RemoveAll(this);
		}
	}
}

void SBindingsPanel::SaveSettings()
{
	GConfig->SetInt(TEXT("MVVMViewBindingPanel"), TEXT("LastAddBindingMode"), static_cast<int32>(AddBindingMode), *GEditorPerProjectIni);
}

bool SBindingsPanel::IsDetailsViewEditingEnabled() const
{
	return false;
}

void SBindingsPanel::LoadSettings()
{
	GConfig->SetInt(TEXT("MVVMViewBindingPanel"), TEXT("LastAddBindingMode"), static_cast<int32>(AddBindingMode), *GEditorPerProjectIni);
	if (GConfig->DoesSectionExist(TEXT("MVVMViewBindingPanel"), *GEditorPerProjectIni))
	{
		int32 AddBindingModeAsInt = static_cast<int32>(EAddBindingMode::Selected);
		GConfig->GetInt(TEXT("MVVMViewBindingPanel"), TEXT("LastAddBindingMode"), AddBindingModeAsInt, *GEditorPerProjectIni);
		if (AddBindingModeAsInt >= 0 && AddBindingModeAsInt <= 1)
		{
			AddBindingMode = static_cast<EAddBindingMode>(AddBindingModeAsInt);
		}
	}
}

FReply SBindingsPanel::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();
	if (TSharedPtr<FWidgetBlueprintEditor> WidgetEditor = WeakBlueprintEditor.Pin())
	{
		if (WidgetEditor->GetToolkitCommands()->ProcessCommandBindings(InKeyEvent))
		{
			Reply = FReply::Handled();
		}
	}

	return Reply;
}

bool SBindingsPanel::SupportsKeyboardFocus() const
{
	return true;
}

void SBindingsPanel::HandleBlueprintViewChangedDelegate()
{
	ChildSlot
	[
		GenerateEditViewWidget()
	];
}

void SBindingsPanel::OnBindingListSelectionChanged(TConstArrayView<FBindingsSelectionVariantType> Selection)
{
	NotifyHook->Binding = FMVVMBlueprintViewBinding();

	const int SelectionCount = Selection.Num();

	if (Selection.Num() == 1)
	{
		TUnion<UObject*, TSharedPtr<FStructOnScope>> ObjectOrStruct;
		FBindingsSelectionVariantType SelectionVariant = Selection.Last();

		if (FMVVMBlueprintViewBinding* Binding = SelectionVariant.Get<FMVVMBlueprintViewBinding*>(nullptr))
		{
			NotifyHook->Binding = *Binding;

			TSharedRef<FStructOnScope> StructScope = MakeShared<FStructOnScope>(FMVVMBlueprintViewBinding::StaticStruct(), reinterpret_cast<uint8*>(Binding));
			ObjectOrStruct.SetSubtype<TSharedPtr<FStructOnScope>>(StructScope);
		}

		if (UMVVMBlueprintViewCondition* Condition = SelectionVariant.Get<UMVVMBlueprintViewCondition*>(nullptr))
		{
			ObjectOrStruct.SetSubtype<UObject*>(Condition);
		}

		if (UMVVMBlueprintViewEvent* Event = SelectionVariant.Get<UMVVMBlueprintViewEvent*>(nullptr))
		{
			ObjectOrStruct.SetSubtype<UObject*>(Event);
		}

		SetDetailsView(ObjectOrStruct);

	}
	else if(Selection.Num() > 1)
	{
		TUnion<UObject*, TSharedPtr<FStructOnScope>> EmptyObjectOrStruct;
		SetDetailsView(EmptyObjectOrStruct, LOCTEXT("DetailsNotAvailable", "Details are not available when multiple bindings are selected.\nSelect only one binding to view details"));
	}
	else
	{
		TUnion<UObject*, TSharedPtr<FStructOnScope>> EmptyObjectOrStruct;
		SetDetailsView(EmptyObjectOrStruct);
	}
}

bool SBindingsPanel::IsFilterModeEnabled() const
{
	return CVar_MVVMViewBindingFilterMode_Enable;
}

void SBindingsPanel::SetDetailsView(TUnion<UObject*, TSharedPtr<FStructOnScope>> ObjectOrStruct, const FText& ErrorMessage)
{
	if (!ErrorMessage.IsEmpty())
	{
		DetailsView->SetObject(nullptr);
		StructDetailsView->SetStructureData(nullptr);
		DetailContainer->SetContent(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
					.Text(ErrorMessage)
			]
		);

		return;
	}

	if (ObjectOrStruct.HasSubtype<UObject*>())
	{
		UObject* Object = ObjectOrStruct.GetSubtype<UObject*>();
		DetailsView->SetObject(Object);
		StructDetailsView->SetStructureData(nullptr);
		DetailContainer->SetContent(DetailsView.ToSharedRef());

		return;
	}

	if (ObjectOrStruct.HasSubtype<TSharedPtr<FStructOnScope>>())
	{
		TSharedPtr<FStructOnScope> SctructScope = ObjectOrStruct.GetSubtype<TSharedPtr<FStructOnScope>>();
		DetailsView->SetObject(nullptr);
		StructDetailsView->SetStructureData(SctructScope);
		DetailContainer->SetContent(StructDetailsView->GetWidget().ToSharedRef());

		return;
	}

	// Show empty details
	DetailsView->SetObject(nullptr);
	StructDetailsView->SetStructureData(nullptr);
	DetailContainer->SetContent(DetailsView.ToSharedRef());
}

void SBindingsPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	RefreshNotifyHookBinding();
	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SBindingsPanel::RefreshNotifyHookBinding()
{
	if (NotifyHook->Binding.BindingId.IsValid())
	{
		if (UMVVMWidgetBlueprintExtension_View* Extension = MVVMExtension.Get())
		{
			if (const FMVVMBlueprintViewBinding* Binding = Extension->GetBlueprintView()->GetBinding(NotifyHook->Binding.BindingId))
			{
				NotifyHook->Binding = *Binding;
			}
		}
	}
}

void SBindingsPanel::AddBindingToWidgetList(const TSet<FWidgetReference>& WidgetsToAddBinding)
{
	if (!CanAddBinding())
	{
		return;
	}
	if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get())
	{
		FGuid AddedBindingId;
		UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		if (TSharedPtr<FWidgetBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin())
		{
			bool bBindingAdded = false;

			if (AddBindingMode == EAddBindingMode::Selected)
			{
				TArray<FGuid> BindingIds;
				if (FMVVMBindingEditorHelper::CreateWidgetBindings(MVVMExtensionPtr->GetWidgetBlueprint(), WidgetsToAddBinding, BindingIds))
				{
					bBindingAdded = true;
					AddedBindingId = BindingIds.Last();
				}
			}

			if (!bBindingAdded)
			{
				FMVVMBlueprintViewBinding& Binding = EditorSubsystem->AddBinding(MVVMExtensionPtr->GetWidgetBlueprint());
				AddedBindingId = Binding.BindingId;
			}

			if (AddedBindingId.IsValid() && BindingsList)
			{
				BindingsList->RequestNavigateToBinding(AddedBindingId);
			}
		}
	}
}

void SBindingsPanel::AddDefaultBinding()
{
	if (TSharedPtr<FWidgetBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin())
	{
		AddBindingToWidgetList(BlueprintEditor->GetSelectedWidgets());
	}
}

bool SBindingsPanel::CanAddBinding() const
{
	UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get();
	return MVVMExtensionPtr && MVVMExtensionPtr->GetBlueprintView() != nullptr;
}

void SBindingsPanel::AddEmptyCondition()
{
	if (!CanAddEmptyCondition())
	{
		return;
	}
	if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get())
	{
		UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		if (TSharedPtr<FWidgetBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin())
		{
			UMVVMBlueprintViewCondition* Condition = EditorSubsystem->AddCondition(MVVMExtensionPtr->GetWidgetBlueprint());

			if (Condition && BindingsList)
			{
				BindingsList->RequestNavigateToCondition(Condition);
			}
		}
	}

}

bool SBindingsPanel::CanAddEmptyCondition() const
{
	UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get();
	return MVVMExtensionPtr && MVVMExtensionPtr->GetBlueprintView() != nullptr;
}

FText SBindingsPanel::GetAddEmptyConditionToolTip() const
{
	if (CanAddEmptyCondition())
	{
		return LOCTEXT("AddEmptyConditionTooltip", "Add an empty condition.");
	}
	else
	{
		return LOCTEXT("CannotAddEmptyConditionToolTip", "A viewmodel is required before adding conditions.");
	}

}


void SBindingsPanel::RefreshDetailsView()
{
	UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get();
	if (MVVMExtensionPtr && MVVMExtensionPtr->GetBlueprintView())
	{
		RefreshNotifyHookBinding();
		if (NotifyHook->Binding.BindingId.IsValid() && MVVMExtensionPtr->GetBlueprintView()->GetBinding(NotifyHook->Binding.BindingId))
		{
			TSharedRef<FStructOnScope> StructScope = MakeShared<FStructOnScope>(FMVVMBlueprintViewBinding::StaticStruct(), reinterpret_cast<uint8*>(&NotifyHook->Binding));
			StructDetailsView->SetStructureData(StructScope);
			DetailContainer->SetContent(StructDetailsView->GetWidget().ToSharedRef());
			return;
		}
	}
	DetailsView->SetObject(nullptr);
	StructDetailsView->SetStructureData(nullptr);
	DetailContainer->SetContent(DetailsView.ToSharedRef());
}

FText SBindingsPanel::GetAddBindingText() const
{
	if (TSharedPtr<FWidgetBlueprintEditor> WidgetEditor = WeakBlueprintEditor.Pin())
	{
		if (AddBindingMode == EAddBindingMode::Selected)
		{
			const TSet<FWidgetReference>& SelectedWidgets = WidgetEditor->GetSelectedWidgets();
			int32 NumberOfWidgetSelected = SelectedWidgets.Num();
			if (NumberOfWidgetSelected > 1)
			{
				return LOCTEXT("AddWidgets", "Add Widgets");
			}
			else if (NumberOfWidgetSelected == 1)
			{
				if (const FWidgetReference* Item = SelectedWidgets.FindArbitraryElement())
				{
					UWidget* WidgetTemplate = Item->GetTemplate();
					if (WidgetTemplate)
					{
						FText LabelText = WidgetTemplate->GetLabelText();
						if (!LabelText.IsEmpty())
						{ 
							return FText::Format(LOCTEXT("AddForWidget", "Add Widget {0}"), LabelText);
						}
					}
				}
			}
		}
	}
	return LOCTEXT("AddWidget", "Add Widget");
}

FText SBindingsPanel::GetAddBindingToolTip() const
{
	if (CanAddBinding())
	{
		if (AddBindingMode == EAddBindingMode::Selected)
		{
			return LOCTEXT("AddBindingSelectedTooltip", "Add a binding for each selected widget.");
		}
		else
		{
			return LOCTEXT("AddBindingTooltip", "Add an empty binding.");
		}
	}
	else
	{
		return LOCTEXT("CannotAddBindingToolTip", "A viewmodel is required before adding bindings.");
	}
}

TSharedRef<SWidget> SBindingsPanel::HandleAddDefaultBindingContextMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddSelectedWidget", "Add Selected Widget(s) binding"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SBindingsPanel::HandleAddDefaultBindingButtonClick, EAddBindingMode::Selected),
			FCanExecuteAction::CreateSP(this, &SBindingsPanel::CanAddBinding)
		));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddEmptyWidget", "Add Empty binding"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SBindingsPanel::HandleAddDefaultBindingButtonClick, EAddBindingMode::Empty),
			FCanExecuteAction::CreateSP(this, &SBindingsPanel::CanAddBinding)
		));

	return MenuBuilder.MakeWidget();
}

void SBindingsPanel::HandleAddDefaultBindingButtonClick(EAddBindingMode NewMode)
{
	if (AddBindingMode != NewMode)
	{
		AddBindingMode = NewMode;
		SaveSettings();
	}
	AddDefaultBinding();
}

TSharedRef<SWidget> SBindingsPanel::CreateDrawerDockButton()
{
	if (bIsDrawerTab)
	{
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("BindingDockInLayout_Tooltip", "Docks the binding drawer in tab."))
			.ContentPadding(FMargin(1.f, 0.f))
			.OnClicked(this, &SBindingsPanel::CreateDrawerDockButtonClicked)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.Layout"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DockInLayout", "Dock in Layout"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	return SNullWidget::NullWidget;
}

FReply SBindingsPanel::CreateDrawerDockButtonClicked()
{
	if (TSharedPtr<FWidgetBlueprintEditor> WidgetEditor = WeakBlueprintEditor.Pin())
	{
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ForceDismissDrawer();

		if (TSharedPtr<SDockTab> ExistingTab = WidgetEditor->GetToolkitHost()->GetTabManager()->TryInvokeTab(FMVVMBindingSummoner::TabID))
		{
			ExistingTab->ActivateInParent(ETabActivationCause::SetDirectly); 
		}
	}

	return FReply::Handled();
}

void SBindingsPanel::HandleExtensionAdded(UBlueprintExtension* NewExtension)
{
	if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = Cast<UMVVMWidgetBlueprintExtension_View>(NewExtension))
	{
		UWidgetBlueprint* WidgetBlueprint = MVVMExtensionPtr->GetWidgetBlueprint();

		if (WidgetBlueprint)
		{
			WidgetBlueprint->OnExtensionAdded.RemoveAll(this);

			MVVMExtension = MVVMExtensionPtr;

			if (!BlueprintViewChangedDelegateHandle.IsValid())
			{
				BlueprintViewChangedDelegateHandle = MVVMExtensionPtr->OnBlueprintViewChangedDelegate().AddSP(this, &SBindingsPanel::HandleBlueprintViewChangedDelegate);
			}

			if (MVVMExtensionPtr->GetBlueprintView() == nullptr)
			{
				MVVMExtensionPtr->CreateBlueprintViewInstance();
			}

			HandleBlueprintViewChangedDelegate();
		}
	}
}

TSharedRef<SWidget> SBindingsPanel::GenerateSettingsMenu()
{
	UMVVMViewBindingMenuContext* BindingMenuContext = NewObject<UMVVMViewBindingMenuContext>();
	BindingMenuContext->WidgetBlueprintEditor = WeakBlueprintEditor;
	BindingMenuContext->BindingsPanel = SharedThis(this);

	FToolMenuContext MenuContext(BindingMenuContext);
	return UToolMenus::Get()->GenerateWidget("MVVM.ViewBindings.Toolbar", MenuContext);
}


void SBindingsPanel::RegisterSettingsMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("MVVM.ViewBindings.Toolbar");
	FToolMenuSection& Section = Menu->FindOrAddSection("Settings");
	Section.AddDynamicEntry("Settings", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if (const UMVVMViewBindingMenuContext* Context = InSection.FindContext<UMVVMViewBindingMenuContext>())
			{
				if (GetDefault<UMVVMDeveloperProjectSettings>()->bShowViewSettings)
				{
					InSection.AddMenuEntry(
						"ViewSettings"
						, LOCTEXT("ViewSettings", "View Settings")
						, LOCTEXT("ViewSettingsTooltip", "View Settings")
						, FSlateIcon(FMVVMEditorStyle::Get().GetStyleSetName(), "BlueprintView.TabIcon")
						, FUIAction(FExecuteAction::CreateStatic(UE::MVVM::Private::SetSelectObjectsToViewSettings, Context->WidgetBlueprintEditor))
						, EUserInterfaceActionType::Button
					);
				}

				if (TSharedPtr<SBindingsPanel> BindingsPanel = Context->BindingsPanel.Pin())
				{
					if (!BindingsPanel->BindingsList.IsValid())
					{
						return;
					}

					TSharedRef<SBindingsList> BindingsListRef = BindingsPanel->BindingsList.ToSharedRef();

					InSection.AddMenuEntry(
						"ExpandAllCategories"
						, LOCTEXT("ExpandAllCategories", "Expand All Categories")
						, LOCTEXT("ExpandAllCategoriesTooltip", "Expands all root categories")
						, FSlateIcon()
						, FUIAction(FExecuteAction::CreateSP(BindingsListRef, &SBindingsList::SetRootGroupsExpansion, true))
						, EUserInterfaceActionType::Button
					);

					InSection.AddMenuEntry(
						"CollapseAllCategories"
						, LOCTEXT("CollapseAllCategories", "Collapse All Categories")
						, LOCTEXT("CollapseAllCategoriesTooltip", "Collapses all root categories")
						, FSlateIcon()
						, FUIAction(FExecuteAction::CreateSP(BindingsListRef, &SBindingsList::SetRootGroupsExpansion, false))
						, EUserInterfaceActionType::Button
					);

					InSection.AddMenuEntry(
						"ExpandAllBindings"
						, LOCTEXT("ExpandAllBindings", "Expand All Bindings")
						, LOCTEXT("ExpandAllBindingsTooltip", "Expands all bindings in every category")
						, FSlateIcon()
						, FUIAction(FExecuteAction::CreateSP(BindingsListRef, &SBindingsList::SetBindingsExpansion, true))
						, EUserInterfaceActionType::Button
					);

					InSection.AddMenuEntry(
						"CollapseAllBindings"
						, LOCTEXT("CollapseAllBindings", "Collapse All Bindings")
						, LOCTEXT("CollapseAllBindingsTooltip", "Collapses all bindings in every category")
						, FSlateIcon()
						, FUIAction(FExecuteAction::CreateSP(BindingsListRef, &SBindingsList::SetBindingsExpansion, false))
						, EUserInterfaceActionType::Button
					);
				}
			}
		}));
}

FReply SBindingsPanel::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FDragDropOperation> DragDropOp = DragDropEvent.GetOperation())
	{
		if (TSharedPtr<FHierarchyWidgetDragDropOp> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyWidgetDragDropOp>())
		{
			if (HierarchyDragDropOp->HasOriginatedFrom(WeakBlueprintEditor.Pin()))
			{
				if (CanAddBinding())
				{
					HierarchyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
					return FReply::Handled();
				}
			}
			HierarchyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
		}
		else if (TSharedPtr<FViewModelFieldDragDropOp> ViewModelFieldDragDropOp = DragDropEvent.GetOperationAs<FViewModelFieldDragDropOp>())
		{
			UWidgetBlueprint* DragWidgetBlueprint = ViewModelFieldDragDropOp->WidgetBP.Get();
			UWidgetBlueprint* CurrentWidgetBlueprint = WeakBlueprintEditor.Pin()->GetWidgetBlueprintObj();
			if (CurrentWidgetBlueprint == DragWidgetBlueprint && ViewModelFieldDragDropOp->ViewModelId.IsValid())
			{
				ViewModelFieldDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));	
				return FReply::Handled();
			}

			ViewModelFieldDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
		}
		else if (TSharedPtr<FWidgetPropertyDragDropOp> WidgetPropertyDragDropOp = DragDropEvent.GetOperationAs<FWidgetPropertyDragDropOp>())
		{
			UWidgetBlueprint* DragWidgetBlueprint = WidgetPropertyDragDropOp->WidgetBP.Get();
			UWidgetBlueprint* CurrentWidgetBlueprint = WeakBlueprintEditor.Pin()->GetWidgetBlueprintObj();
			if (CurrentWidgetBlueprint == DragWidgetBlueprint)
			{
				WidgetPropertyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				return FReply::Handled();
			}

			WidgetPropertyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
		}
		else
		{
			TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = nullptr;
			if (DragDropOp->IsOfType<FDecoratedDragDropOp>())
			{
				DecoratedDragDropOp = StaticCastSharedPtr<FDecoratedDragDropOp>(DragDropOp);
				DecoratedDragDropOp->ResetToDefaultToolTip();
				DecoratedDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
			}
		}
	}
	return FReply::Unhandled();
}

void SBindingsPanel::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>())
	{
		DecoratedDragDropOp->ResetToDefaultToolTip();
	}
}

FReply SBindingsPanel::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FHierarchyWidgetDragDropOp> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyWidgetDragDropOp>())
	{
		if (HierarchyDragDropOp->HasOriginatedFrom(WeakBlueprintEditor.Pin()))
		{
			if (CanAddBinding())
			{
				TSet<FWidgetReference> DraggedWidgetSet;
				for (const FWidgetReference& WidgetRef : HierarchyDragDropOp->GetWidgetReferences())
				{
					DraggedWidgetSet.Add(WidgetRef);
				}

				AddBindingToWidgetList(DraggedWidgetSet);
				return FReply::Handled();
			}
		}
	}
	else if (TSharedPtr<FViewModelFieldDragDropOp> ViewModelFieldDragDropOp = DragDropEvent.GetOperationAs<FViewModelFieldDragDropOp>())
	{
		UWidgetBlueprint* DragWidgetBlueprint = ViewModelFieldDragDropOp->WidgetBP.Get();
		UWidgetBlueprint* CurrentWidgetBlueprint = WeakBlueprintEditor.Pin()->GetWidgetBlueprintObj();
		if (CurrentWidgetBlueprint == DragWidgetBlueprint && ViewModelFieldDragDropOp->ViewModelId.IsValid())
		{
			FMVVMBlueprintPropertyPath PropertyPath;
			for (const FFieldVariant& Field : ViewModelFieldDragDropOp->DraggedField)
			{
				PropertyPath.AppendPropertyPath(DragWidgetBlueprint, FMVVMConstFieldVariant(Field));
			}

			PropertyPath.SetViewModelId(ViewModelFieldDragDropOp->ViewModelId);

			if (PropertyPath.IsValid())
			{
				UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
				FMVVMBlueprintViewBinding& ViewBinding = Subsystem->AddBinding(DragWidgetBlueprint);

				Subsystem->SetSourcePathForBinding(DragWidgetBlueprint, ViewBinding, PropertyPath);

				if (BindingsList)
				{
					BindingsList->RequestNavigateToBinding(ViewBinding.BindingId);
				}
				return FReply::Handled();
			}
		}
	}
	else if (TSharedPtr<FWidgetPropertyDragDropOp> WidgetPropertyDragDropOp = DragDropEvent.GetOperationAs<FWidgetPropertyDragDropOp>())
	{
		UWidgetBlueprint* DragWidgetBlueprint = WidgetPropertyDragDropOp->WidgetBP.Get();
		UWidgetBlueprint* CurrentWidgetBlueprint = WeakBlueprintEditor.Pin()->GetWidgetBlueprintObj();
		if (CurrentWidgetBlueprint == DragWidgetBlueprint)
		{
			FMVVMBlueprintPropertyPath PropertyPath;
			for (const FFieldVariant& Field : WidgetPropertyDragDropOp->DraggedPropertyPath)
			{
				PropertyPath.AppendPropertyPath(DragWidgetBlueprint, FMVVMConstFieldVariant(Field));
			}

			if (UWidget* OwnerWidgetPtr = WidgetPropertyDragDropOp->OwnerWidget.Get())
			{
				if (WeakBlueprintEditor.Pin()->GetPreview() == OwnerWidgetPtr)
				{
					PropertyPath.SetSelfContext();
				}
				else
				{
					PropertyPath.SetWidgetName(OwnerWidgetPtr->GetFName());
				}
			}

			if (PropertyPath.IsValid())
			{
				UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
				FMVVMBlueprintViewBinding& ViewBinding = Subsystem->AddBinding(DragWidgetBlueprint);

				Subsystem->SetDestinationPathForBinding(DragWidgetBlueprint, ViewBinding, PropertyPath, false);

				if (BindingsList)
				{
					BindingsList->RequestNavigateToBinding(ViewBinding.BindingId);
				}
				return FReply::Handled();
			}
		}
	}


	return FReply::Unhandled();
}

TSharedRef<SWidget> SBindingsPanel::GenerateEditViewWidget()
{
	FSlateIcon EmptyIcon(FAppStyle::GetAppStyleSetName(), "Icon.Empty");

	BindingsList = nullptr;
	if (MVVMExtension.Get())
	{
		BindingsList = SNew(SBindingsList, StaticCastSharedRef<SBindingsPanel>(AsShared()), WeakBlueprintEditor.Pin(), MVVMExtension.Get());
		if (UMVVMBlueprintView* View = MVVMExtension->GetBlueprintView())
		{
			View->OnBindingsUpdated.AddSP(this, &SBindingsPanel::RefreshDetailsView);
		}
	}

	DetailContainer = SNew(SBorder)
		.Visibility(EVisibility::Collapsed)
		[
			DetailsView.ToSharedRef()
		];

	TSharedPtr<SHorizontalBox> BindingPanelToolBar = SNew(SHorizontalBox);

	FSlimHorizontalToolBarBuilder ToolbarBuilderGlobal(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);

	// Insert widgets in the toolbar to the left of the search bar
	ToolbarBuilderGlobal.BeginSection("Picking");
	{
		ToolbarBuilderGlobal.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &SBindingsPanel::AddDefaultBinding),
				FCanExecuteAction::CreateSP(this, &SBindingsPanel::CanAddBinding),
				FGetActionCheckState()
			),
			NAME_None,
			MakeAttributeSP(this, &SBindingsPanel::GetAddBindingText),
			MakeAttributeSP(this, &SBindingsPanel::GetAddBindingToolTip),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), FName("Icons.Plus")),
			EUserInterfaceActionType::Button
		);

		ToolbarBuilderGlobal.AddComboButton(
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateSP(this, &SBindingsPanel::CanAddBinding),
				FGetActionCheckState()
			),
			FOnGetContent::CreateSP(this, &SBindingsPanel::HandleAddDefaultBindingContextMenu),
			FText::GetEmpty(),
			MakeAttributeSP(this, &SBindingsPanel::GetAddBindingToolTip),
			EmptyIcon,
			true
		);

		if (GetDefault<UMVVMDeveloperProjectSettings>()->bAllowConditionBinding)
		{
			ToolbarBuilderGlobal.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateSP(this, &SBindingsPanel::AddEmptyCondition),
					FCanExecuteAction::CreateSP(this, &SBindingsPanel::CanAddEmptyCondition),
					FGetActionCheckState()
				),
				NAME_None,
				LOCTEXT("AddCondition", "Add Condition"),
				MakeAttributeSP(this, &SBindingsPanel::GetAddEmptyConditionToolTip),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), FName("Icons.Plus")),
				EUserInterfaceActionType::Button
			);
		}
	}
	ToolbarBuilderGlobal.EndSection();

	// Pre-search box slot
	BindingPanelToolBar->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			ToolbarBuilderGlobal.MakeWidget()
		];

	TSharedRef<SWidget> SearchTextWidget = SNullWidget::NullWidget;
	if (BindingsList)
	{
		SearchTextWidget =
			SNew(SSearchBox)
			.HintText(LOCTEXT("SearchHint", "Search"))
			.SelectAllTextWhenFocused(false)
			.OnTextChanged(BindingsList.ToSharedRef(), &SBindingsList::OnFilterTextChanged);
		
	}

	// Search box slot
	BindingPanelToolBar->AddSlot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		[
			SearchTextWidget
		];

	TSharedRef<SWidget> FilterWidget = SNullWidget::NullWidget;
	if (BindingsList && IsFilterModeEnabled())
	{
		FilterWidget = SNew(SSegmentedControl<SBindingsList::EFilterMode>)
						.Value(BindingsList.ToSharedRef(), &SBindingsList::GetActiveFilterMode)
						.OnValueChanged(BindingsList.ToSharedRef(), &SBindingsList::OnActiveFilterModeChanged)
						+ SSegmentedControl<SBindingsList::EFilterMode>::Slot(SBindingsList::EFilterMode::All)
						.Icon(FAppStyle::GetBrush("Icons.Link"))
						.ToolTip(LOCTEXT("BindingPanelAllFilterModeToolTip", "All bindings"))
						+ SSegmentedControl<SBindingsList::EFilterMode>::Slot(SBindingsList::EFilterMode::Viewmodel)
						.Icon(FMVVMEditorStyle::Get().GetBrush("BlueprintView.TabIcon"))
						.ToolTip(LOCTEXT("BindingPanelViewModelFilterModeToolTip", "ViewModel bindings"))
						+ SSegmentedControl<SBindingsList::EFilterMode>::Slot(SBindingsList::EFilterMode::Verse)
						.Icon(FMVVMEditorStyle::Get().GetBrush("BindingView.VerseBinding"))
						.ToolTip(LOCTEXT("BindingPanelVerseFilterModeToolTip", "Verse bindings"));
	}

	BindingPanelToolBar->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			FilterWidget
		];

	// Reset the toolbar builder and insert widgets to the right of the search bar
	ToolbarBuilderGlobal = FSlimHorizontalToolBarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilderGlobal.AddWidget(SNew(SSpacer), NAME_None, true, HAlign_Right);

	{
		ToolbarBuilderGlobal.AddWidget(CreateDrawerDockButton());

		ToolbarBuilderGlobal.BeginSection("Options");

		if (GetDefault<UMVVMDeveloperProjectSettings>()->bShowDetailViewOptionInBindingPanel)
		{
			ToolbarBuilderGlobal.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateSP(this, &SBindingsPanel::ToggleDetailsVisibility),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SBindingsPanel::GetDetailsVisibleCheckState)
				),
				"ToggleDetails",
				LOCTEXT("Details", "Details"),
				LOCTEXT("DetailsToolTip", "Open Details View"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "WorldBrowser.DetailsButtonBrush"),
				EUserInterfaceActionType::ToggleButton
			);
		}

		ToolbarBuilderGlobal.AddWidget(
			SNew(SComboButton)
			.HasDownArrow(false)
			.ContentPadding(0)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.MenuContent()
			[
				GenerateSettingsMenu()
			]
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("DetailsView.ViewOptions"))
			]
		);

		ToolbarBuilderGlobal.EndSection();
	}

	// Post-search box slot
	BindingPanelToolBar->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			ToolbarBuilderGlobal.MakeWidget()
		];

	TSharedRef<SWidget> BindingWidget = SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage(FMVVMEditorStyle::Get().GetBrush("BindingView.ViewModelWarning"))
			.Visibility(this, &SBindingsPanel::GetViewModelMessageVisibility)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(20.0f, 20.0f, 12.0f, 20.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
				]
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 8.0f, 0.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MissingViewModel", "This editor requires a viewmodel that widgets can bind to, would you like to add a viewmodel now?"))
				]
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 20.0f, 0.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.OnClicked(this, &SBindingsPanel::HandleCreateViewModelClicked)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CreateViewModel", "Add Viewmodel"))
					]
				]
			]
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage(FMVVMEditorStyle::Get().GetBrush("BindingView.ViewModelWarning"))
			.Visibility(this, &SBindingsPanel::GetBindingMessageVisibility)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(20.0f, 20.0f, 12.0f, 20.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Info"))
				]
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 20.0f, 0.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DragWidgetCreateBinding", "Drag a widget from the Hierarchy to create a binding."))
				]
			]
		]
		+ SOverlay::Slot()
		[
			SNew(SBox)
			.Visibility(this, &SBindingsPanel::GetVisibility, true)
			[
				BindingsList ? BindingsList.ToSharedRef() : SNullWidget::NullWidget
			]
		];

	return SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 2.0f, 0.0f, 2.0f)
		[
			BindingPanelToolBar.ToSharedRef()
		]

		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SBorder)
			.BorderImage(FMVVMEditorStyle::Get().GetBrush("BindingView.Background"))
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Horizontal)
	
				+ SSplitter::Slot()
				.Value(0.75f)
				[
					BindingWidget
				]
	
				+ SSplitter::Slot()
				.Value(0.25f)
				[
					DetailContainer.ToSharedRef()
				]
			]
		];
}

ECheckBoxState SBindingsPanel::GetDetailsVisibleCheckState() const
{
	if (DetailContainer.IsValid())
	{
		return DetailContainer->GetVisibility() == EVisibility::Visible ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void SBindingsPanel::ToggleDetailsVisibility() 
{
	if (DetailContainer.IsValid())
	{
		EVisibility NewVisibility = GetDetailsVisibleCheckState() == ECheckBoxState::Checked ? EVisibility::Collapsed : EVisibility::Visible;
		DetailContainer->SetVisibility(NewVisibility);
	}
}

EVisibility SBindingsPanel::GetVisibility(bool bVisibleWithBindings) const
{
	if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get())
	{
		if (MVVMExtensionPtr->GetBlueprintView() != nullptr && MVVMExtensionPtr->GetBlueprintView()->HasAnyTypeOfBinding())
		{
			return bVisibleWithBindings ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}
	return bVisibleWithBindings ? EVisibility::Collapsed : EVisibility::Visible;
}

FReply SBindingsPanel::HandleCreateViewModelClicked()
{
	if (TSharedPtr<FWidgetBlueprintEditor> BPEditor = WeakBlueprintEditor.Pin())
	{
		if (TSharedPtr<SDockTab> DockTab = BPEditor->GetTabManager()->TryInvokeTab(FTabId(FViewModelSummoner::TabID)))
		{
			TSharedPtr<SWidget> ViewModelPanel = DockTab->GetContent();
			if (ViewModelPanel.IsValid())
			{
				StaticCastSharedPtr<SMVVMViewModelPanel>(ViewModelPanel)->OpenAddViewModelMenu();
			}
		}
	}
	return FReply::Handled();
}

EVisibility SBindingsPanel::GetViewModelMessageVisibility() const
{
	//Don't display the message about requiring viewmodels if viewmodels are actually not necessary.
	const UMVVMDeveloperProjectSettings* MVVMSettings = GetDefault<UMVVMDeveloperProjectSettings>();
	if (MVVMSettings->bAllowBindingEditingWithoutViewModel)
	{
		return EVisibility::Collapsed;
	}

	if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get())
	{
		if (MVVMExtensionPtr->GetBlueprintView() != nullptr &&
			MVVMExtensionPtr->GetBlueprintView()->GetViewModels().Num() > 0)
		{
			return EVisibility::Collapsed;
		}
	}
	return EVisibility::Visible;
}

EVisibility SBindingsPanel::GetBindingMessageVisibility() const
{
	if (UMVVMWidgetBlueprintExtension_View* MVVMExtensionPtr = MVVMExtension.Get())
	{
		if (MVVMExtensionPtr->GetBlueprintView() != nullptr &&
			MVVMExtensionPtr->GetBlueprintView()->GetViewModels().Num() > 0 &&
			!MVVMExtensionPtr->GetBlueprintView()->HasAnyTypeOfBinding())
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
