// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Union.h"
#include "WidgetReference.h"
#include "Widgets/SCompoundWidget.h"

struct FMVVMBlueprintViewBinding;
class FWidgetBlueprintEditor;
class IDetailsView;
class IStructureDetailsView;
class SBorder;
class UBlueprintExtension;
class UMVVMBlueprintViewCondition;
class UMVVMBlueprintViewEvent;
class UMVVMWidgetBlueprintExtension_View;

enum class ECheckBoxState : uint8;

namespace UE::MVVM
{
class SBindingsList;
namespace Private { struct FStructDetailNotifyHook; }

using FBindingsSelectionVariantType = TVariant<FMVVMBlueprintViewBinding*, UMVVMBlueprintViewCondition*, UMVVMBlueprintViewEvent*>;

class SBindingsPanel : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;

public:
	SLATE_BEGIN_ARGS(SBindingsPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor>, bool bInIsDrawerTab);
	virtual ~SBindingsPanel();

	//~ Begin SWidget Interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	//~ End SWidget Interface

	void OnBindingListSelectionChanged(TConstArrayView<FBindingsSelectionVariantType> Selection);
	bool IsFilterModeEnabled() const;

	static void RegisterSettingsMenu();

private:
	enum class EAddBindingMode
	{
		Selected = 0,
		Empty = 1,
	};

	void LoadSettings();
	void SaveSettings();

	void HandleBlueprintViewChangedDelegate();

	TSharedRef<SWidget> GenerateEditViewWidget();

	void AddBindingToWidgetList(const TSet<FWidgetReference>& WidgetsToAddBinding);
	void AddDefaultBinding();
	bool CanAddBinding() const;
	FText GetAddBindingText() const;
	FText GetAddBindingToolTip() const;
	TSharedRef<SWidget> HandleAddDefaultBindingContextMenu();
	void HandleAddDefaultBindingButtonClick(EAddBindingMode NewMode);

	void AddEmptyCondition();
	bool CanAddEmptyCondition() const;
	FText GetAddEmptyConditionToolTip() const;

	TSharedRef<SWidget> GenerateSettingsMenu();

	TSharedRef<SWidget> CreateDrawerDockButton();
	FReply CreateDrawerDockButtonClicked();

	void HandleExtensionAdded(UBlueprintExtension* NewExtension);

	FReply HandleCreateViewModelClicked();

	bool IsDetailsViewEditingEnabled() const;
	void RefreshDetailsView();
	void RefreshNotifyHookBinding();
	void SetDetailsView(TUnion<UObject*, TSharedPtr<FStructOnScope>> ObjectOrStruct, const FText& ErrorMessage = FText::GetEmpty());

	EVisibility GetViewModelMessageVisibility() const;
	EVisibility GetBindingMessageVisibility() const;

	EVisibility GetVisibility(bool bVisibleWithBindings) const;

	ECheckBoxState GetDetailsVisibleCheckState() const;
	void ToggleDetailsVisibility();

private:
	TWeakPtr<FWidgetBlueprintEditor> WeakBlueprintEditor;
	TSharedPtr<SBindingsList> BindingsList;
	TSharedPtr<SBorder> DetailContainer;
	TSharedPtr<IDetailsView> DetailsView;
	TSharedPtr<IStructureDetailsView> StructDetailsView;
	TWeakObjectPtr<UMVVMWidgetBlueprintExtension_View> MVVMExtension;
	TPimplPtr<Private::FStructDetailNotifyHook> NotifyHook;
	FDelegateHandle BlueprintViewChangedDelegateHandle;
	EAddBindingMode AddBindingMode = EAddBindingMode::Selected;
	bool bIsDrawerTab;
};

} // namespace UE::MVVM
