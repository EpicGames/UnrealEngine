// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMVVMViewModelBindingListWidget.h"

#include "Bindings/MVVMBindingHelper.h"
#include "Bindings/MVVMFieldPathHelper.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Blueprint/WidgetTree.h"
#include "BlueprintEditorSettings.h"
#include "Editor/EditorEngine.h"
#include "Extensions/UIComponentContainer.h"
#include "GraphEditorSettings.h"
#include "MVVMBlueprintPin.h"
#include "MVVMBlueprintViewModelContext.h"
#include "MVVMEditorSubsystem.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMSubsystem.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "PropertyPermissionList.h"
#include "Styling/MVVMEditorStyle.h"
#include "Types/MVVMAvailableBinding.h"
#include "Types/MVVMBindingSource.h"
#include "Types/MVVMConversionFunctionValue.h"
#include "UIComponentWidgetBlueprintExtension.h"
#include "WidgetBlueprint.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"


#define LOCTEXT_NAMESPACE "SSourceBindingList"

extern UNREALED_API UEditorEngine* GEditor;

using UE::PropertyViewer::SPropertyViewer;

namespace UE::MVVM
{

namespace Private
{
	struct FPassFilterInputs
	{
		FPassFilterInputs(
			const UWidgetBlueprint* WidgetBlueprint,
			const FMVVMAvailableBinding Binding,
			const UStruct* Struct,
			EFieldVisibility FieldVisibilityFlags,
			const FProperty* AssignableTo,
			bool bDoObjectProperty,
			bool bAllowComponentProperty,
			bool bCanConvert)
			: WidgetBlueprint(WidgetBlueprint)
			, Binding(Binding)
			, Struct(Struct)
			, FieldVisibilityFlags(FieldVisibilityFlags)
			, AssignableTo(AssignableTo)
			, bDoObjectProperty(bDoObjectProperty)
			, bAllowComponentProperty(bAllowComponentProperty)
			, bCanConvert(bCanConvert)
		{}

		const UWidgetBlueprint* WidgetBlueprint;
		const FMVVMAvailableBinding Binding;
		const UStruct* Struct;
		EFieldVisibility FieldVisibilityFlags;
		const FProperty* AssignableTo;
		bool bDoObjectProperty;
		bool bAllowComponentProperty;
		bool bCanConvert;
	};

	TOptional<FFieldVariant> PassFilter(const FPassFilterInputs& Inputs, const FMVVMFieldVariant& FieldVariant);
	TOptional<FFieldVariant> PassFilter(const FPassFilterInputs& Inputs);
	bool PassFilter_ConversionFunction(const FPassFilterInputs& Inputs, const FProperty* Source);

	TArray<FMVVMFieldConversion> GetCompatibleSourceConversions(const UWidgetBlueprint* WidgetBlueprint, const FProperty* Source, const FProperty* Destination, bool bReturnFirst);

	TOptional<FFieldVariant> PassFilter(const FPassFilterInputs& Inputs)
	{
		if (EnumHasAllFlags(Inputs.FieldVisibilityFlags, EFieldVisibility::Readable) && !Inputs.Binding.IsReadable())
		{
			return TOptional<FFieldVariant>();
		}

		if (EnumHasAllFlags(Inputs.FieldVisibilityFlags, EFieldVisibility::Notify | EFieldVisibility::Readable) && !Inputs.Binding.HasNotify())
		{
			return TOptional<FFieldVariant>();
		}

		FMVVMFieldVariant FieldVariant = BindingHelper::FindFieldByName(Inputs.Struct, Inputs.Binding.GetBindingName());
		return PassFilter(Inputs, FieldVariant);
	}

	TOptional<FFieldVariant> PassFilter(const FPassFilterInputs& Inputs, const FMVVMFieldVariant& FieldVariant)
	{
		static FName NAME_Hidden("Hidden");
		if (ensure(!FieldVariant.IsEmpty()))
		{
			if (FieldVariant.IsFunction())
			{
				const UFunction* Function = FieldVariant.GetFunction();
				if (Function == nullptr || Function->HasMetaData(NAME_Hidden))
				{
					return TOptional<FFieldVariant>();
				}

				if (!GetDefault<UMVVMDeveloperProjectSettings>()->IsFunctionAllowed(Inputs.WidgetBlueprint, Cast<const UClass>(Inputs.Struct), Function))
				{
					return TOptional<FFieldVariant>();
				}

				const FProperty* ReturnProperty = BindingHelper::GetReturnProperty(Function);
				bool bDoCompatibleTest = ReturnProperty == nullptr || Inputs.AssignableTo != nullptr;
				bool bDoWritableTest = true;
				// Do we allow walking up the tree
				if (CastField<FObjectPropertyBase>(ReturnProperty))
				{
					bDoCompatibleTest = bDoCompatibleTest && Inputs.bDoObjectProperty;
					bDoWritableTest = Inputs.bDoObjectProperty;
				}

				if (bDoWritableTest)
				{
					if (EnumHasAllFlags(Inputs.FieldVisibilityFlags, EFieldVisibility::Writable) && !Inputs.Binding.IsWritable())
					{
						return TOptional<FFieldVariant>();
					}
				}

				if (bDoCompatibleTest)
				{
					if (EnumHasAllFlags(Inputs.FieldVisibilityFlags, EFieldVisibility::Writable))
					{
						if (Inputs.AssignableTo && !BindingHelper::ArePropertiesCompatible(Inputs.AssignableTo, BindingHelper::GetFirstArgumentProperty(Function)))
						{
							if (Inputs.bCanConvert && PassFilter_ConversionFunction(Inputs, ReturnProperty))
							{
								return FFieldVariant(ReturnProperty);
							}
						}
					}
					if (EnumHasAllFlags(Inputs.FieldVisibilityFlags, EFieldVisibility::Readable))
					{
						if (Inputs.AssignableTo && !BindingHelper::ArePropertiesCompatible(ReturnProperty, Inputs.AssignableTo))
						{
							return TOptional<FFieldVariant>();
						}
					}
				}

				return FFieldVariant(Function);
			}
			else if (FieldVariant.IsProperty())
			{
				const FProperty* Property = FieldVariant.GetProperty();
				if (Property == nullptr || Property->HasMetaData(NAME_Hidden))
				{
					return TOptional<FFieldVariant>();
				}

				if (!GetDefault<UMVVMDeveloperProjectSettings>()->IsPropertyAllowed(Inputs.WidgetBlueprint, Inputs.Struct, Property))
				{
					return TOptional<FFieldVariant>();
				}

				bool bDoCompatibleTest = Inputs.AssignableTo != nullptr;
				bool bDoWritableTest = true;
				// Do we allow walking up the tree
				if (CastField<FObjectPropertyBase>(Property))
				{
					bDoCompatibleTest = bDoCompatibleTest && Inputs.bDoObjectProperty;
					bDoWritableTest = Inputs.bDoObjectProperty;
				}

				// If the path ends with the object property, then it needs to follow the writable rule
				if (bDoWritableTest)
				{
					if (EnumHasAllFlags(Inputs.FieldVisibilityFlags, EFieldVisibility::Writable) && !Inputs.Binding.IsWritable())
					{
						return TOptional<FFieldVariant>();
					}
				}

				if (bDoCompatibleTest)
				{
					if (EnumHasAllFlags(Inputs.FieldVisibilityFlags, EFieldVisibility::Writable))
					{
						if (!BindingHelper::ArePropertiesCompatible(Inputs.AssignableTo, Property))
						{
							if (Inputs.bCanConvert && PassFilter_ConversionFunction(Inputs, Property))
							{
								return FFieldVariant(Property);
							}
						}
					}
					if (EnumHasAllFlags(Inputs.FieldVisibilityFlags, EFieldVisibility::Readable))
					{
						if (!BindingHelper::ArePropertiesCompatible(Property, Inputs.AssignableTo))
						{
							return TOptional<FFieldVariant>();
						}
					}
				}

				// Hide the component property on the userwidget. It should only be visible from the owning widget.
				// The metadata helps us determine whether this is a component property generated by the compiler or not. We want to filter only those ones when bAllowComponentProperty is false.
				if (Property->HasMetaData(UUIComponentWidgetBlueprintExtension::MD_ComponentVariable) && !Inputs.bAllowComponentProperty)
				{
					return TOptional<FFieldVariant>();
				}

				return FFieldVariant(Property);
			}
		}

		return TOptional<FFieldVariant>();
	}

	bool PassFilter_ConversionFunction(const FPassFilterInputs& Inputs, const FProperty* Source)
	{
		const TArray<FMVVMFieldConversion>& Conversions = GetCompatibleSourceConversions(Inputs.WidgetBlueprint, Source, Inputs.AssignableTo, true);
		return Conversions.Num() > 0;
	}

	TArray<FMVVMFieldConversion> GetCompatibleSourceConversions(const UWidgetBlueprint* WidgetBlueprint, const FProperty* Source, const FProperty* Destination, bool bReturnFirst)
	{
		TArray<FMVVMFieldConversion> OutConversions;

		// No conversion necessary
		if (BindingHelper::ArePropertiesCompatible(Source, Destination))
		{
			return OutConversions;
		}

		if (UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>())
		{
			TArray<FConversionFunctionValue> ConversionFunctions = Subsystem->GetConversionFunctions(WidgetBlueprint, Source, Destination);
			for (const FConversionFunctionValue& ConversionFunction : ConversionFunctions)
			{
				// Returns function pins that support the Source property. If there are multiple, just pick the first. 
				// Currently no method to determine which parameter is the conversion value
				TArray<FMVVMBlueprintPinId> ArgumentPins = Subsystem->GetConversionFunctionPins(WidgetBlueprint, ConversionFunction, Source);
				if (!ArgumentPins.IsEmpty()) 
				{
					FMVVMFieldConversion Conversion(ConversionFunction, ArgumentPins[0]);
					OutConversions.Add(Conversion);
					if (bReturnFirst)
					{
						break;
					}
				}
			}
		}
		return OutConversions;
	}
}

FFieldIterator_Bindable::FFieldIterator_Bindable(const UWidgetBlueprint* InWidgetBlueprint, EFieldVisibility InVisibilityFlags, const FProperty* InAssignableTo, const bool InIsBindingToEvent, const bool bEnableConversion) :
	WidgetBlueprint(InWidgetBlueprint),
	FieldVisibilityFlags(InVisibilityFlags),
	AssignableTo(InAssignableTo),
	bIsBindingToEvent(InIsBindingToEvent),
	bEnableConversion(bEnableConversion)
{
}

TArray<FFieldVariant> FFieldIterator_Bindable::GetFields(const UStruct* Struct, const FName FieldName, const UStruct* Container) const
{
	TArray<FFieldVariant> Result;

	auto AddResult = [this, &Result](const TArray<FMVVMAvailableBinding>& AvailableBindingsList, const UStruct* Struct, const bool bAllowComponentProperty)
	{
		Result.Reserve(AvailableBindingsList.Num());

		EFilterFlag FilterFlags;
		if (UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint.Get()))
		{
			FilterFlags = ExtensionView->GetFilterSettings().FilterFlags;
		}
		else
		{
			FilterFlags = GetDefault<UMVVMDeveloperProjectSettings>()->FilterSettings.FilterFlags;
		}

		for (const FMVVMAvailableBinding& Value : AvailableBindingsList)
		{
			TOptional<FFieldVariant> PassResult;
			if (FilterFlags == EFilterFlag::All)
			{
				const bool bDoObjectProperty = AssignableTo != nullptr;
				const Private::FPassFilterInputs Params(WidgetBlueprint.Get(), Value, Struct, FieldVisibilityFlags, AssignableTo, bDoObjectProperty, bAllowComponentProperty, bEnableConversion);
				PassResult = Private::PassFilter(Params);
			}
			else
			{
				const Private::FPassFilterInputs Params(WidgetBlueprint.Get(), Value, Struct, EFieldVisibility::None, nullptr, false, bAllowComponentProperty, bEnableConversion);
				PassResult = Private::PassFilter(Params);
			}
			if (PassResult.IsSet())
			{
				Result.Add(MoveTemp(PassResult.GetValue()));
			}
		}
	};

	const UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get();
	TSubclassOf<UObject> AccessorClass = nullptr;
	if (WidgetBlueprintPtr)
	{
		AccessorClass = WidgetBlueprintPtr->SkeletonGeneratedClass ? WidgetBlueprintPtr->SkeletonGeneratedClass : WidgetBlueprintPtr->GeneratedClass;
	}

	if (const UClass* Class = Cast<const UClass>(Struct))
	{
		TArray<FMVVMAvailableBinding> Bindings = bIsBindingToEvent ? UMVVMSubsystem::GetAvailableBindingsForEvent(const_cast<UClass*>(Class), AccessorClass) : UMVVMSubsystem::GetAvailableBindings(const_cast<UClass*>(Class), AccessorClass);
		AddResult(Bindings, Struct, false);
	}
	else if (const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(Struct))
	{
		TArray<FMVVMAvailableBinding> Bindings = UMVVMSubsystem::GetAvailableBindingsForStruct(ScriptStruct);
		AddResult(Bindings, Struct, false);
	}

	Result.Sort([](const FFieldVariant& A, const FFieldVariant& B)
		{
			bool bIsAViewModel = A.Get<FObjectPropertyBase>() && A.Get<FObjectPropertyBase>()->PropertyClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass());
			bool bIsBViewModel = B.Get<FObjectPropertyBase>() && B.Get<FObjectPropertyBase>()->PropertyClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass());
			if (A.IsUObject() && B.IsUObject())
			{
				return A.GetFName().LexicalLess(B.GetFName());
			}
			else if (bIsAViewModel && bIsBViewModel)
			{
				return A.GetFName().LexicalLess(B.GetFName());
			}
			else if (bIsAViewModel)
			{
				return true;
			}
			else if (bIsBViewModel)
			{
				return false;
			}
			else if (A.IsUObject())
			{
				return true;
			}
			else if (B.IsUObject())
			{
				return false;
			}
			return A.GetFName().LexicalLess(B.GetFName());
		});

	// Add any component properties at the end.
	if (const UWidgetBlueprintGeneratedClass* WidgetBPG = Cast<UWidgetBlueprintGeneratedClass>(Container))
	{
		if (const UWidgetBlueprint* ContainerWidgetBlueprintPtr = Cast<UWidgetBlueprint>(WidgetBPG->ClassGeneratedBy))
		{
			if (const UClass* Class = Cast<const UClass>(Struct))
			{
				if (UWidget* Widget = ContainerWidgetBlueprintPtr->WidgetTree->FindWidget(FieldName))
				{
					if (UUIComponentWidgetBlueprintExtension* ComponentWidgetBlueprintExtension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(ContainerWidgetBlueprintPtr))
					{
						TArray<FMVVMAvailableBinding> ComponentBindings = bIsBindingToEvent ? UMVVMSubsystem::GetAvailableBindingsForEvent(Cast<const UClass>(Container), AccessorClass) : UMVVMSubsystem::GetAvailableBindings(Cast<const UClass>(Container), AccessorClass);

						TArray<UUIComponent*> ComponentsList = ComponentWidgetBlueprintExtension->GetComponentsFor(Widget);
						for (UUIComponent* Component : ComponentsList)
						{
							const FMVVMAvailableBinding* FoundBinding = ComponentBindings.FindByPredicate(
								[Component, Widget](const FMVVMAvailableBinding& Binding)
								{
									FName ComponentPropertyName = UUIComponentContainer::GetPropertyNameForComponent(Component, Widget->GetFName());
									return Binding.GetBindingName().ToString() == ComponentPropertyName;
								});
							if (FoundBinding)
							{
								TArray <FMVVMAvailableBinding> Bindings;
								Bindings.Reserve(1);
								Bindings.Add(*FoundBinding);
								AddResult(Bindings, Container, true);
							}
						}
					}
				}
			}
		}
	}

	return Result;
}

/** */
FFieldExpander_Bindable::FFieldExpander_Bindable()
{
	SetExpandObject(UE::PropertyViewer::FFieldExpander_Default::EObjectExpandFlag::UseInstanceClass);
	SetExpandScriptStruct(true);
	SetExpandFunction(UE::PropertyViewer::FFieldExpander_Default::EFunctionExpand::FunctionProperties);
}

TOptional<const UClass*> FFieldExpander_Bindable::CanExpandObject(const FObjectPropertyBase* Property, const UObject* Instance) const
{
	if (CastField<FObjectProperty>(Property))
	{
		TOptional<const UClass*> Result = UE::PropertyViewer::FFieldExpander_Default::CanExpandObject(Property, Instance);
		if (Result.IsSet() && Result.GetValue())
		{
			if (GetDefault<UBlueprintEditorSettings>()->IsClassAllowedOnPin(Result.GetValue()))
			{
				return Result;
			}
		}
	}
	return TOptional<const UClass*>();
}

bool FFieldExpander_Bindable::CanExpandScriptStruct(const FStructProperty* StructProperty) const
{
	if (UE::PropertyViewer::FFieldExpander_Default::CanExpandScriptStruct(StructProperty))
	{
		const FPathPermissionList& StructPermissions = GetMutableDefault<UBlueprintEditorSettings>()->GetStructPermissions();
		return !StructPermissions.HasFiltering() || StructPermissions.PassesFilter(StructProperty->Struct->GetPathName());
	}
	return false;
}

TOptional<const UStruct*> FFieldExpander_Bindable::GetExpandedFunction(const UFunction* Function) const
{
	const FProperty* ReturnProperty = Function ? BindingHelper::GetReturnProperty(Function) : nullptr;
	if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(ReturnProperty))
	{
		if (GetDefault<UBlueprintEditorSettings>()->IsClassAllowedOnPin(ObjectProperty->PropertyClass))
		{
			return ObjectProperty->PropertyClass.Get();
		}
	}
	//else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(ReturnProperty))
	//{
	//	if (CanExpandScriptStruct(StructProperty))
	//	{
	//		return StructProperty->Struct.Get();
	//	}
	//}
	return TOptional<const UStruct*>();
}

TSharedRef<SWidget> ConstructFieldPreSlot(const UWidgetBlueprint* WidgetBlueprint, UE::PropertyViewer::SPropertyViewer::FHandle Handle, const FFieldVariant FieldPath, const bool bIsForEvent)
{
	TSharedRef<SWidget> ImageWidget = SNullWidget::NullWidget;
	const UClass* AccessorClass = nullptr;
	if (WidgetBlueprint != nullptr)
	{
		AccessorClass = WidgetBlueprint->SkeletonGeneratedClass ? WidgetBlueprint->SkeletonGeneratedClass : WidgetBlueprint->GeneratedClass;
	}
	FMVVMAvailableBinding Binding = bIsForEvent ? UMVVMSubsystem::GetAvailableBindingForEvent(FMVVMConstFieldVariant(FieldPath), AccessorClass) : UMVVMSubsystem::GetAvailableBindingForField(FMVVMConstFieldVariant(FieldPath), AccessorClass);
	if (Binding.IsValid())
	{
		const FSlateBrush* Brush = nullptr;
		if (Binding.HasNotify())
		{
			if (Binding.IsReadable() && Binding.IsWritable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.TwoWay");
			}
			else if (Binding.IsReadable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.OneWayToSource");
			}
			else if (Binding.IsWritable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.OneWay");
			}
		}
		else
		{
			if (Binding.IsReadable() && Binding.IsWritable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.OneTimeTwoWay");
			}
			else if (Binding.IsReadable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.OneTimeOneWay");
			}
			else if (Binding.IsWritable())
			{
				Brush = FMVVMEditorStyle::Get().GetBrush("BindingMode.OneTimeOneWayToSource");
			}
		}
		
		if (Brush)
		{
			ImageWidget = SNew(SImage)
				.Image(Brush);
		}
	}

	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.WidthOverride(16.0f)
		.HeightOverride(16.0f)
		[
			ImageWidget
		];
}

/** */

FMVVMFieldConversion::FMVVMFieldConversion(FConversionFunctionValue Function, FMVVMBlueprintPinId ParameterId)
	: Function(Function), ParameterId(ParameterId)
{
}

bool FMVVMFieldConversion::IsValid() const
{
	return Function.IsValid() && ParameterId.IsValid(); 
}

FName FMVVMFieldConversion::GetFName() const
{ 
	return Function.GetFName(); 
}

FMVVMBlueprintFunctionReference FMVVMFieldConversion::ToBlueprintFunctionReference(UBlueprint* Blueprint) const
{
	if (Function.IsFunction())
	{
		return FMVVMBlueprintFunctionReference(Blueprint, Function.GetFunction());
	}
	else if (Function.IsNode())
	{
		return FMVVMBlueprintFunctionReference(Function.GetNode());
	}
	return FMVVMBlueprintFunctionReference();
}

/** */
void SSourceBindingList::Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint)
{
	WidgetBlueprint = InWidgetBlueprint;
	FieldIterator = MakeUnique<FFieldIterator_Bindable>(InWidgetBlueprint, InArgs._FieldVisibilityFlags, InArgs._AssignableTo, InArgs._IsBindingToEvent, InArgs._bEnableConversion);
	FieldExpander = MakeUnique<FFieldExpander_Bindable>();

	bIsBindingToEvent = InArgs._IsBindingToEvent;
	OnDoubleClicked = InArgs._OnDoubleClicked;

	const UGraphEditorSettings* GraphEditorSettings = GetDefault<UGraphEditorSettings>();
	UE::PropertyViewer::FFieldColorSettings OverrideIconColorSettings;
	OverrideIconColorSettings.DefaultTypeColor = GraphEditorSettings->DefaultPinTypeColor;
	OverrideIconColorSettings.BooleanTypeColor = GraphEditorSettings->BooleanPinTypeColor;
	OverrideIconColorSettings.ClassTypeColor = GraphEditorSettings->ClassPinTypeColor;
	OverrideIconColorSettings.EnumTypeColor = GraphEditorSettings->BytePinTypeColor;
	OverrideIconColorSettings.IntTypeColor = GraphEditorSettings->IntPinTypeColor;
	OverrideIconColorSettings.FloatTypeColor = GraphEditorSettings->FloatPinTypeColor;
	OverrideIconColorSettings.NameTypeColor = GraphEditorSettings->NamePinTypeColor;
	OverrideIconColorSettings.DelegateTypeColor = GraphEditorSettings->DelegatePinTypeColor;
	OverrideIconColorSettings.ObjectTypeColor = GraphEditorSettings->ObjectPinTypeColor;
	OverrideIconColorSettings.InterfaceTypeColor = GraphEditorSettings->InterfacePinTypeColor;
	OverrideIconColorSettings.StringTypeColor = GraphEditorSettings->StringPinTypeColor;
	OverrideIconColorSettings.TextTypeColor = GraphEditorSettings->TextPinTypeColor;
	OverrideIconColorSettings.DefaultStructTypeColor = GraphEditorSettings->StructPinTypeColor;
	OverrideIconColorSettings.StructColors.Add(TBaseStructure<FVector>::Get()->GetStructPathName().ToString(), GraphEditorSettings->VectorPinTypeColor);
	OverrideIconColorSettings.StructColors.Add(TBaseStructure<FRotator>::Get()->GetStructPathName().ToString(), GraphEditorSettings->RotatorPinTypeColor);
	OverrideIconColorSettings.StructColors.Add(TBaseStructure<FTransform>::Get()->GetStructPathName().ToString(), GraphEditorSettings->TransformPinTypeColor);

	PropertyViewer = SNew(SPropertyViewer)
		.FieldIterator(FieldIterator.Get())
		.FieldExpander(FieldExpander.Get())
		.PropertyVisibility(SPropertyViewer::EPropertyVisibility::Hidden)
		.bShowFieldIcon(true)
		.OverrideIconColorSettings(OverrideIconColorSettings)
		.bSanitizeName(true)
		.SelectionMode(InArgs._EnableSelection ? ESelectionMode::Single : ESelectionMode::None)
		.bShowSearchBox(InArgs._ShowSearchBox)
		.OnGetPreSlot(this, &SSourceBindingList::HandleGetPreSlot)
		.OnSelectionChanged(this, &SSourceBindingList::HandleSelectionChanged)
		.OnDoubleClicked(this, &SSourceBindingList::HandleDoubleClicked);

	ChildSlot
	[
		PropertyViewer.ToSharedRef()
	];
}

void SSourceBindingList::ClearSources()
{
	SelectedPath = FMVVMBlueprintPropertyPath();

	Sources.Reset();
	if (PropertyViewer)
	{
		PropertyViewer->RemoveAll();
	}
}

void SSourceBindingList::AddWidgetBlueprint()
{
	FBindingSource Source = FBindingSource::CreateForBlueprint(WidgetBlueprint.Get());
	AddSources(MakeArrayView(&Source, 1 ));
}

void SSourceBindingList::AddWidgets(TArrayView<const UWidget*> InWidgets)
{
	TArray<FBindingSource, TInlineAllocator<16>> NewSources;
	NewSources.Reserve(InWidgets.Num());

	for (const UWidget* Widget : InWidgets)
	{
		NewSources.Add(FBindingSource::CreateForWidget(WidgetBlueprint.Get(), Widget));
	}

	AddSources(NewSources);
}

void SSourceBindingList::AddViewModels(TArrayView<const FMVVMBlueprintViewModelContext> InViewModels)
{
	TArray<FBindingSource, TInlineAllocator<16>> NewSources;
	NewSources.Reserve(InViewModels.Num());

	for (const FMVVMBlueprintViewModelContext& ViewModelContext : InViewModels)
	{
		NewSources.Add(FBindingSource::CreateForViewModel(WidgetBlueprint.Get(), ViewModelContext));
	}

	AddSources(NewSources);
}

void SSourceBindingList::AddSource(const FBindingSource& InSource)
{
	AddSources(MakeArrayView(&InSource, 1));
}

void SSourceBindingList::AddSources(TArrayView<const FBindingSource> InSources)
{
	if (ensure(PropertyViewer))
	{
		const UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get();
		for (const FBindingSource& Source : InSources)
		{
			if (const UClass* SourceClass = Source.GetClass())
			{
				SPropertyViewer::FHandle Handle;
				if (SourceClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
				{
					UWidget* Widget = nullptr;
					if (WidgetBlueprintPtr && Source.GetSource() == EMVVMBlueprintFieldPathSource::Widget)
					{
						Widget = WidgetBlueprintPtr->WidgetTree->FindWidget(Source.GetWidgetName());
					}

					if (Widget)
					{
						Handle = PropertyViewer->AddInstance(Widget);
					
						// Add a new source for each component on this widget
						if (UUIComponentWidgetBlueprintExtension* ComponentWidgetBlueprintExtension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprintPtr))
						{
							TArray<UUIComponent*> ComponentsList = ComponentWidgetBlueprintExtension->GetComponentsFor(Widget);
							for (UUIComponent* Component : ComponentsList)
							{
								SPropertyViewer::FHandle ComponentHandle = PropertyViewer->AddInstance(Component);
								Sources.Emplace(FBindingSource::CreateForWidgetComponent(WidgetBlueprint.Get(), Widget->GetFName(), Component), ComponentHandle);
							}
						}
					}
					else
					{
						Handle = PropertyViewer->AddContainer(SourceClass);
					}
				}
				Sources.Emplace(Source, Handle);
			}
		}
	}
}

bool IsPropertyPathComponent(TArrayView<const FFieldVariant> FieldPath, FMVVMBlueprintPropertyPath& PropertyPath, const UWidgetBlueprint* WidgetBlueprintPtr)
{
	if (!WidgetBlueprintPtr)
	{
		ensure(false);
		return false;
	}

	const FObjectPropertyBase* WidgetProperty = nullptr;
	if (PropertyPath.GetSource(WidgetBlueprintPtr) == EMVVMBlueprintFieldPathSource::Widget)
	{
		if (UClass* GeneratedClass = WidgetBlueprintPtr->SkeletonGeneratedClass ? WidgetBlueprintPtr->SkeletonGeneratedClass : WidgetBlueprintPtr->GeneratedClass)
		{
			WidgetProperty = CastField<const FObjectPropertyBase>(GeneratedClass->FindPropertyByName(PropertyPath.GetWidgetName()));
		}
	}
	const FObjectPropertyBase* OwningWidget = WidgetProperty;

	/* We need 3 pieces of information to decide if this property path includes a valid component that is registered with a UIComponentWidgetBlueprintExtension:
		1. The widget blueprint that holds a component extension. This is the parent class of the component property.
		We need to use LastWidgetBP when the component object is on a userwidget.  
		2. OwningWidget: The widget that owns this component. We care about the name of this widget so we can verify it with the widget blueprint extension.
		3. The component itself. 
	*/
	for (const FFieldVariant& FieldVariant : FieldPath)
	{
		const FObjectPropertyBase* ObjectProperty = FieldVariant.Get<FObjectPropertyBase>();
			
		if (!ObjectProperty)
		{
			continue;
		}
		if (ObjectProperty->PropertyClass->IsChildOf(UWidget::StaticClass()))
		{
			OwningWidget = ObjectProperty;
		}
		else if (ObjectProperty->PropertyClass->IsChildOf(UUIComponent::StaticClass()))
		{
			if (!OwningWidget)
			{
				continue;
			}
			if (UWidgetBlueprintGeneratedClass* OwnerWidgetBlueprintGeneratedClass = Cast<UWidgetBlueprintGeneratedClass>(ObjectProperty->GetOwnerClass()))
			{
				UWidgetBlueprint* OwnerWidgetBlueprint = Cast<UWidgetBlueprint>(OwnerWidgetBlueprintGeneratedClass->ClassGeneratedBy);
				if (UUIComponentWidgetBlueprintExtension* ComponentWidgetBlueprintExtension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(OwnerWidgetBlueprint))
				{
					if (UWidget* Widget = OwnerWidgetBlueprint->WidgetTree->FindWidget(OwningWidget->GetFName()))
					{
						TArray<UUIComponent*> ComponentsList = ComponentWidgetBlueprintExtension->GetComponentsFor(Widget);
						for (UUIComponent* Component : ComponentsList)
						{
							check(Component);
							FName ComponentPropertyName = UUIComponentContainer::GetPropertyNameForComponent(Component, Widget->GetFName());
							if (ObjectProperty->PropertyClass == Component->GetClass() && ObjectProperty->GetFName() == ComponentPropertyName)
							{
									return true;
							}
						}
					}			
				}
			}
		}
	}

	return false;
}

FMVVMBlueprintPropertyPath SSourceBindingList::CreateBlueprintPropertyPath(SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> FieldPath) const
{
	if (FieldPath.Num() < 0 || !Handle.IsValid())
	{
		return FMVVMBlueprintPropertyPath();
	}

	const UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get();
	ensure(WidgetBlueprintPtr);
	if (!WidgetBlueprintPtr)
	{
		return FMVVMBlueprintPropertyPath();
	}

	const TPair<FBindingSource, SPropertyViewer::FHandle>* Source = Sources.FindByPredicate(
		[Handle](const TPair<FBindingSource, SPropertyViewer::FHandle>& Source)
		{
			return Source.Value == Handle;
		});

	if (!Source)
	{
		return FMVVMBlueprintPropertyPath();
	}

	if (FieldPath.Num() > 0)
	{
		if (const FProperty* Property = FieldPath.Last().Get<FProperty>())
		{
			if (Property->HasMetaData(UUIComponentWidgetBlueprintExtension::MD_ComponentVariable))
			{
				return FMVVMBlueprintPropertyPath();
			}
		}
	}

	const UClass* AccessorClass = WidgetBlueprintPtr->SkeletonGeneratedClass ? WidgetBlueprintPtr->SkeletonGeneratedClass : WidgetBlueprintPtr->GeneratedClass;
	FMVVMBlueprintPropertyPath PropertyPath;
	if (FieldPath.Num() > 0)
	{
		// Backward, test if the object can be access.
		//The last property can be a struct variable, inside a struct, inside..., inside an object. 
		bool bPassFilter = false;
		const UStruct* CurrentContainer = Source->Key.GetClass();
		for (const FFieldVariant& FieldVariant : FieldPath)
		{
			FMVVMConstFieldVariant NewField;
			FName FieldName;
			if (const FProperty* Property = FieldVariant.Get<FProperty>())
			{
				NewField = FMVVMConstFieldVariant(Property);
				FieldName = Property->GetFName();
			}
			else if (const UFunction* Function = FieldVariant.Get<UFunction>())
			{
				NewField = FMVVMConstFieldVariant(Function);
				FieldName = Function->GetFName();
			}

			if (const UClass* OwnerClass = Cast<const UClass>(CurrentContainer))
			{
				FMVVMAvailableBinding Binding = bIsBindingToEvent ? UMVVMSubsystem::GetAvailableBindingForEvent(OwnerClass, FMVVMBindingName(FieldName), AccessorClass) : UMVVMSubsystem::GetAvailableBinding(OwnerClass, FMVVMBindingName(FieldName), AccessorClass);
				if (Binding.IsValid())
				{
					EFilterFlag FilterFlags;
					if (UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint.Get()))
					{
						FilterFlags = ExtensionView->GetFilterSettings().FilterFlags;
					}
					else
					{
						FilterFlags = GetDefault<UMVVMDeveloperProjectSettings>()->FilterSettings.FilterFlags;
					}

					if (FilterFlags == EFilterFlag::All)
					{
						const Private::FPassFilterInputs Params(WidgetBlueprint.Get(), Binding, OwnerClass, FieldIterator->GetFieldVisibilityFlags(), FieldIterator->GetAssignableTo(), true, false, IsConversionEnabled());
						bPassFilter = Private::PassFilter(Params).IsSet();
					}
					else
					{
						const Private::FPassFilterInputs Params(WidgetBlueprint.Get(), Binding, OwnerClass, EFieldVisibility::None, nullptr, true, false, IsConversionEnabled());
						bPassFilter = Private::PassFilter(Params).IsSet();
					}
				}
				break;
			}

			TValueOrError<const UStruct*, void> NewContainerResult = FieldPathHelper::GetFieldAsContainer(NewField);
			CurrentContainer = NewContainerResult.HasValue() ? NewContainerResult.GetValue() : nullptr;
		}

		if (bPassFilter)
		{
			Source->Key.SetSourceTo(PropertyPath);
			PropertyPath.ResetPropertyPath();

			// When the component is directly added to the property viewer, we will need to add it to the path manually since the given path doesn't include the container.
			FFieldVariant ComponentField;
			TArray<const FFieldVariant> CompleteFieldPath;
			if (PropertyPath.GetSource(WidgetBlueprintPtr) == EMVVMBlueprintFieldPathSource::Widget)
			{
				if (UUIComponentWidgetBlueprintExtension* ComponentWidgetBlueprintExtension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprintPtr))
				{
					if (UWidget* Widget = WidgetBlueprint->WidgetTree->FindWidget(Source->Key.GetWidgetName()))
					{
						TArray<UUIComponent*> ComponentsList = ComponentWidgetBlueprintExtension->GetComponentsFor(Widget);
						for (UUIComponent* Component : ComponentsList)
						{
							if (Source->Key.GetClass() == Component->GetClass())
							{
								FName ComponentPropertyName = UUIComponentContainer::GetPropertyNameForComponent(Component, Widget->GetFName());
								if (FProperty* Property = FindFProperty<FObjectPropertyBase>(Cast<UWidgetBlueprintGeneratedClass>(AccessorClass), ComponentPropertyName))
								{
									FMVVMConstFieldVariant Field = FMVVMConstFieldVariant(Property);
									PropertyPath.AppendPropertyPath(WidgetBlueprintPtr, Field);
									CompleteFieldPath.Add(FFieldVariant(Property));
								}
								break;
							}
						}
					}
				}
			}

			for (const FFieldVariant& Field : FieldPath)
			{
				PropertyPath.AppendPropertyPath(WidgetBlueprintPtr, FMVVMConstFieldVariant(Field));
				CompleteFieldPath.Add(Field);
			}

			// Determine if this property path holds a valid registered UI component.
			PropertyPath.SetIsComponent(IsPropertyPathComponent(CompleteFieldPath, PropertyPath, WidgetBlueprintPtr));
		}
	}
	else if(AccessorClass)
	{
		FMVVMBindingName BindingName = Source->Key.ToBindingName(WidgetBlueprintPtr);
		FMVVMAvailableBinding Binding = bIsBindingToEvent ? UMVVMSubsystem::GetAvailableBindingForEvent(AccessorClass, BindingName, AccessorClass) : UMVVMSubsystem::GetAvailableBinding(AccessorClass, BindingName, AccessorClass);
		if (Binding.IsValid())
		{
			EFilterFlag FilterFlags;
			if (UMVVMWidgetBlueprintExtension_View* ExtensionView = UMVVMWidgetBlueprintExtension_View::GetExtension<UMVVMWidgetBlueprintExtension_View>(WidgetBlueprint.Get()))
			{
				FilterFlags = ExtensionView->GetFilterSettings().FilterFlags;
			}
			else
			{
				FilterFlags = GetDefault<UMVVMDeveloperProjectSettings>()->FilterSettings.FilterFlags;
			}
			bool bPassFilter;

			if (FilterFlags == EFilterFlag::All)
			{
				const Private::FPassFilterInputs Params(WidgetBlueprint.Get(), Binding, AccessorClass, FieldIterator->GetFieldVisibilityFlags(), FieldIterator->GetAssignableTo(), true, false, IsConversionEnabled());
				bPassFilter = Private::PassFilter(Params).IsSet();
			}
			else
			{
				const Private::FPassFilterInputs Params(WidgetBlueprint.Get(), Binding, AccessorClass, EFieldVisibility::None, nullptr, true, false, IsConversionEnabled());
				bPassFilter = Private::PassFilter(Params).IsSet();
			}
			if (bPassFilter)
			{
				Source->Key.SetSourceTo(PropertyPath);
				PropertyPath.ResetPropertyPath();
			}
		}
	}
	return PropertyPath;
}

TSharedPtr<SWidget> SSourceBindingList::HandleGetPreSlot(SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> FieldPath)
{
	if (FieldPath.Num() > 0)
	{
		return ConstructFieldPreSlot(WidgetBlueprint.Get(), Handle, FieldPath.Last(), bIsBindingToEvent);
	}
	return TSharedPtr<SWidget>();
}

void SSourceBindingList::HandleSelectionChanged(SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> FieldPath, ESelectInfo::Type SelectionType)
{
	SelectedPath = CreateBlueprintPropertyPath(Handle, FieldPath);
}

void SSourceBindingList::HandleDoubleClicked(SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> FieldPath)
{
	if (OnDoubleClicked.IsBound())
	{
		FMVVMBlueprintPropertyPath ClickedPath = CreateBlueprintPropertyPath(Handle, FieldPath);
		OnDoubleClicked.Execute(ClickedPath);
	}
}

void SSourceBindingList::SetRawFilterText(const FText& InFilterText)
{
	if (PropertyViewer)
	{
		PropertyViewer->SetRawFilterText(InFilterText);
	}
}

void SSourceBindingList::EnableConversion(bool InEnabled)
{
	if (FieldIterator)
	{
		FieldIterator->EnableConversion(InEnabled);
	}

	Invalidate(EInvalidateWidgetReason::Layout);
}

bool SSourceBindingList::IsConversionEnabled() const
{
	if (FieldIterator)
	{
		return FieldIterator->IsConversionEnabled();
	}

	return false;
}

FMVVMBlueprintPropertyPath SSourceBindingList::GetSelectedProperty() const
{
	return SelectedPath;
}

void SSourceBindingList::SetSelectedProperty(const FMVVMBlueprintPropertyPath& PropertyPath)
{
	if (!PropertyViewer.IsValid())
	{
		return;
	}

	const UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get();
	if (!WidgetBlueprintPtr)
	{
		return;
	}

	SPropertyViewer::FHandle SelectedHandle;
	for (TPair<FBindingSource, SPropertyViewer::FHandle>& Source : Sources)
	{
		if (Source.Key.Matches(WidgetBlueprintPtr, PropertyPath))
		{
			SelectedHandle = Source.Value;
			break;
		}
	}

	FMemMark Mark(FMemStack::Get());
	TArray<FFieldVariant, TMemStackAllocator<>> FieldPath;
	if (SelectedHandle.IsValid())
	{
		TArray<FMVVMConstFieldVariant> FieldVariants = PropertyPath.GetFields(WidgetBlueprintPtr->SkeletonGeneratedClass);
		FieldPath.Reserve(FieldVariants.Num());

		for (const FMVVMConstFieldVariant& Variant : FieldVariants)
		{
			FFieldVariant& Field = FieldPath.AddDefaulted_GetRef();
			if (Variant.IsFunction())
			{
				Field = FFieldVariant(Variant.GetFunction());
			}
			else if (Variant.IsProperty())
			{
				Field = FFieldVariant(Variant.GetProperty());
			}
		}
	}
	
	PropertyViewer->SetSelection(SelectedHandle, FieldPath);
}

TArray<FMVVMFieldConversion> SSourceBindingList::GetSelectedConversions() const
{
	TArray<FMVVMFieldConversion> SelectedConversions;

	if (!PropertyViewer.IsValid())
	{
		return SelectedConversions;
	}

	const UWidgetBlueprint* WidgetBlueprintPtr = WidgetBlueprint.Get();
	if (!WidgetBlueprintPtr)
	{
		return SelectedConversions;
	}

	TArray<SPropertyViewer::FSelectedItem> SelectedItems = PropertyViewer->GetSelectedItems();
	for (const SPropertyViewer::FSelectedItem& SelectedItem : SelectedItems)
	{
		if (SelectedItem.Fields.IsEmpty())
		{
			continue;
		}

		for(const TArray<FFieldVariant>& Fields : SelectedItem.Fields)
		{
			if (Fields.IsEmpty())
			{
				continue;
			}

			if (FProperty* Property = CastField<FProperty>(Fields.Last().ToField()))
			{
				TArray<FMVVMFieldConversion> Conversions = Private::GetCompatibleSourceConversions(WidgetBlueprintPtr, Property, FieldIterator->GetAssignableTo(), false);
				SelectedConversions.Append(MoveTemp(Conversions));
			}
		}
	}
	return SelectedConversions;
}

} // namespace UE::MVVM

#undef LOCTEXT_NAMESPACE
