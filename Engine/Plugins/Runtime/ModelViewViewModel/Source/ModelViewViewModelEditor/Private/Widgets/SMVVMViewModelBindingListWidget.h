// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/PropertyViewer/IFieldExpander.h"
#include "Framework/PropertyViewer/IFieldIterator.h"
#include "MVVMBlueprintPin.h"
#include "MVVMBlueprintFunctionReference.h"
#include "MVVMPropertyPath.h"
#include "Types/MVVMConversionFunctionValue.h"
#include "Widgets/PropertyViewer/SPropertyViewer.h"

namespace UE::MVVM { struct FBindingSource; }

class UWidgetBlueprint;
class UWidget;

struct FMVVMBlueprintViewModelContext;

namespace UE::MVVM
{
	enum class EFieldVisibility : uint8
	{
		None = 0,
		Readable = 1 << 1,
		Writable = 1 << 2,
		Notify = 1 << 3,
		All = 0xff
	};

	ENUM_CLASS_FLAGS(EFieldVisibility)

	/**
	 * 
	 */
	class FFieldIterator_Bindable : public UE::PropertyViewer::FFieldIterator_BlueprintVisible
	{
	public:
		FFieldIterator_Bindable(const UWidgetBlueprint* WidgetBlueprint, EFieldVisibility InVisibilityFlags, const FProperty* InAssignableTo = nullptr, const bool InIsBindingToEvent = false, const bool bEnableConversion = false);
		virtual TArray<FFieldVariant> GetFields(const UStruct* Struct, const FName FieldName, const UStruct* Container) const override;
		EFieldVisibility GetFieldVisibilityFlags() const
		{
			return FieldVisibilityFlags;
		}
		const FProperty* GetAssignableTo() const
		{
			return AssignableTo;
		}
		bool IsConversionEnabled() const
		{
			return bEnableConversion;
		}
		void EnableConversion(bool InEnabled)
		{
			bEnableConversion = InEnabled;
		}
	private:
		TWeakObjectPtr<const UWidgetBlueprint> WidgetBlueprint; 
		EFieldVisibility FieldVisibilityFlags = EFieldVisibility::All;
		const FProperty* AssignableTo = nullptr;
		bool bIsBindingToEvent = false;
		bool bEnableConversion = false;
	};

	/**
	 *
	 */
	class FFieldExpander_Bindable : public UE::PropertyViewer::FFieldExpander_Default
	{
	public:
		FFieldExpander_Bindable();
		virtual TOptional<const UClass*> CanExpandObject(const FObjectPropertyBase* Property, const UObject* Instance) const override;
		virtual bool CanExpandScriptStruct(const FStructProperty*) const override;
		virtual TOptional<const UStruct*> GetExpandedFunction(const UFunction* Function) const override;
	};

	/** Genereate the icon for the binding type. */
	TSharedRef<SWidget> ConstructFieldPreSlot(const UWidgetBlueprint* WidgetBlueprint, UE::PropertyViewer::SPropertyViewer::FHandle Handle, const FFieldVariant FieldPath, const bool bIsForEvent = false);

	struct FMVVMFieldConversion : TSharedFromThis<FMVVMFieldConversion>
	{
		FMVVMFieldConversion() = default;
		FMVVMFieldConversion(FConversionFunctionValue Function, FMVVMBlueprintPinId ParameterId);

		FConversionFunctionValue Function;
		FMVVMBlueprintPinId ParameterId;

		bool IsValid() const; 
		FName GetFName() const;
		FMVVMBlueprintFunctionReference ToBlueprintFunctionReference(UBlueprint* Blueprint) const;
	};

	/** 
	 * 
	 */
	class SSourceBindingList : public SCompoundWidget
	{
	public:
		DECLARE_DELEGATE_OneParam(FOnDoubleClicked, const FMVVMBlueprintPropertyPath&);
	
		SLATE_BEGIN_ARGS(SSourceBindingList) {}
			SLATE_ARGUMENT_DEFAULT(bool, ShowSearchBox) = false;
			SLATE_ARGUMENT_DEFAULT(EFieldVisibility, FieldVisibilityFlags) = EFieldVisibility::All;
			SLATE_ARGUMENT_DEFAULT(bool, EnableSelection) = true;
			SLATE_ARGUMENT_DEFAULT(bool, IsBindingToEvent) = false;

			/** 
			 * Show only properties that are assignable to the given property. 
			 * This means that they are either the exact same type, or that they are trivially convertible to the target type. 
			 * Eg. bool -> bool, int32 -> int64, float -> double
			 */
			SLATE_ARGUMENT_DEFAULT(const FProperty*, AssignableTo) = nullptr;

			/**
			* If enabled, will show properties that can be converted to AssignableTo. 
			* The compatible conversion functions will be accessible via GetSelectedConversions
			*/
			SLATE_ARGUMENT_DEFAULT(bool, bEnableConversion) = false;

			SLATE_EVENT(FOnDoubleClicked, OnDoubleClicked)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const UWidgetBlueprint* WidgetBlueprint);

		void ClearSources();

		void AddSource(const FBindingSource& InSource);
		void AddSources(TArrayView<const FBindingSource> InSources);

		void AddWidgetBlueprint();
		void AddWidgets(TArrayView<const UWidget*> Widgets);
		void AddViewModels(TArrayView<const FMVVMBlueprintViewModelContext> ViewModels);

		FMVVMBlueprintPropertyPath GetSelectedProperty() const; 
		void SetSelectedProperty(const FMVVMBlueprintPropertyPath& Property);

		TArray<FMVVMFieldConversion> GetSelectedConversions() const;

		void SetRawFilterText(const FText& InFilterText);

		void EnableConversion(bool InEnabled);
		bool IsConversionEnabled() const;

	private:
		using SPropertyViewer = UE::PropertyViewer::SPropertyViewer;

		TSharedPtr<SWidget> HandleGetPreSlot(SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> Path);
		void HandleSelectionChanged(SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> Path, ESelectInfo::Type SelectionType);
		void HandleDoubleClicked(SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> Path);

		FMVVMBlueprintPropertyPath CreateBlueprintPropertyPath(SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> Path) const;

	private:
		FOnDoubleClicked OnDoubleClicked;
		TUniquePtr<FFieldIterator_Bindable> FieldIterator;
		TUniquePtr<UE::PropertyViewer::FFieldExpander_Default> FieldExpander;
		using FSourceType = TTuple<FBindingSource, SPropertyViewer::FHandle>;
		TArray<TPair<FBindingSource, SPropertyViewer::FHandle>> Sources;
		FMVVMBlueprintPropertyPath SelectedPath;
		TSharedPtr<SPropertyViewer> PropertyViewer;
		TWeakObjectPtr<const UWidgetBlueprint> WidgetBlueprint;
		bool bIsBindingToEvent = false;
	};

} //namespace UE::MVVM
