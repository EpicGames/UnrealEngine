// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMViewModelPanel.h"

#include "DetailsViewArgs.h"
#include "Dialogs/Dialogs.h"
#include "IModelViewViewModelEditorModule.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "MVVMBlueprintInstancedViewModel.h"
#include "MVVMBlueprintView.h"
#include "MVVMDeveloperProjectSettings.h"
#include "MVVMEditorCommands.h"
#include "MVVMEditorSubsystem.h"
#include "MVVMSubsystem.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "View/MVVMView.h"
#include "View/MVVMViewModelContextResolver.h"
#include "ViewModelFieldDragDropOp.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"
#include "WidgetBlueprintToolMenuContext.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/MVVMEditorStyle.h"
#include "Styling/ToolBarStyle.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/PropertyViewer/SFieldIcon.h"
#include "Widgets/SMVVMPopoutEditor.h"
#include "Widgets/SMVVMSelectViewModel.h"
#include "Widgets/SMVVMViewModelBindingListWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "SPositiveActionButton.h"
#include "SWarningOrErrorBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMVVMViewModelPanel)

#define LOCTEXT_NAMESPACE "ViewModelPanel"


void UMVVMBlueprintViewModelContextWrapper::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (UMVVMBlueprintView* BlueprintViewPtr = BlueprintView.Get())
	{
		if (FMVVMBlueprintViewModelContext* ViewModelContextPtr = BlueprintViewPtr->FindViewModel(ViewModelId))
		{
			if (Wrapper.Resolver && (Wrapper.Resolver->GetOuter() == this || Wrapper.Resolver->GetOuter() == GetTransientPackage()))
			{
				Wrapper.Resolver->Rename(nullptr, BlueprintViewPtr);
			}
			*ViewModelContextPtr = Wrapper;
		}
	}
}

namespace UE::MVVM::Private
{

void SetSelectObjectsToView(TWeakPtr<FWidgetBlueprintEditor> WeakEditor)
{
	if (TSharedPtr<FWidgetBlueprintEditor> Editor = WeakEditor.Pin())
	{
		UMVVMEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		if (!Subsystem)
		{
			return;
		}

		if (UMVVMBlueprintView* BlueprintView = Subsystem->GetView(Editor->GetWidgetBlueprintObj()))
		{
			Editor->CleanSelection();
			TSet<UObject*> Selections;
			Selections.Add(BlueprintView->GetSettings());
			Editor->SelectObjects(Selections);
		}
	}
}

void CreateViewModelInstance(TWeakPtr<FWidgetBlueprintEditor> WeakEditor)
{
	if (TSharedPtr<FWidgetBlueprintEditor> Editor = WeakEditor.Pin())
	{
		UMVVMEditorSubsystem* MVVMEditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		if (!MVVMEditorSubsystem)
		{
			return;
		}

		UMVVMBlueprintView* BlueprintView = MVVMEditorSubsystem->RequestView(Editor->GetWidgetBlueprintObj());
		MVVMEditorSubsystem->AddInstancedViewModel(Editor->GetWidgetBlueprintObj());
	}
}

} //namespace UE::MVVM::Private

namespace UE::MVVM
{

void SMVVMViewModelPanel::RegisterMenu()
{
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("MVVM.Viewmodels.Toolbar");
		Menu->MenuType = EMultiBoxType::SlimHorizontalToolBar;
		FToolMenuSection& Section = Menu->FindOrAddSection("Left");
		Section.AddDynamicEntry("AddViewmodel", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UMVVMViewModelPanelToolMenuContext* Context = InSection.FindContext<UMVVMViewModelPanelToolMenuContext>())
				{
					if (TSharedPtr<SMVVMViewModelPanel> ViewModelPanel = Context->ViewModelPanel.Pin())
					{
						ViewModelPanel->BuildContextMenu(InSection);
					}
				}
			}));
	} {
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("MVVM.Viewmodels.Add");
		FToolMenuSection& Section = Menu->FindOrAddSection("Main");
		Section.AddDynamicEntry("Main", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UWidgetBlueprintToolMenuContext* Context = InSection.FindContext<UWidgetBlueprintToolMenuContext>())
				{
					if (GetDefault<UMVVMDeveloperProjectSettings>()->bCanCreateViewModelInView)
					{
						InSection.AddMenuEntry(
							"CreateViewmodel"
							, LOCTEXT("CreateViewmodel", "Instanced Viewmodel")
							, LOCTEXT("CreateViewmodelTooltip", "Create a Viewmodel inside the View. The Viewmodel is not accessible from outside the View.")
							, FSlateIcon(FMVVMEditorStyle::Get().GetStyleSetName(), "BlueprintView.TabIcon")
							, FUIAction(FExecuteAction::CreateStatic(UE::MVVM::Private::CreateViewModelInstance, Context->WidgetBlueprintEditor))
							, EUserInterfaceActionType::Button
						);
					}
				}
			}));
	}
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("MVVM.Viewmodels.Settings");
		FToolMenuSection& Section = Menu->FindOrAddSection("Main");
		Section.AddDynamicEntry("Main", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UWidgetBlueprintToolMenuContext* Context = InSection.FindContext<UWidgetBlueprintToolMenuContext>())
				{
					if (GetDefault<UMVVMDeveloperProjectSettings>()->bShowViewSettings)
					{
						InSection.AddMenuEntry(
							"ViewSettings"
							, LOCTEXT("ViewSettings", "View Settings")
							, LOCTEXT("ViewSettingsTooltip", "View Settings")
							, FSlateIcon(FMVVMEditorStyle::Get().GetStyleSetName(), "BlueprintView.TabIcon")
							, FUIAction(FExecuteAction::CreateStatic(UE::MVVM::Private::SetSelectObjectsToView, Context->WidgetBlueprintEditor))
							, EUserInterfaceActionType::Button
						);
					}
				}
			}));
	}
}


void SMVVMViewModelPanel::BuildContextMenu(FToolMenuSection& InSection)
{
	SAssignNew(AddMenuButton, SPositiveActionButton)
		.OnGetMenuContent(this, &SMVVMViewModelPanel::MakeAddMenu)
		.Text(LOCTEXT("Viewmodel", "Viewmodel"))
		.IsEnabled(this, &SMVVMViewModelPanel::HandleCanEditViewmodelList);

	InSection.AddEntry(FToolMenuEntry::InitWidget("AddViewmodel", AddMenuButton.ToSharedRef(), FText()));

	const FToolBarStyle& ToolBarStyle = FCoreStyle::Get().GetWidgetStyle<FToolBarStyle>("CalloutToolbar");
	const FVector2f IconSize = ToolBarStyle.IconSize;
	const FComboButtonStyle* ComboStyle = &ToolBarStyle.SettingsComboButton;
	const FButtonStyle* ButtonStyle = &ComboStyle->ButtonStyle;
	FSlateColor OpenForegroundColor = ButtonStyle->HoveredForeground;

	TWeakPtr<SComboButton> WeakComboBox;
	TSharedRef<SWidget> ComboBottom = SAssignNew(WeakComboBox, SComboButton)
		.ContentPadding(0.f)
		.ComboButtonStyle(ComboStyle)
		.ButtonStyle(ButtonStyle)
		.ForegroundColor_Lambda([WeakComboBox, OpenForegroundColor]()
		{
			TSharedPtr<SComboButton> LocalComboButton = WeakComboBox.Pin();
			return LocalComboButton ? OpenForegroundColor : FSlateColor::UseStyle();
		})
		// Route the content generator event
		.OnGetMenuContent(this, &SMVVMViewModelPanel::HandleAddViewModelContextMenu)
		.IsEnabled(this, &SMVVMViewModelPanel::HandleCanEditViewmodelList)
		.ButtonContent()
		[
			SNullWidget::NullWidget
		];
	InSection.AddEntry(FToolMenuEntry::InitWidget("AddViewmodelContext", ComboBottom, FText()));
}


void SMVVMViewModelPanel::Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor)
{
	UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj();
	check(WidgetBlueprint);
	UMVVMBlueprintView* CurrentBlueprintView = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->GetView(WidgetBlueprint);

	const UMVVMDeveloperProjectSettings* MVVMSettings = GetDefault<UMVVMDeveloperProjectSettings>();
	const bool NeedToRequestView = MVVMSettings->bAllowBindingEditingWithoutViewModel;
	if (NeedToRequestView)
	{	//Make a view available from start without waiting for a view model to be added.
		CurrentBlueprintView = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->RequestView(WidgetBlueprint);
	}

	WeakBlueprintEditor = WidgetBlueprintEditor;
	WeakBlueprintView = CurrentBlueprintView;
	FieldIterator = MakeUnique<FFieldIterator_Bindable>(WidgetBlueprint, EFieldVisibility::None);
	FieldExpander = MakeUnique<FFieldExpander_Bindable>();

	WidgetBlueprintEditor->OnSelectedWidgetsChanging.AddSP(this, &SMVVMViewModelPanel::HandleEditorSelectionChanged);
	
	WidgetBlueprint->OnSetObjectBeingDebugged().AddSP(this, &SMVVMViewModelPanel::HandleSetObjectBeingDebugged);

	if (CurrentBlueprintView)
	{
		// Listen to when the viewmodel are modified
		ViewModelsUpdatedHandle = CurrentBlueprintView->OnViewModelsUpdated.AddSP(this, &SMVVMViewModelPanel::HandleViewModelsUpdated);
	}
	else
	{
		ExtensionAddeddHandle = WidgetBlueprint->OnExtensionAdded.AddSP(this, &SMVVMViewModelPanel::HandleViewUpdated);
	}

	WidgetBlueprintTransactionHandle = WidgetBlueprintEditor->GetOnWidgetBlueprintTransaction().AddSP(this, &SMVVMViewModelPanel::FillViewModel);

	CreateCommandList();

	ModelContextWrapper.Reset(NewObject<UMVVMBlueprintViewModelContextWrapper>());

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.Padding(0)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SAssignNew(ViewModelTreeViewContainer, SBox)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 8.0f, 0.0f, 4.0f))
			[
				SNew(SWarningOrErrorBox)
				.Visibility(this, &SMVVMViewModelPanel::GetWarningPanelVisibility)
				.MessageStyle(EMessageStyle::Warning)
				.Message(this, &SMVVMViewModelPanel::GetWarningMessage)
				[
					SNew(SButton)
					.OnClicked(this, &SMVVMViewModelPanel::HandleDisableWarningPanel)
					.TextStyle(FAppStyle::Get(), "DialogButtonText")
					.Text(LOCTEXT("WarningDisable", "Dismiss"))
				]
			]
		]
	];

	HandleSetObjectBeingDebugged(WidgetBlueprint->GetObjectBeingDebugged());
}

SMVVMViewModelPanel::SMVVMViewModelPanel()
	: PropertyVisibility(UE::PropertyViewer::SPropertyViewer::EPropertyVisibility::Hidden)
{}

SMVVMViewModelPanel::~SMVVMViewModelPanel()
{
	TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin();

	if (ExtensionAddeddHandle.IsValid() && WidgetBlueprintEditor.IsValid())
	{
		if (UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj())
		{
			WidgetBlueprint->OnExtensionAdded.Remove(ExtensionAddeddHandle);
		}
	}

	if (ViewModelsUpdatedHandle.IsValid())
	{
		if (UMVVMBlueprintView* CurrentBlueprintView = WeakBlueprintView.Get())
		{
			// bind to check if the view is enabled
			CurrentBlueprintView->OnViewModelsUpdated.Remove(ViewModelsUpdatedHandle);
		}
	}

	if (WidgetBlueprintTransactionHandle.IsValid() && WidgetBlueprintEditor.IsValid())
	{
		WidgetBlueprintEditor->GetOnWidgetBlueprintTransaction().Remove(WidgetBlueprintTransactionHandle);
	}

	if (UObjectInitialized() && WidgetBlueprintEditor.IsValid())
	{
		if (UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj())
		{
			WidgetBlueprint->OnSetObjectBeingDebugged().RemoveAll(this);
		}
	}
}


void SMVVMViewModelPanel::FillViewModel()
{
	PropertyViewerHandles.Reset();
	EditableTextBlocks.Reset();
	SelectedViewModelGuid = FGuid();

	if (ViewModelTreeView)
	{
		ViewModelTreeView->RemoveAll();

		UUserWidget* Widget = nullptr;
		UMVVMView* WidgetView = nullptr;

		if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin())
		{
			if (UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj())
			{
				Widget = Cast<UUserWidget>(WidgetBlueprint->GetObjectBeingDebugged());

				if (Widget)
				{
					WidgetView = UMVVMSubsystem::GetViewFromUserWidget(Widget);
				}
			}
		}

		if (UMVVMBlueprintView* View = WeakBlueprintView.Get())
		{
			const TArrayView<const FMVVMBlueprintViewModelContext> ViewModelContexts = View->GetViewModels();

			for (const FMVVMBlueprintViewModelContext& ViewModelContext : ViewModelContexts)
			{
				UE::PropertyViewer::SPropertyViewer::FHandle Handle;
				UClass* ViewModelClass = ViewModelContext.GetViewModelClass();

				if (Widget)
				{
					if (WidgetView)
					{
						if (UObject* Object = WidgetView->GetViewModel(ViewModelContext.GetViewModelName()).GetObject())
						{
							Handle = ViewModelTreeView->AddInstance(Object);
						}
					}
				}
				
				if (!Handle.IsValid())
				{
					Handle = ViewModelTreeView->AddContainer(ViewModelClass);
				}

				if (Handle.IsValid())
				{
					PropertyViewerHandles.Add(Handle, ViewModelContext.GetViewModelId());
				}
			}
		}

		// Only add widget values when in editable mode.
		if (Widget && PropertyVisibility == UE::PropertyViewer::SPropertyViewer::EPropertyVisibility::Editable)
		{
			UE::PropertyViewer::SPropertyViewer::FHandle Handle = ViewModelTreeView->AddInstance(Widget);

			if (Handle.IsValid())
			{
				// Empty GUID for the widget instance.
				PropertyViewerHandles.Add(Handle, FGuid());
			}
		}
	}
}


FReply SMVVMViewModelPanel::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}


void SMVVMViewModelPanel::HandleViewUpdated(UBlueprintExtension*)
{
	bool bViewUpdated = false;

	if (!ViewModelsUpdatedHandle.IsValid())
	{
		if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin())
		{
			if (UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj())
			{
				UMVVMBlueprintView* CurrentBlueprintView = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->GetView(WidgetBlueprint);
				if (CurrentBlueprintView == nullptr)
				{
					CurrentBlueprintView = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->RequestView(WidgetBlueprint);
				}
				WeakBlueprintView = CurrentBlueprintView;

				if (ensure(CurrentBlueprintView))
				{
					ViewModelsUpdatedHandle = CurrentBlueprintView->OnViewModelsUpdated.AddSP(this, &SMVVMViewModelPanel::HandleViewModelsUpdated);

					bViewUpdated = true;

					WidgetBlueprint->OnExtensionAdded.Remove(ExtensionAddeddHandle);
					ExtensionAddeddHandle.Reset();
				}
			}
		}
	}

	if (bViewUpdated)
	{
		FillViewModel();
	}
}


void SMVVMViewModelPanel::HandleViewModelsUpdated()
{
	FillViewModel();
}

TSet<UObject*> SMVVMViewModelPanel::GetSelectedEditableObjects() const
{
	if (!ViewModelTreeView.IsValid())
	{
		return {};
	}

	TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin();
	if (!WidgetBlueprintEditor)
	{
		return {};
	}

	UWidgetBlueprint* WidgetBP = WidgetBlueprintEditor->GetWidgetBlueprintObj();
	if (!WidgetBP)
	{
		return {};
	}

	UMVVMBlueprintView* BlueprintView = WeakBlueprintView.Get();
	if (!BlueprintView)
	{
		return {};
	}

	UUserWidget* Widget = Cast<UUserWidget>(WidgetBP->GetObjectBeingDebugged());
	if (!Widget)
	{
		return {};
	}

	UMVVMView* WidgetView = UMVVMSubsystem::GetViewFromUserWidget(Widget);

	TArray<UE::PropertyViewer::SPropertyViewer::FSelectedItem> Items = ViewModelTreeView->GetSelectedItems();
	TSet<UObject*> Selected;

	for (const UE::PropertyViewer::SPropertyViewer::FSelectedItem& Item : Items)
	{
		if (Item.bIsContainerSelected)
		{
			if (const FGuid* VMGuidPtr = PropertyViewerHandles.Find(Item.Handle))
			{
				// The widget instance itself is 0,0,0,0 = invalid.
				if (!VMGuidPtr->IsValid())
				{
					Selected.Add(Widget);
				}
				else if (WidgetView)
				{
					if (const FMVVMBlueprintViewModelContext* ViewModelContext = BlueprintView->FindViewModel(*VMGuidPtr))
					{
						if (UObject* Object = WidgetView->GetViewModel(ViewModelContext->GetViewModelName()).GetObject())
						{
							Selected.Add(Object);
						}
					}
				}
			}
		}
	}

	return Selected;
}

bool SMVVMViewModelPanel::HandleCanPopoutEditor() const
{
	return !GetSelectedEditableObjects().IsEmpty();
}

void SMVVMViewModelPanel::HandlePopoutEditor()
{
	IModelViewViewModelEditorModule& Module = FModuleManager::Get().GetModuleChecked<IModelViewViewModelEditorModule>("ModelViewViewModelEditor");

	for (UObject* Object : GetSelectedEditableObjects())
	{
		Module.OpenPopoutEditor(Object, /* Read Only */ false);
	}
}

void SMVVMViewModelPanel::OpenAddViewModelMenu()
{
	AddMenuButton->SetIsMenuOpen(true, true);
}


TSharedRef<SWidget> SMVVMViewModelPanel::MakeAddMenu()
{
	const UWidgetBlueprint* WidgetBlueprint = nullptr;
	if (TSharedPtr<FWidgetBlueprintEditor> EditorPin = WeakBlueprintEditor.Pin())
	{
		WidgetBlueprint = EditorPin->GetWidgetBlueprintObj();
	}

	return SNew(SBox)
		.WidthOverride(600)
		.HeightOverride(500)
		[
			SNew(SMVVMSelectViewModel, WidgetBlueprint)
			.OnCancel(this, &SMVVMViewModelPanel::HandleCancelAddMenu)
			.OnViewModelCommitted(this, &SMVVMViewModelPanel::HandleAddMenuViewModel)
			.DisallowedClassFlags(CLASS_HideDropDown | CLASS_Hidden | CLASS_Deprecated | CLASS_NotPlaceable)
		];
}


void SMVVMViewModelPanel::HandleCancelAddMenu()
{
	if (AddMenuButton)
	{
		AddMenuButton->SetIsMenuOpen(false, false);
	}
}

void SMVVMViewModelPanel::HandleAddMenuViewModel(const UClass* SelectedClass)
{
	if (AddMenuButton)
	{
		AddMenuButton->SetIsMenuOpen(false, false);
		if (SelectedClass)
		{
			if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin())
			{
				if (UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj())
				{
					UMVVMEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
					check(EditorSubsystem);
					EditorSubsystem->RequestView(WidgetBlueprint);
					EditorSubsystem->AddViewModel(WidgetBlueprint, SelectedClass);
				}
			}
		}
	}
}


bool SMVVMViewModelPanel::HandleCanEditViewmodelList() const
{
	if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin())
	{
		return WidgetBlueprintEditor->InEditingMode();
	}
	return false;
}


TSharedRef<SWidget> SMVVMViewModelPanel::HandleAddViewModelContextMenu()
{
	FToolMenuContext GenerateWidgetContext;
	{
		UWidgetBlueprintToolMenuContext* WidgetBlueprintMenuContext = NewObject<UWidgetBlueprintToolMenuContext>();
		WidgetBlueprintMenuContext->WidgetBlueprintEditor = WeakBlueprintEditor;
		GenerateWidgetContext.AddObject(WidgetBlueprintMenuContext);

		UMVVMViewModelPanelToolMenuContext* ViewModelPanelToolMenuContext = NewObject<UMVVMViewModelPanelToolMenuContext>();
		ViewModelPanelToolMenuContext->ViewModelPanel = SharedThis(this);
		GenerateWidgetContext.AddObject(ViewModelPanelToolMenuContext);
	}

	return UToolMenus::Get()->GenerateWidget("MVVM.Viewmodels.Add", GenerateWidgetContext);
}


TSharedPtr<SWidget> SMVVMViewModelPanel::HandleGetPreSlot(UE::PropertyViewer::SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> FieldPath)
{
	if (FieldPath.Num() > 0)
	{
		UWidgetBlueprint* WidgetBlueprint = nullptr;
		if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin())
		{
			WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj();
		}

		return ConstructFieldPreSlot(WidgetBlueprint, Handle, FieldPath.Last());
	}

	return TSharedPtr<SWidget>();
}


TSharedRef<SWidget> SMVVMViewModelPanel::HandleGenerateContainer(UE::PropertyViewer::SPropertyViewer::FHandle ContainerHandle, TOptional<FText> DisplayName)
{
	if (const FGuid* VMGuidPtr = PropertyViewerHandles.Find(ContainerHandle))
	{
		UWidgetBlueprint* WidgetBlueprint = nullptr;

		if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin())
		{
			WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj();
		}

		const FGuid VMGuid = *VMGuidPtr;
		// Widget itself, not a view model
		if (!VMGuid.IsValid())
		{
			if (WidgetBlueprint)
			{
				return SNew(STextBlock)
					.Text(FText::FromString(WidgetBlueprint->GetName()));
			}
		}

		if (UMVVMBlueprintView* View = WeakBlueprintView.Get())
		{
			if (const FMVVMBlueprintViewModelContext* ViewModelContext = View->FindViewModel(VMGuid))
			{
				if (ViewModelContext->GetViewModelClass())
				{
					bool bReadOnly = true;

					if (WidgetBlueprint)
					{
						if (UUserWidget* Widget = Cast<UUserWidget>(WidgetBlueprint->GetObjectBeingDebugged()))
						{
							if (UMVVMView* WidgetView = UMVVMSubsystem::GetViewFromUserWidget(Widget))
							{
								if (WidgetView->GetViewModel(ViewModelContext->GetViewModelName()).GetObject())
								{
									bReadOnly = false;
								}
							}
						}
					}

					TSharedRef<SInlineEditableTextBlock> EditableTextBlock = SNew(SInlineEditableTextBlock)
						.Text(ViewModelContext->GetDisplayName())
						.IsReadOnly_Lambda([this, VMGuid]() { return !HandleCanRename(VMGuid); })
						.OnVerifyTextChanged(this, &SMVVMViewModelPanel::HandleVerifyNameTextChanged, VMGuid)
						.OnTextCommitted(this, &SMVVMViewModelPanel::HandleNameTextCommited, VMGuid);
					EditableTextBlocks.Add(VMGuid, EditableTextBlock);

					TSharedRef<SHorizontalBox> Widget = SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						[
							SNew(UE::PropertyViewer::SFieldIcon, ViewModelContext->GetViewModelClass())
						]
						+ SHorizontalBox::Slot()
						.Padding(4.0f)
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							EditableTextBlock
						];

					if (bReadOnly)
					{
						Widget->AddSlot()
							.AutoWidth()
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("ReadOnly", "[Read Only]"))
							];
					}

					return Widget;
				}
				else
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.ErrorWithColor"))
						]
						+ SHorizontalBox::Slot()
						.Padding(4.0f)
						[
							SNew(STextBlock)
							.Text(ViewModelContext->GetDisplayName())
						];
				}
			}
		}
	}

	return SNew(STextBlock)
		.Text(LOCTEXT("ViewmodelError", "Error. Invalid viewmodel."));
}


bool SMVVMViewModelPanel::HandleCanRename(FGuid ViewModelGuid) const
{
	const UMVVMBlueprintView* BlueprintView = WeakBlueprintView.Get();
	const FMVVMBlueprintViewModelContext* ViewModelContext = BlueprintView ? BlueprintView->FindViewModel(ViewModelGuid) : nullptr;
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin();
	return ViewModelContext && ViewModelContext->CanRename() && BlueprintEditor && BlueprintEditor->InEditingMode();
}


bool SMVVMViewModelPanel::HandleVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage, FGuid ViewModelGuid)
{
	return RenameViewModelProperty(ViewModelGuid, InText, false, OutErrorMessage);
}


void SMVVMViewModelPanel::HandleNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo, FGuid ViewModelGuid)
{
	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		FText OutErrorMessage;
		RenameViewModelProperty(ViewModelGuid, InText, true, OutErrorMessage);
	}
}


FReply SMVVMViewModelPanel::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, UE::PropertyViewer::SPropertyViewer::FHandle ContainerHandle, TArrayView<const FFieldVariant> Fields) const
{
	TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin();
	if (WidgetBlueprintEditor == nullptr)
	{
		return FReply::Unhandled();
	}

	UWidgetBlueprint* WidgetBP = WidgetBlueprintEditor->GetWidgetBlueprintObj();
	if (WidgetBP == nullptr)
	{
		return FReply::Unhandled();
	}

	if (ViewModelTreeView.IsValid() && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		if (Fields.Num() > 0)
		{
			if (const FGuid* Id = PropertyViewerHandles.Find(ContainerHandle))
			{
				TArray<FFieldVariant> FieldsArray;
				for (const FFieldVariant& Field : Fields)
				{
					FieldsArray.Add(Field);
				}
				
				return FReply::Handled().BeginDragDrop(FViewModelFieldDragDropOp::New(FieldsArray, *Id, WidgetBP));
			}
		}
	}
	return FReply::Unhandled();
}


void SMVVMViewModelPanel::CreateCommandList()
{
	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SMVVMViewModelPanel::HandleDeleteViewModel),
		FCanExecuteAction::CreateSP(this, &SMVVMViewModelPanel::HandleCanDeleteViewModel)
	);

	CommandList->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SMVVMViewModelPanel::HandleRenameViewModel),
		FCanExecuteAction::CreateSP(this, &SMVVMViewModelPanel::HandleCanRenameViewModel)
	);

	CommandList->MapAction(
		FMVVMEditorCommands::Get().PopoutEditor,
		FExecuteAction::CreateSP(this, &SMVVMViewModelPanel::HandlePopoutEditor),
		FCanExecuteAction::CreateSP(this, &SMVVMViewModelPanel::HandleCanPopoutEditor)
	);
}


namespace Private
{
static bool DisplayInUseWarningAndEarlyExit(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewModelContext* ViewModelContext, const FMVVMBlueprintViewBinding* Binding)
{
	const FText DeleteConfirmationPrompt = FText::Format(LOCTEXT("DeleteConfirmationPrompt", "The viewmodel {0} is in use by binding {1}! Do you really want to delete it?")
		, ViewModelContext->GetDisplayName()
		, Binding ? FText::FromString(Binding->GetDisplayNameString(WidgetBlueprint)) : LOCTEXT("DeleteConfirmation_Unknowed", "Unknowed"));
	const FText DeleteConfirmationTitle = LOCTEXT("DeleteConfirmationTitle", "Delete viewmodel");

	// Warn the user that this may result in data loss
	FSuppressableWarningDialog::FSetupInfo Info(DeleteConfirmationPrompt, DeleteConfirmationTitle, TEXT("Viewmodel_Warning"));
	Info.ConfirmText = LOCTEXT("DeleteConfirmation_Yes", "Yes");
	Info.CancelText = LOCTEXT("DeleteConfirmation_No", "No");

	FSuppressableWarningDialog DeleteFunctionInUse(Info);
	return DeleteFunctionInUse.ShowModal() == FSuppressableWarningDialog::Cancel;
}
}


void SMVVMViewModelPanel::HandleDeleteViewModel()
{
	TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin();
	if (WidgetBlueprintEditor == nullptr)
	{
		return;
	}

	UWidgetBlueprint* WidgetBP = WidgetBlueprintEditor->GetWidgetBlueprintObj();
	if (WidgetBP == nullptr)
	{
		return;
	}

	UMVVMBlueprintView* BlueprintView = WeakBlueprintView.Get();
	if (BlueprintView == nullptr)
	{
		return;
	}

	TArray<UE::PropertyViewer::SPropertyViewer::FSelectedItem> Items = ViewModelTreeView->GetSelectedItems();
	if (Items.Num() == 0)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("RemoveViewModel", "Remove viewmodel"));
	UMVVMBlueprintView* BpView = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->GetView(WidgetBP);
	if (BpView != nullptr)
	{
		BpView->Modify();
	}

	for (UE::PropertyViewer::SPropertyViewer::FSelectedItem& SelectedItem : Items)
	{
		if (SelectedItem.bIsContainerSelected)
		{
			if (FGuid* VMGuidPtr = PropertyViewerHandles.Find(SelectedItem.Handle))
			{
				const FMVVMBlueprintViewModelContext* ViewModelContext = BlueprintView->FindViewModel(*VMGuidPtr);
				if (ViewModelContext && ViewModelContext->bCanRemove)
				{
					FGuid BindingId = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->GetFirstBindingThatUsesViewModel(WidgetBP, *VMGuidPtr);
					if (BindingId.IsValid())
					{
						if (UE::MVVM::Private::DisplayInUseWarningAndEarlyExit(WidgetBP, ViewModelContext, BlueprintView->GetBinding(BindingId)))
						{
							return;
						}
					}

					GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->RemoveViewModel(WidgetBP, ViewModelContext->GetViewModelName());
				}
			}
		}
	}
}


bool SMVVMViewModelPanel::HandleCanDeleteViewModel() const
{
	TArray<UE::PropertyViewer::SPropertyViewer::FSelectedItem> Items = ViewModelTreeView->GetSelectedItems();
	if (Items.Num() == 1 && Items[0].bIsContainerSelected)
	{
		const FGuid* VMGuidPtr = PropertyViewerHandles.Find(Items[0].Handle);
		const UMVVMBlueprintView* BlueprintView = WeakBlueprintView.Get();
		const FMVVMBlueprintViewModelContext* ViewModelContext = BlueprintView && VMGuidPtr ? BlueprintView->FindViewModel(*VMGuidPtr) : nullptr;
		TSharedPtr<FWidgetBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin();
		if (ViewModelContext && ViewModelContext->bCanRemove && BlueprintEditor)
		{
			return BlueprintEditor->InEditingMode() && ViewModelContext->bCanRemove;
		}
	}
	return false;
}


void SMVVMViewModelPanel::HandleRenameViewModel()
{
	TArray<UE::PropertyViewer::SPropertyViewer::FSelectedItem> Items = ViewModelTreeView->GetSelectedItems();
	if (Items.Num() == 1 && Items[0].bIsContainerSelected)
	{
		const FGuid* VMGuidPtr = PropertyViewerHandles.Find(Items[0].Handle);
		if (VMGuidPtr && HandleCanRename(*VMGuidPtr))
		{
			if (TSharedPtr<SInlineEditableTextBlock>* TextBlockPtr = EditableTextBlocks.Find(*VMGuidPtr))
			{
				(*TextBlockPtr)->EnterEditingMode();
			}
		}
	}
}


bool SMVVMViewModelPanel::HandleCanRenameViewModel() const
{
	TArray<UE::PropertyViewer::SPropertyViewer::FSelectedItem> Items = ViewModelTreeView->GetSelectedItems();
	if (Items.Num() == 1 && Items[0].bIsContainerSelected)
	{
		const FGuid* VMGuidPtr = PropertyViewerHandles.Find(Items[0].Handle);
		return VMGuidPtr && HandleCanRename(*VMGuidPtr);
	}
	return false;
}


TSharedPtr<SWidget> SMVVMViewModelPanel::HandleContextMenuOpening(UE::PropertyViewer::SPropertyViewer::FHandle ContainerHandle, TArrayView<const FFieldVariant> Fields) const
{
	if (Fields.Num() == 0 && ContainerHandle.IsValid() )
	{
		FMenuBuilder MenuBuilder(true, CommandList);

		MenuBuilder.BeginSection("Edit", LOCTEXT("Edit", "Edit"));
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
			
			MenuBuilder.AddMenuEntry(
				FMVVMEditorCommands::Get().PopoutEditor,
				NAME_None,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.OpenInExternalEditor")
			);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}
	return TSharedPtr<SWidget>();
}


void SMVVMViewModelPanel::HandleSelectionChanged(UE::PropertyViewer::SPropertyViewer::FHandle ContainerHandle, TArrayView<const FFieldVariant> Fields, ESelectInfo::Type SelectionType)
{
	SelectedViewModelGuid = FGuid();
	ModelContextWrapper->ViewModelId = FGuid();
	ModelContextWrapper->BlueprintView.Reset();

	if (bIsViewModelSelecting)
	{
		return;
	}

	TGuardValue Tmp = TGuardValue(bIsViewModelSelecting, true);

	if (TSharedPtr<FWidgetBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin())
	{
		bool bSet = false;
		if (Fields.Num() == 0)
		{
			if (const FGuid* VMGuidPtr = PropertyViewerHandles.Find(ContainerHandle))
			{
				SelectedViewModelGuid = *VMGuidPtr;

				if (UMVVMBlueprintView* BlueprintView = WeakBlueprintView.Get())
				{
					if (FMVVMBlueprintViewModelContext* ViewModelContext = BlueprintView->FindViewModel(SelectedViewModelGuid))
					{
						ModelContextWrapper->Wrapper = *ViewModelContext;
						ModelContextWrapper->ViewModelId = SelectedViewModelGuid;
						ModelContextWrapper->BlueprintView = WeakBlueprintView;
						
						BlueprintEditor->CleanSelection();

						TSet<UObject*> Selections;
						Selections.Add(ModelContextWrapper.Get());
						BlueprintEditor->SelectObjects(Selections);
						bSet = true;
					}
				}
			}
		}

		if (!bSet && BlueprintEditor->GetSelectedObjects().Contains(ModelContextWrapper.Get()))
		{
			BlueprintEditor->SelectObjects(TSet<UObject*>());
		}
	}
}


void SMVVMViewModelPanel::HandleEditorSelectionChanged()
{
	if (!bIsViewModelSelecting && SelectedViewModelGuid.IsValid())
	{
		ViewModelTreeView->SetSelection(UE::PropertyViewer::SPropertyViewer::FHandle(), TArrayView<const FFieldVariant>());
	}
}

void SMVVMViewModelPanel::HandleSetObjectBeingDebugged(UObject* DebugObject)
{
	using namespace UE::PropertyViewer;

	SPropertyViewer::EPropertyVisibility RequiredPropertyVisibility = SPropertyViewer::EPropertyVisibility::Visible;

	// If there's a debug object, we confirm it's a user widget and set the properties to be editable.
	// If there's not a debug object, leave the properties as visible (not editable).
	if (DebugObject && DebugObject->IsA<UUserWidget>())
	{
		if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin())
		{
			if (DebugObject->GetClass()->ClassGeneratedBy == WidgetBlueprintEditor->GetWidgetBlueprintObj())
			{
				RequiredPropertyVisibility = SPropertyViewer::EPropertyVisibility::Editable;
			}
		}
	}

	// Cannot change this after creation
	if (PropertyVisibility != RequiredPropertyVisibility)
	{
		CreateViewModelTreeView(RequiredPropertyVisibility);
	}

	FillViewModel();
}


bool SMVVMViewModelPanel::RenameViewModelProperty(FGuid ViewModelGuid, const FText& RenameTo, bool bCommit, FText& OutErrorMessage) const
{
	if (RenameTo.IsEmptyOrWhitespace())
	{
		OutErrorMessage = LOCTEXT("EmptyViewModelName", "Empty viewmodel name");
		return false;
	}

	const FString& NewNameString = RenameTo.ToString();
	if (NewNameString.Len() >= NAME_SIZE)
	{
		OutErrorMessage = LOCTEXT("ViewModelNameTooLong", "Viewmodel Name is Too Long");
		return false;
	}

	FString GeneratedName = SlugStringForValidName(NewNameString);
	if (GeneratedName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyViewModelName", "Empty viewmodel name");
		return false;
	}

	const FName GeneratedFName(*GeneratedName);
	check(GeneratedFName.IsValidXName(INVALID_OBJECTNAME_CHARACTERS));

	if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin())
	{
		if (UWidgetBlueprint* WidgetBP = WidgetBlueprintEditor->GetWidgetBlueprintObj())
		{
			if (UMVVMBlueprintView* View = WeakBlueprintView.Get())
			{
				if (const FMVVMBlueprintViewModelContext* ViewModelContext = View->FindViewModel(ViewModelGuid))
				{
					if (ViewModelContext->CanRename())
					{
						if (bCommit)
						{
							return GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->RenameViewModel(WidgetBP, ViewModelContext->GetViewModelName(), *NewNameString, OutErrorMessage);
						}
						else
						{
							return GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->VerifyViewModelRename(WidgetBP, ViewModelContext->GetViewModelName(), *NewNameString, OutErrorMessage);
						}
					}
				}
			}
		}
	}
	return false;
}

void SMVVMViewModelPanel::CreateViewModelTreeView(UE::PropertyViewer::SPropertyViewer::EPropertyVisibility InPropertyVisibility)
{
	PropertyVisibility = InPropertyVisibility;

	if (!ToolbarWidget.IsValid() || !SettingsWidget.IsValid())
	{
		FToolMenuContext GenerateWidgetContext;
		{
			UWidgetBlueprintToolMenuContext* WidgetBlueprintMenuContext = NewObject<UWidgetBlueprintToolMenuContext>();
			WidgetBlueprintMenuContext->WidgetBlueprintEditor = WeakBlueprintEditor;
			GenerateWidgetContext.AddObject(WidgetBlueprintMenuContext);

			UMVVMViewModelPanelToolMenuContext* ViewModelPanelToolMenuContext = NewObject<UMVVMViewModelPanelToolMenuContext>();
			ViewModelPanelToolMenuContext->ViewModelPanel = SharedThis(this);
			GenerateWidgetContext.AddObject(ViewModelPanelToolMenuContext);
		}

		ToolbarWidget = UToolMenus::Get()->GenerateWidget("MVVM.Viewmodels.Toolbar", GenerateWidgetContext);
		SettingsWidget = UToolMenus::Get()->GenerateWidget("MVVM.Viewmodels.Settings", GenerateWidgetContext);
	}

	ViewModelTreeView = SNew(UE::PropertyViewer::SPropertyViewer)
		.PropertyVisibility(InPropertyVisibility)
		.bShowSearchBox(true)
		.bShowFieldIcon(true)
		.bSanitizeName(true)
		.FieldIterator(FieldIterator.Get())
		.FieldExpander(FieldExpander.Get())
		.OnGetPreSlot(this, &SMVVMViewModelPanel::HandleGetPreSlot)
		.OnContextMenuOpening(this, &SMVVMViewModelPanel::HandleContextMenuOpening)
		.OnSelectionChanged(this, &SMVVMViewModelPanel::HandleSelectionChanged)
		.OnGenerateContainer(this, &SMVVMViewModelPanel::HandleGenerateContainer)
		.OnDragDetected(this, &SMVVMViewModelPanel::HandleDragDetected)
		.SearchBoxPreSlot()
		[
			ToolbarWidget.ToSharedRef()
		]
		.SearchBoxPostSlot()
		[
			SNew(SComboButton)
			.HasDownArrow(false)
			.ContentPadding(0)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.MenuContent()
			[
				SettingsWidget.ToSharedRef()
			]
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("DetailsView.ViewOptions"))
			]
		];

	ViewModelTreeViewContainer->SetContent(ViewModelTreeView.ToSharedRef());
}


EVisibility SMVVMViewModelPanel::GetWarningPanelVisibility() const
{
	if (GetWarningMessage().IsEmpty())
	{
		return EVisibility::Collapsed;
	}
	return EVisibility::Visible;
}


FText SMVVMViewModelPanel::GetWarningMessage() const
{
	if (!bDisableWarningPanel)
	{
		if (UMVVMBlueprintView* WidgetBlueprint = WeakBlueprintView.Get())
		{
			if (WidgetBlueprint->GetNumBindings() == 0 && WidgetBlueprint->GetEvents().Num() == 0 && WidgetBlueprint->GetViewModels().Num() != 0 && !WidgetBlueprint->GetSettings()->bCreateViewWithoutBindings)
			{
				return LOCTEXT("NoBindingWarningDescription", "No view will be created for this widget. There are no bindings and there are no events. Your viewmodels won't be initialized. If you want to create the view anyway, choose the Create View Without Bindings option from the View Settings.");
			}
			if (!WidgetBlueprint->GetSettings()->bInitializeSourcesOnConstruct || !WidgetBlueprint->GetSettings()->bInitializeBindingsOnConstruct)
			{
				return LOCTEXT("InitializationPanelWarningDescription", "The view will not initialize automatically. It was manually set in the View Settings.");
			}
		}
	}
	return FText::GetEmpty();
}


FReply SMVVMViewModelPanel::HandleDisableWarningPanel()
{
	bDisableWarningPanel = true;
	return FReply::Handled();
}

namespace Private
{
	static FName NAME_ViewModelName = TEXT("ViewModelName");
}

void SMVVMViewModelPanel::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	PreviousViewModelPropertyName = FName();
	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == Private::NAME_ViewModelName && SelectedViewModelGuid.IsValid())
	{
		if (UMVVMBlueprintView* BlueprintView = WeakBlueprintView.Get())
		{
			if (const FMVVMBlueprintViewModelContext* ViewModelContext = BlueprintView->FindViewModel(SelectedViewModelGuid))
			{
				PreviousViewModelPropertyName = ViewModelContext->GetViewModelName();
			}
		}
	}
}


void SMVVMViewModelPanel::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (PropertyThatChanged && PropertyThatChanged->GetFName() == Private::NAME_ViewModelName && SelectedViewModelGuid.IsValid())
	{
		UMVVMBlueprintView* BlueprintView = WeakBlueprintView.Get();
		TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakBlueprintEditor.Pin();
		UWidgetBlueprint* WidgetBP = WidgetBlueprintEditor ? WidgetBlueprintEditor->GetWidgetBlueprintObj() : nullptr;
		if (BlueprintView && WidgetBP)
		{
			if (FMVVMBlueprintViewModelContext* ViewModelContext = BlueprintView->FindViewModel(SelectedViewModelGuid))
			{
				FName NewName = ViewModelContext->GetViewModelName();
				ViewModelContext->ViewModelName = PreviousViewModelPropertyName;
				FText OutErrorMessage;
				GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>()->RenameViewModel(WidgetBP, PreviousViewModelPropertyName, NewName, OutErrorMessage);
			}
		}
	}
}

} // namespace

#undef LOCTEXT_NAMESPACE
