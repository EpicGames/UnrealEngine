// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Misc/NotifyHook.h"
#include "Widgets/PropertyViewer/SPropertyViewer.h"

#include "Containers/ContainersFwd.h"
#include "MVVMBlueprintViewModelContext.h"
#include "UObject/StrongObjectPtr.h"

#include "SMVVMViewModelPanel.generated.h"

namespace ETextCommit { enum Type : int; }
namespace UE::MVVM { class FFieldIterator_Bindable; }
namespace UE::MVVM { class SMVVMViewModelPanel; }
namespace UE::PropertyViewer { class FFieldExpander_Default; }

class INotifyFieldValueChanged;
class FWidgetBlueprintEditor;
class SBox;
class SInlineEditableTextBlock;
class SPositiveActionButton;
class UBlueprintExtension;
class UMVVMBlueprintView;
class FUICommandList;
class IDetailsView;
struct FToolMenuSection;

UCLASS()
class UMVVMBlueprintViewModelContextWrapper : public UObject
{
	GENERATED_BODY()

public:
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	UPROPERTY(EditAnywhere, Category="Viewmodel", meta = (ShowOnlyInnerProperties))
	FMVVMBlueprintViewModelContext Wrapper;

	TWeakObjectPtr<UMVVMBlueprintView> BlueprintView;
	FGuid ViewModelId;
};

UCLASS()
class UMVVMViewModelPanelToolMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<UE::MVVM::SMVVMViewModelPanel> ViewModelPanel;
};


namespace UE::MVVM
{

/**
 *
 */
class SMVVMViewModelPanel : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SMVVMViewModelPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> Editor);
	SMVVMViewModelPanel();
	virtual ~SMVVMViewModelPanel();

	void OpenAddViewModelMenu();

	static void RegisterMenu();

private:
	void BuildContextMenu(FToolMenuSection& InSection);

	void HandleViewUpdated(UBlueprintExtension* Extension);
	void HandleViewModelsUpdated();
	TSet<UObject*> GetSelectedEditableObjects() const;
	bool HandleCanPopoutEditor() const;
	void HandlePopoutEditor();

	TSharedRef<SWidget> MakeAddMenu();
	void HandleCancelAddMenu();
	void HandleAddMenuViewModel(const UClass* SelectedClass);
	TSharedPtr<SWidget> HandleGetPreSlot(UE::PropertyViewer::SPropertyViewer::FHandle Handle, TArrayView<const FFieldVariant> FieldPath);
	bool HandleCanEditViewmodelList() const;
	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	TSharedRef<SWidget> HandleGenerateContainer(UE::PropertyViewer::SPropertyViewer::FHandle ContainerHandle, TOptional<FText> DisplayName);
	TSharedPtr<SWidget> HandleContextMenuOpening(UE::PropertyViewer::SPropertyViewer::FHandle ContainerHandle, TArrayView<const FFieldVariant> Field) const;
	void HandleSelectionChanged(UE::PropertyViewer::SPropertyViewer::FHandle, TArrayView<const FFieldVariant>, ESelectInfo::Type);
	void HandleEditorSelectionChanged();
	void HandleSetObjectBeingDebugged(UObject* DebugObject);
	bool HandleCanRename(FGuid ViewModelGuid) const;
	bool HandleVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage, FGuid ViewModelGuid);
	void HandleNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo, FGuid ViewModelGuid);
	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, UE::PropertyViewer::SPropertyViewer::FHandle ContainerHandle, TArrayView<const FFieldVariant> Fields) const;

	void HandleDeleteViewModel();
	bool HandleCanDeleteViewModel() const;
	void HandleRenameViewModel();
	bool HandleCanRenameViewModel() const;
	TSharedRef<SWidget> HandleAddViewModelContextMenu();

	EVisibility GetWarningPanelVisibility() const;
	FText GetWarningMessage() const;
	FReply HandleDisableWarningPanel();

	void CreateCommandList();
	void FillViewModel();
	bool RenameViewModelProperty(FGuid ViewModelGuid, const FText& RenameTo, bool bCommit, FText& OutErrorMessage) const;

	void CreateViewModelTreeView(UE::PropertyViewer::SPropertyViewer::EPropertyVisibility InPropertyVisibility);

	//~ Begin FNotifyHook
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	//~ End FNotifyHook

private:
	TSharedPtr<SPositiveActionButton> AddMenuButton;
	TSharedPtr<SBox> ViewModelTreeViewContainer;
	TSharedPtr<UE::PropertyViewer::SPropertyViewer> ViewModelTreeView;
	TUniquePtr<FFieldIterator_Bindable> FieldIterator;
	TUniquePtr<UE::PropertyViewer::FFieldExpander_Default> FieldExpander;
	TSharedPtr<FUICommandList> CommandList;

	TMap<UE::PropertyViewer::SPropertyViewer::FHandle, FGuid> PropertyViewerHandles;
	TMap<FGuid, TSharedPtr<SInlineEditableTextBlock>> EditableTextBlocks;

	TWeakPtr<FWidgetBlueprintEditor> WeakBlueprintEditor;
	TWeakObjectPtr<UMVVMBlueprintView> WeakBlueprintView;
	FDelegateHandle ViewModelsUpdatedHandle;
	FDelegateHandle ExtensionAddeddHandle;
	FDelegateHandle WidgetBlueprintTransactionHandle;

	TStrongObjectPtr<UMVVMBlueprintViewModelContextWrapper> ModelContextWrapper;

	FGuid SelectedViewModelGuid;
	FName PreviousViewModelPropertyName;
	bool bIsViewModelSelecting = false;
	bool bDisableWarningPanel = false;

	UE::PropertyViewer::SPropertyViewer::EPropertyVisibility PropertyVisibility;
	TSharedPtr<SWidget> ToolbarWidget;
	TSharedPtr<SWidget> SettingsWidget;
};

} // namespace
