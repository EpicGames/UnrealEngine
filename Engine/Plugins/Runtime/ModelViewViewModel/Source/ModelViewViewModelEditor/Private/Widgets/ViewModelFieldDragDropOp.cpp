// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/ViewModelFieldDragDropOp.h"
#include "CoreMinimal.h"
#include "UObject/Field.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Misc/Guid.h"
#include "ScopedTransaction.h"
#include "WidgetBlueprint.h"

#define LOCTEXT_NAMESPACE "ViewModelFieldDragDropOp"

namespace UE::MVVM
{
FViewModelFieldDragDropOp::FViewModelFieldDragDropOp()
	: WidgetBP(nullptr)
	, Transaction(nullptr)
{
}

void FViewModelFieldDragDropOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	if (!bDropWasHandled)
	{
		if (Transaction)
		{
			Transaction->Cancel();
		}
		Transaction = nullptr;
	}
}

/** Constructs a new drag/drop operation */
TSharedRef<FViewModelFieldDragDropOp> FViewModelFieldDragDropOp::New(TArray<FFieldVariant> InField, const FGuid& ViewModelId, UWidgetBlueprint* InWidgetBP)
{
	TSharedRef<FViewModelFieldDragDropOp> Operation = MakeShared<FViewModelFieldDragDropOp>();

	// Set the display text and the transaction name based on whether we're dragging a single or multiple widgets
	Operation->CurrentHoverText = Operation->DefaultHoverText = FText::FromName(InField[0].GetFName());
	Operation->Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("Designer_DragViewModelField", "Drag ViewModel Field"));
	Operation->DraggedField = InField;
	Operation->ViewModelId = ViewModelId;
	Operation->WidgetBP = InWidgetBP;

	Operation->Construct();
	return Operation;
}
} // namespace
#undef LOCTEXT_NAMESPACE