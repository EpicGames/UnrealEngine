// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Field.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Misc/Guid.h"

class FScopedTransaction;
class UWidgetBlueprint;

namespace UE::MVVM
{

class FViewModelFieldDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FViewModelFieldDragDropOp, FDecoratedDragDropOp)

	FViewModelFieldDragDropOp();
	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;
	static TSharedRef<FViewModelFieldDragDropOp> New(TArray<FFieldVariant> InField, const FGuid& ViewModelId, UWidgetBlueprint* InWidgetBP);

	TArray<FFieldVariant> DraggedField;
	FGuid ViewModelId;
	TWeakObjectPtr<UWidgetBlueprint> WidgetBP;
	TUniquePtr<FScopedTransaction> Transaction;
};

} // namespace