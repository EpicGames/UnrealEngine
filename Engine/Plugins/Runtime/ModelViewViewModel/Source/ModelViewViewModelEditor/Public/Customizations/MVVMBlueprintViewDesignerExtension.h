// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DesignerExtension.h"
#include "IHasDesignerExtensibility.h"

namespace UE::MVVM
{
class FBlueprintViewDesignerExtensionFactory : public IDesignerExtensionFactory, public TSharedFromThis<FBlueprintViewDesignerExtensionFactory>
{
public:
	FBlueprintViewDesignerExtensionFactory() = default;
	virtual ~FBlueprintViewDesignerExtensionFactory() = default;

	virtual TSharedRef<FDesignerExtension> CreateDesignerExtension() const override;
};

class FBlueprintViewDesignerExtension : public FDesignerExtension
{
public:
	virtual void PreviewContentChanged(TSharedRef<SWidget> NewContent) override;
};
}
