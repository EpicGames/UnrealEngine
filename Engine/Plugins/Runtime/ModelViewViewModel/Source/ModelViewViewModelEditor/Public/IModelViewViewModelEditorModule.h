// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Misc/NotNull.h"

class UObject;

class IModelViewViewModelEditorModule : public IModuleInterface
{
public:
	/**
	 * Pops open a new window with a standalone object editor in either read only or edit mode.
	 */
	virtual void OpenPopoutEditor(TNotNull<UObject*> InObject, bool bInReadyOnly = true) = 0;
};
