// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FMVVMEditorCommands : public TCommands<FMVVMEditorCommands>
{
public:
	FMVVMEditorCommands();
	void RegisterCommands() override;
	static const FMVVMEditorCommands& Get();

	/** Command to toggle MVVM drawer in widget editor */
	TSharedPtr<FUICommandInfo> ToggleMVVMDrawer;

	/** Pops out a standalone property editor window. */
	TSharedPtr<FUICommandInfo> PopoutEditor;
};

