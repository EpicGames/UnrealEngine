// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "MVVMBlueprintPin.h"
#include "Types/MVVMConversionFunctionValue.h"
#include "UObject/Package.h"

#include "MVVMEditorSubsystem.generated.h"

#define UE_API MODELVIEWVIEWMODELEDITOR_API

class UEdGraphPin;
class UMVVMBlueprintView;
enum class EMVVMBindingMode : uint8;
enum class EMVVMExecutionMode : uint8;
enum class EMVVMConditionOperation : uint8;
namespace UE::MVVM { struct FBindingSource; }
namespace UE::MVVM::ConversionFunctionLibrary { class FCollection; }
struct FMVVMAvailableBinding;
struct FMVVMBlueprintFunctionReference;
struct FMVVMBlueprintPropertyPath;
struct FMVVMBlueprintViewBinding;
template <typename T> class TSubclassOf;

class UEdGraph;
class UK2Node_CallFunction;
class UMVVMBlueprintViewEvent;
class UMVVMBlueprintViewCondition;
class UWidgetBlueprint;

/** */
UCLASS(MinimalAPI, DisplayName="Viewmodel Editor Subsystem")
class UMVVMEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API UMVVMBlueprintView* RequestView(UWidgetBlueprint* WidgetBlueprint) const;

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API UMVVMBlueprintView* GetView(const UWidgetBlueprint* WidgetBlueprint) const;

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API FGuid AddViewModel(UWidgetBlueprint* WidgetBlueprint, const UClass* ViewModel);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API FGuid AddInstancedViewModel(UWidgetBlueprint* WidgetBlueprint);
	
	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API void RemoveViewModel(UWidgetBlueprint* WidgetBlueprint, FName ViewModel);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API bool VerifyViewModelRename(UWidgetBlueprint* WidgetBlueprint, FName ViewModel, FName NewViewModel, FText& OutError);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API bool RenameViewModel(UWidgetBlueprint* WidgetBlueprint, FName ViewModel, FName NewViewModel, FText& OutError);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API bool ReparentViewModel(UWidgetBlueprint* WidgetBlueprint, FName ViewModel, const UClass* NewViewModel, FText& OutError);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API FMVVMBlueprintViewBinding& AddBinding(UWidgetBlueprint* WidgetBlueprint);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API void RemoveBinding(UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API UMVVMBlueprintViewEvent* AddEvent(UWidgetBlueprint* WidgetBlueprint);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API void RemoveEvent(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API UMVVMBlueprintViewCondition* AddCondition(UWidgetBlueprint* WidgetBlueprint);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API void RemoveCondition(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewCondition* Condition);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API TArray<FMVVMAvailableBinding> GetChildViewModels(TSubclassOf<UObject> Class, TSubclassOf<UObject> Accessor);

	UE_DEPRECATED(5.4, "SetSourceToDestinationConversionFunction with a UFunction is deprecated.")
	UE_API void SetSourceToDestinationConversionFunction(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const UFunction* ConversionFunction);
	UE_API void SetSourceToDestinationConversionFunction(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, FMVVMBlueprintFunctionReference ConversionFunction);
	UE_DEPRECATED(5.4, "SetDestinationToSourceConversionFunction  with a UFunction is deprecated.")
	UE_API void SetDestinationToSourceConversionFunction(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const UFunction* ConversionFunction);
	UE_API void SetDestinationToSourceConversionFunction(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, FMVVMBlueprintFunctionReference ConversionFunction);
	UE_API void SetDestinationPathForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, FMVVMBlueprintPropertyPath Field, bool bAllowEventConversion);
	UE_API void SetSourcePathForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, FMVVMBlueprintPropertyPath Field);
	UE_API void OverrideExecutionModeForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, EMVVMExecutionMode Mode);
	UE_API void ResetExecutionModeForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding);
	UE_API void SetBindingTypeForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, EMVVMBindingMode Type);
	UE_API void SetEnabledForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, bool bEnabled);
	UE_API void SetCompileForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, bool bCompile);
	UE_API void GenerateBindToDestinationPathsForBinding(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding);

	UE_API void SetEventPath(UMVVMBlueprintViewEvent* Event, FMVVMBlueprintPropertyPath PropertyPath, bool bRequestBindingConversion);
	UE_API void SetEventDestinationPath(UMVVMBlueprintViewEvent* Event, FMVVMBlueprintPropertyPath PropertyPath);
	UE_API void SetEventArgumentPath(UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& PinId, const FMVVMBlueprintPropertyPath& PropertyPath) const;
	UE_API void SetEnabledForEvent(UMVVMBlueprintViewEvent* Event, bool bEnabled);
	UE_API void SetCompileForEvent(UMVVMBlueprintViewEvent* Event, bool bCompile);

	UE_API void SetConditionPath(UMVVMBlueprintViewCondition* Condition, FMVVMBlueprintPropertyPath PropertyPath, bool bRequestBindingConversion);
	UE_API void SetConditionDestinationPath(UMVVMBlueprintViewCondition* Condition, FMVVMBlueprintPropertyPath PropertyPath);
	UE_API void SetConditionArgumentPath(UMVVMBlueprintViewCondition* Condition, const FMVVMBlueprintPinId& PinId, const FMVVMBlueprintPropertyPath& PropertyPath) const;
	UE_API void SetEnabledForCondition(UMVVMBlueprintViewCondition* Condition, bool bEnabled);
	UE_API void SetCompileForCondition(UMVVMBlueprintViewCondition* Condition, bool bCompile);
	UE_API void SetConditionOperation(UMVVMBlueprintViewCondition* Condition, EMVVMConditionOperation Operation);
	UE_API void SetConditionOperationValue(UMVVMBlueprintViewCondition* Condition, float Value);
	UE_API void SetConditionOperationMaxValue(UMVVMBlueprintViewCondition* Condition, float MaxValue);

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API bool IsValidConversionFunction(const UWidgetBlueprint* WidgetBlueprint, const UFunction* Function, const FMVVMBlueprintPropertyPath& Source, const FMVVMBlueprintPropertyPath& Destination) const;
	UE_API bool IsValidConversionFunction(const UWidgetBlueprint* WidgetBlueprint, const UFunction* Function, const FProperty* ExpectedArgumentType, const FProperty* ExptectedReturnType) const;
	UE_API bool IsValidConversionFunction(const UWidgetBlueprint* WidgetBlueprint, UE::MVVM::FConversionFunctionValue Function, const FProperty* ExpectedArgumentType, const FProperty* ExptectedReturnType) const;

	UE_API bool IsValidConversionNode(const UWidgetBlueprint* WidgetBlueprint, const TSubclassOf<UK2Node> Function, const FMVVMBlueprintPropertyPath& Source, const FMVVMBlueprintPropertyPath& Destination) const;
	UE_API bool IsValidConversionNode(const UWidgetBlueprint* WidgetBlueprint, const TSubclassOf<UK2Node> Function, const FProperty* ExpectedArgumentType, const FProperty* ExptectedReturnType) const;

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API bool IsSimpleConversionFunction(const UFunction* Function) const;

	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API UEdGraph* GetConversionFunctionGraph(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, bool bSourceToDestination) const;

	UE_DEPRECATED(5.4, "GetConversionFunction was moved to MVVMBlueprintViewConversionFunction.")
	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API UFunction* GetConversionFunction(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, bool bSourceToDestination) const;

	UE_DEPRECATED(5.3, "GetConversionFunctionNode was moved to MVVMBlueprintViewConversionFunction.")
	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API UK2Node_CallFunction* GetConversionFunctionNode(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, bool bSourceToDestination) const;

	UE_DEPRECATED(5.5, "GetAvailableConversionFunctions return value changes.")
	UFUNCTION(BlueprintCallable, Category = "Viewmodel")
	UE_API TArray<UFunction*> GetAvailableConversionFunctions(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintPropertyPath& Source, const FMVVMBlueprintPropertyPath& Destination) const;

	UE_API TArray<UE::MVVM::FConversionFunctionValue> GetConversionFunctions(const UWidgetBlueprint* WidgetBlueprint, const FProperty* ExpectedArgumentType, const FProperty* ExptectedReturnType) const;
	UE_API TArray<FMVVMBlueprintPinId> GetConversionFunctionPins(const UWidgetBlueprint* WidgetBlueprint, const UE::MVVM::FConversionFunctionValue& Function, const FProperty* ExpectedArgumentType) const;

	UE_API FMVVMBlueprintPropertyPath GetPathForConversionFunctionArgument(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& PinId, bool bSourceToDestination) const;
	UE_API void SetPathForConversionFunctionArgument(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& PinId, const FMVVMBlueprintPropertyPath& Path, bool bSourceToDestination) const;
	UE_API UEdGraphPin* GetConversionFunctionArgumentPin(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& PinId, bool bSourceToDestination) const;

	UE_API void SplitPin(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& PinId, bool bSourceToDestination) const;
	UE_API bool CanSplitPin(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& PinId, bool bSourceToDestination) const;
	UE_API void SplitPin(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& PinId) const;
	UE_API bool CanSplitPin(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& PinId) const;
	UE_API void RecombinePin(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& PinId, bool bSourceToDestination) const;
	UE_API bool CanRecombinePin(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& PinId, bool bSourceToDestination) const;
	UE_API void RecombinePin(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& PinId) const;
	UE_API bool CanRecombinePin(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& PinId) const;
	UE_API void ResetPinToDefaultValue(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding,const FMVVMBlueprintPinId& PinId, bool bSourceToDestination) const;
	UE_API bool CanResetPinToDefaultValue(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& PinId, bool bSourceToDestination) const;
	UE_API void ResetPinToDefaultValue(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& PinId) const;
	UE_API bool CanResetPinToDefaultValue(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& PinId) const;
	UE_API void ResetOrphanedPin(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& PinId, bool bSourceToDestination) const;
	UE_API bool CanResetOrphanedPin(const UWidgetBlueprint* WidgetBlueprint, const FMVVMBlueprintViewBinding& Binding, const FMVVMBlueprintPinId& PinId, bool bSourceToDestination) const;
	UE_API void ResetOrphanedPin(UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event,const FMVVMBlueprintPinId& PinId) const;
	UE_API bool CanResetOrphanedPin(const UWidgetBlueprint* WidgetBlueprint, UMVVMBlueprintViewEvent* Event, const FMVVMBlueprintPinId& PinId) const;

	UE_API TArray<UE::MVVM::FBindingSource> GetBindableWidgets(const UWidgetBlueprint* WidgetBlueprint) const;
	UE_API TArray<UE::MVVM::FBindingSource> GetAllViewModels(const UWidgetBlueprint* WidgetBlueprint) const;

	UE_API FGuid GetFirstBindingThatUsesViewModel(const UWidgetBlueprint* WidgetBlueprint, FGuid ViewModelId) const;
	static UE_API FString GetDefaultViewModelName(const UClass* ViewModelClass);

private:
	mutable TUniquePtr<UE::MVVM::ConversionFunctionLibrary::FCollection> ConversionFunctionCollection;
};

#undef UE_API
