// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"
#include "Templates/SubclassOf.h"

#define UE_API MODELVIEWVIEWMODELEDITOR_API

class UFunction;
class UK2Node;

namespace UE::MVVM
{
/** */
struct FConversionFunctionValue
{
public:

	FConversionFunctionValue() = default;
	explicit FConversionFunctionValue(const UFunction* Function)
		: ConversionFunction(Function)
	{}
	explicit FConversionFunctionValue(TSubclassOf<UK2Node> Node)
		: ConversionNode(Node)
	{}

public:
	bool IsFunction() const
	{
		return ConversionFunction != nullptr;
	}

	const UFunction* GetFunction() const
	{
		return ConversionFunction;
	}

	bool IsNode() const
	{
		return ConversionNode.Get() != nullptr;
	}

	TSubclassOf<UK2Node> GetNode() const
	{
		return ConversionNode;
	}

	UE_API FString GetName() const;
	UE_API FName GetFName() const;
	UE_API FString GetFullGroupName(bool bStartWithOuter) const;
	UE_API FText GetDisplayName() const;
	UE_API FText GetTooltip() const;
	UE_API FText GetCategory() const;
	UE_API TArray<FString> GetSearchKeywords() const;

public:
	bool IsValid() const
	{
		return ConversionFunction != nullptr || ConversionNode.Get() != nullptr;
	}

	bool operator== (const FConversionFunctionValue& Other) const
	{
		return ConversionNode == Other.ConversionNode
			&& ConversionFunction == Other.ConversionFunction;
	}
	bool operator== (const UFunction* Other) const
	{
		return ConversionNode.Get() == nullptr
			&& ConversionFunction == Other;
	}
	bool operator== (const TSubclassOf<UK2Node> Other) const
	{
		return ConversionNode == Other
			&& ConversionFunction == nullptr;
	}

	friend int32 GetTypeHash(const FConversionFunctionValue& Value)
	{
		return HashCombine(GetTypeHash(Value.ConversionNode), GetTypeHash(Value.ConversionFunction));
	}

private:
	const UFunction* ConversionFunction = nullptr;
	TSubclassOf<UK2Node> ConversionNode;
};

} // namespace UE::MVVM

#undef UE_API
