// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMBlueprintFunctionReference.h"
#include "MVVMPropertyPath.h"
#include "Types/MVVMConversionFunctionValue.h"
#include "Templates/SubclassOf.h"

#include "MVVMLinkedPinValue.generated.h"

#define UE_API MODELVIEWVIEWMODELEDITOR_API

class UFunction;
class UK2Node;

/** */
USTRUCT(BlueprintType)
struct FMVVMLinkedPinValue
{
public:
	GENERATED_BODY()

	FMVVMLinkedPinValue() = default;
	UE_API explicit FMVVMLinkedPinValue(FMVVMBlueprintPropertyPath InPath);
	UE_API explicit FMVVMLinkedPinValue(const UBlueprint* InBlueprint, FMVVMBlueprintFunctionReference InConversion);
	UE_API explicit FMVVMLinkedPinValue(UE::MVVM::FConversionFunctionValue Function);
	UE_API explicit FMVVMLinkedPinValue(const UFunction* Function);
	UE_API explicit FMVVMLinkedPinValue(TSubclassOf<UK2Node> Node);

public:
	UE_API bool IsPropertyPath() const;
	UE_API const FMVVMBlueprintPropertyPath& GetPropertyPath() const;

	UE_API bool IsConversionFunction() const;
	UE_API const UFunction* GetConversionFunction() const;

	UE_API bool IsConversionNode() const;
	UE_API TSubclassOf<UK2Node> GetConversionNode() const;

public:
	UE_API bool IsValid() const;
	bool operator== (const FMVVMLinkedPinValue& Other) const
	{
		return ConversionNode == Other.ConversionNode && ConversionFunction == Other.ConversionFunction && PropertyPath == Other.PropertyPath;
	}

protected:
	UPROPERTY(BlueprintReadWrite, Category = "MVVM")
	FMVVMBlueprintPropertyPath PropertyPath;

	UPROPERTY(BlueprintReadWrite, Category = "MVVM")
	TObjectPtr<const UFunction> ConversionFunction = nullptr;

	UPROPERTY(BlueprintReadWrite, Category = "MVVM")
	TSubclassOf<UK2Node> ConversionNode;
};

#undef UE_API
